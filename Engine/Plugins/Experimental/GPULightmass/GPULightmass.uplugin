{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "GPU Lightmass",
	"Description" : "Static lighting building & previewing system using DXR",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "GPULightmass",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "PostConfigInit",
			"PlatformAllowList": [
				"Win64"
			]
		},		
		{
			"Name" : "GPULightmassEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default",
			"PlatformAllowList": [
				"Win64"
			]
		}
	]
}
