// Copyright Epic Games, Inc. All Rights Reserved.

struct FGPUTileDescription
{
	int2 LightmapSize;
	int2 VirtualTilePosition;
	int2 WorkingSetPosition;
	int2 ScratchPosition;
	int2 OutputLayer0Position;
	int2 OutputLayer1Position;
	int2 OutputLayer2Position;
	int2 OutputLayer3Position;
	int FrameIndex;
	int RenderPassIndex;
};
 
StructuredBuffer<FGPUTileDescription> BatchedTiles;
