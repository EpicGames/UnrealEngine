// Copyright Epic Games, Inc. All Rights Reserved.
#define Resolution 16

#define BRICK_NOT_ALLOCATED 0
#define BRICK_ALLOCATED 1
#define BRICK_DILATED 2
#define BRICK_IN_IMPORTANCE_VOLUME 3

uint3 ComputeBrickLayoutPosition(int BrickLayoutAllocation, uint3 BrickLayoutDimensions)
{
	uint3 BrickPosition = uint3(
		(BrickLayoutAllocation % BrickLayoutDimensions.x),
		(BrickLayoutAllocation / BrickLayoutDimensions.x) % BrickLayoutDimensions.y,
		(BrickLayoutAllocation / BrickLayoutDimensions.x) / BrickLayoutDimensions.y);

	return BrickPosition;
}

uint ComputeBrickLinearAddress(uint3 BrickPosition, uint3 BrickLayoutDimensions)
{
	return BrickPosition.x + BrickPosition.y * BrickLayoutDimensions.x + BrickPosition.z * BrickLayoutDimensions.x * BrickLayoutDimensions.y;
}

RWBuffer<int> BrickAllocatorParameters;
#define NumBricksRequested BrickAllocatorParameters[0]
#define NumBricksAllocated BrickAllocatorParameters[1]
