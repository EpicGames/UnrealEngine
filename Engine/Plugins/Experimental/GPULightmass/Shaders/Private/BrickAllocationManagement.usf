// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BrickAllocationManagement.usf
=============================================================================*/

#include "/Engine/Public/Platform.ush" 
#include "BrickAllocationDefs.ush"
#include "/Engine/Generated/UniformBuffers/VLMVoxelizationParams.ush"
#include "/Engine/Private/MortonCode.ush"
 
int3 VolumeSize;
RWTexture3D<uint> VoxelizeVolume;
int BrickSize;
int MipLevel;
int bIsHighestMip;

[numthreads(4, 4, 4)]
void ClearVolumeCS(uint3 VoxelPos : SV_DispatchThreadID) 
{
	if (any(VoxelPos >= VolumeSize)) return; 
	VoxelizeVolume[VoxelPos] = BRICK_NOT_ALLOCATED;
}

float3 ImportanceVolumeMin;
float3 ImportanceVolumeMax;

[numthreads(4, 4, 4)]
void VoxelizeImportanceVolumeCS(uint3 VoxelPos : SV_DispatchThreadID) 
{
	if (any(VoxelPos >= VolumeSize)) return; 
	float3 WorldPosition = ((float3)VoxelPos / VLMVoxelizationParams.VolumeMaxDim - 0.5f) * 2.0f * VLMVoxelizationParams.VolumeExtent.xyz + VLMVoxelizationParams.VolumeCenter.xyz;
	if (all(WorldPosition >= ImportanceVolumeMin) && all(WorldPosition <= ImportanceVolumeMax))
	{
		VoxelizeVolume[VoxelPos] = BRICK_IN_IMPORTANCE_VOLUME;
	}
}

RWTexture3D<uint> VoxelizeVolumePrevMip;

[numthreads(4, 4, 4)]
void DownsampleVolumeCS(
	uint3 VoxelPosPrevMip : SV_DispatchThreadID,
	uint3 VoxelPos : SV_GroupID
	) 
{
	// We have 2 waves in each group, any of them evaluating to true will resulting in true
	if (WaveActiveAnyTrue((VoxelizeVolumePrevMip[VoxelPosPrevMip] == BRICK_ALLOCATED || VoxelizeVolumePrevMip[VoxelPosPrevMip] == BRICK_DILATED) || bIsHighestMip == 1))
	{
		VoxelizeVolume[VoxelPos] = BRICK_ALLOCATED;
	}
}

RWTexture3D<uint4> IndirectionTexture;

RWBuffer<uint4> BrickRequests;
RWBuffer<uint4> BrickRequestsUnsorted;
RWBuffer<uint> BrickRequestKeys;
RWBuffer<uint> BrickRequestSortedIndices;
RWBuffer<uint> BrickRequestSortedIndicesInverse;

[numthreads(4, 4, 4)]
void CountNumBricksCS(
	uint3 VoxelPos : SV_DispatchThreadID
	) 
{
	if (any(VoxelPos >= VolumeSize)) return;
	
	if (VoxelizeVolume[VoxelPos] == BRICK_IN_IMPORTANCE_VOLUME)
	{
		VoxelizeVolume[VoxelPos] = BRICK_NOT_ALLOCATED;
	}
	
	if (VoxelizeVolume[VoxelPos] == BRICK_ALLOCATED || VoxelizeVolume[VoxelPos] == BRICK_DILATED)
	{
		int BrickLinearAddress;
		InterlockedAdd(NumBricksRequested, 1, BrickLinearAddress);
		VoxelizeVolume[VoxelPos] = 1 + BrickLinearAddress;
	}
}

[numthreads(4, 4, 4)]
void GatherBrickRequestsCS(
	uint3 VoxelPos : SV_DispatchThreadID
	) 
{
	if (any(VoxelPos >= VolumeSize)) return; 
	if (VoxelizeVolume[VoxelPos])
	{
		int LinearIndex = VoxelizeVolume[VoxelPos] - 1;
		BrickRequestKeys[LinearIndex] = ((2 - MipLevel) << 30) | MortonCode3(VoxelPos.x) | (MortonCode3(VoxelPos.y) << 1) | (MortonCode3(VoxelPos.z) << 2);
		BrickRequestSortedIndices[LinearIndex] = LinearIndex;
		BrickRequestsUnsorted[LinearIndex] = uint4(VoxelPos, BrickSize);
	}
}

[numthreads(4, 4, 4)]
void SplatVolumeCS(
	uint3 VoxelPos : SV_DispatchThreadID
	) 
{
	if (any(VoxelPos >= VolumeSize)) return; 
	if (VoxelizeVolume[VoxelPos / BrickSize])
	{
		IndirectionTexture[VoxelPos] = uint4(ComputeBrickLayoutPosition(VoxelizeVolume[VoxelPos / BrickSize] - 1, uint3(256, 256, 256)), BrickSize);
	}
	else
	{
		if (bIsHighestMip)
		{
			IndirectionTexture[VoxelPos] = uint4(0, 0, 0, 0);
		}
	}
}

int NumTotalBricks;

[numthreads(64, 1, 1)]
void WriteSortedBrickRequestsCS(uint3 RequestIndex : SV_DispatchThreadID)
{
	if (RequestIndex.x >= NumTotalBricks) return;
	BrickRequests[RequestIndex.x] = BrickRequestsUnsorted[BrickRequestSortedIndices[RequestIndex.x]];
	BrickRequestSortedIndicesInverse[BrickRequestSortedIndices[RequestIndex.x]] = RequestIndex.x;
}

[numthreads(4, 4, 4)]
void PermuteVoxelizeVolumeCS(
	uint3 VoxelPos : SV_DispatchThreadID
	) 
{
	if (any(VoxelPos >= VolumeSize)) return; 
	if (VoxelizeVolume[VoxelPos])
	{
		int LinearIndex = VoxelizeVolume[VoxelPos] - 1;
		VoxelizeVolume[VoxelPos] = BrickRequestSortedIndicesInverse[LinearIndex] + 1;
	}
}

int3 IndirectionTextureDim;
int3 BrickDataDimensions;

Texture3D<float4> AmbientVector;
Texture3D<float4> SHCoefficients0R;
Texture3D<float4> SHCoefficients1R;
Texture3D<float4> SHCoefficients0G;
Texture3D<float4> SHCoefficients1G;
Texture3D<float4> SHCoefficients0B;
Texture3D<float4> SHCoefficients1B;
Texture3D<float4> SkyBentNormal;
Texture3D<UNORM float> DirectionalLightShadowing;

RWTexture3D<float3> OutAmbientVector;
RWTexture3D<UNORM float4> OutSHCoefficients0R;
RWTexture3D<UNORM float4> OutSHCoefficients1R;
RWTexture3D<UNORM float4> OutSHCoefficients0G;
RWTexture3D<UNORM float4> OutSHCoefficients1G;
RWTexture3D<UNORM float4> OutSHCoefficients0B;
RWTexture3D<UNORM float4> OutSHCoefficients1B;
RWTexture3D<UNORM float4> OutSkyBentNormal;
RWTexture3D<UNORM float> OutDirectionalLightShadowing;

uint FrameNumber;

float3 DilateWithValidityMask(RWTexture3D<float3> Texture, RWTexture3D<uint> CoverageMask, float3 UVW)
{
	float AccumulatedWeight = 0.0f;
	float3 Value = float3(0, 0, 0);
	
	float BrickSize;
	const int PaddedBrickSize = 5;
	
	{
		float3 IndirectionTextureUVW = UVW;
		int3 IndirectionTextureCoordinates = floor(IndirectionTextureUVW);
		uint4 IndirectionTextureValue = IndirectionTexture[IndirectionTextureCoordinates];
		BrickSize = IndirectionTextureValue.w;
	}
	
	for (int z = -1; z <= 1; z++)
	{
		for (int y = -1; y <= 1; y++)
		{
			for (int x = -1; x <= 1; x++)
			{
				float3 IndirectionTextureUVW = UVW + int3(x, y, z) / BrickSize / 4.0f;
				int3 IndirectionTextureCoordinates = floor(IndirectionTextureUVW);
				uint4 IndirectionTextureValue = IndirectionTexture[IndirectionTextureCoordinates];
				
				int3 BrickOrigin = (IndirectionTextureCoordinates / IndirectionTextureValue.w) * IndirectionTextureValue.w; // Integer Div
				float3 BrickUVW = IndirectionTextureValue.xyz * PaddedBrickSize + frac(IndirectionTextureUVW / IndirectionTextureValue.w) * 4;
			
				int3 Coord = BrickUVW;
				if (CoverageMask[Coord] == 1)
				{
					Value += Texture[Coord];
					AccumulatedWeight += 1;
				}
			}
		}
	}
	
	return Value / max(AccumulatedWeight, 0.00001f);
}

UNORM float4 DilateWithValidityMask(RWTexture3D<UNORM float4> Texture, RWTexture3D<uint> CoverageMask, float3 UVW)
{
	int3 Coord = UVW;
	float3 CoordinateFraction = frac(UVW);
	float AccumulatedWeight = 0.0f;
	float4 Value = float4(0, 0, 0, 0);
	
	float BrickSize;
	const int PaddedBrickSize = 5;
	
	{
		float3 IndirectionTextureUVW = UVW;
		int3 IndirectionTextureCoordinates = floor(IndirectionTextureUVW);
		uint4 IndirectionTextureValue = IndirectionTexture[IndirectionTextureCoordinates];
		BrickSize = IndirectionTextureValue.w;
	}
	
	for (int z = -1; z <= 1; z++)
	{
		for (int y = -1; y <= 1; y++)
		{
			for (int x = -1; x <= 1; x++)
			{
				float3 IndirectionTextureUVW = UVW + int3(x, y, z) / BrickSize / 4.0f;
				int3 IndirectionTextureCoordinates = floor(IndirectionTextureUVW);
				uint4 IndirectionTextureValue = IndirectionTexture[IndirectionTextureCoordinates];
				
				int3 BrickOrigin = (IndirectionTextureCoordinates / IndirectionTextureValue.w) * IndirectionTextureValue.w; // Integer Div
				float3 BrickUVW = IndirectionTextureValue.xyz * PaddedBrickSize + frac(IndirectionTextureUVW / IndirectionTextureValue.w) * 4;
			
				int3 Coord = BrickUVW;
				if (CoverageMask[Coord] == 1)
				{
					Value += Texture[Coord];
					AccumulatedWeight += 1;
				}
			}
		}
	}
	
	return Value / max(AccumulatedWeight, 0.00001f);
}

UNORM float DilateWithValidityMask(RWTexture3D<UNORM float> Texture, RWTexture3D<uint> CoverageMask, float3 UVW)
{
	int3 Coord = UVW;
	float3 CoordinateFraction = frac(UVW);
	float AccumulatedWeight = 0.0f;
	float Value = 0;
	
	float BrickSize;
	const int PaddedBrickSize = 5;
	
	{
		float3 IndirectionTextureUVW = UVW;
		int3 IndirectionTextureCoordinates = floor(IndirectionTextureUVW);
		uint4 IndirectionTextureValue = IndirectionTexture[IndirectionTextureCoordinates];
		BrickSize = IndirectionTextureValue.w;
	}
	
	for (int z = -1; z <= 1; z++)
	{
		for (int y = -1; y <= 1; y++)
		{
			for (int x = -1; x <= 1; x++)
			{
				float3 IndirectionTextureUVW = UVW + int3(x, y, z) / BrickSize / 4.0f;
				int3 IndirectionTextureCoordinates = floor(IndirectionTextureUVW);
				uint4 IndirectionTextureValue = IndirectionTexture[IndirectionTextureCoordinates];
				
				int3 BrickOrigin = (IndirectionTextureCoordinates / IndirectionTextureValue.w) * IndirectionTextureValue.w; // Integer Div
				float3 BrickUVW = IndirectionTextureValue.xyz * PaddedBrickSize + frac(IndirectionTextureUVW / IndirectionTextureValue.w) * 4;
			
				int3 Coord = BrickUVW;
				if (CoverageMask[Coord] == 1)
				{
					Value += Texture[Coord];
					AccumulatedWeight += 1;
				}
			}
		}
	}
	
	return Value / max(AccumulatedWeight, 0.00001f);
}

float3 ManualTrilinearFilter(RWTexture3D<float3> Texture, float3 UVW)
{
	int3 Coord = UVW;
	float3 CoordinateFraction = frac(UVW);
	float3 Weight;
	float3 Value = float3(0, 0, 0);
	
	for (int z = 0; z <= 1; z++)
	{
		Weight.z = (z == 0 ? 1 - CoordinateFraction.z : CoordinateFraction.z);
		for (int y = 0; y <= 1; y++)
		{
			Weight.y = (y == 0 ? 1 - CoordinateFraction.y : CoordinateFraction.y);
			for (int x = 0; x <= 1; x++)
			{
				Weight.x = (x == 0 ? 1 - CoordinateFraction.x : CoordinateFraction.x);
				Value += Texture[Coord + int3(x, y, z)] * Weight.x * Weight.y * Weight.z;
			}
		}
	}
	
	return Value;
}

UNORM float4 ManualTrilinearFilter(RWTexture3D<UNORM float4> Texture, float3 UVW)
{
	int3 Coord = UVW;
	float3 CoordinateFraction = frac(UVW);
	float3 Weight;
	float4 Value = float4(0, 0, 0, 0);
	
	for (int z = 0; z <= 1; z++)
	{
		Weight.z = (z == 0 ? 1 - CoordinateFraction.z : CoordinateFraction.z);
		for (int y = 0; y <= 1; y++)
		{
			Weight.y = (y == 0 ? 1 - CoordinateFraction.y : CoordinateFraction.y);
			for (int x = 0; x <= 1; x++)
			{
				Weight.x = (x == 0 ? 1 - CoordinateFraction.x : CoordinateFraction.x);
				Value += Texture[Coord + int3(x, y, z)] * Weight.x * Weight.y * Weight.z;
			}
		}
	}
	
	return Value;
}

float ManualTrilinearFilter(RWTexture3D<UNORM float> Texture, float3 UVW)
{
	int3 Coord = UVW;
	float3 CoordinateFraction = frac(UVW);
	float3 Weight;
	float Value = 0;
	
	for (int z = 0; z <= 1; z++)
	{
		Weight.z = (z == 0 ? 1 - CoordinateFraction.z : CoordinateFraction.z);
		for (int y = 0; y <= 1; y++)
		{
			Weight.y = (y == 0 ? 1 - CoordinateFraction.y : CoordinateFraction.y);
			for (int x = 0; x <= 1; x++)
			{
				Weight.x = (x == 0 ? 1 - CoordinateFraction.x : CoordinateFraction.x);
				Value += Texture[Coord + int3(x, y, z)] * Weight.x * Weight.y * Weight.z;
			}
		}
	}
	
	return Value;
}

int BrickBatchOffset;
RWTexture3D<uint> ValidityMask;

[numthreads(5, 5, 5)]
void CopyPaddingStripsCS(uint3 BrickID : SV_GroupID, uint3 CellPosInBrick : SV_GroupThreadID)
{
	uint BrickIndex = BrickID.x + BrickBatchOffset;
	if (BrickIndex >= NumTotalBricks) return;	
	int3 CellPosInCurrentMip = BrickRequests[BrickIndex].xyz * 4 + CellPosInBrick;	
	float3 InvCellSizeInCurrentMip = float3(1, 1, 1) / 4.0f;
	float PaddedBrickSize = 4 + 1;
	uint3 VoxelPos = ComputeBrickLayoutPosition(BrickIndex, uint3(256, 256, 256)) * PaddedBrickSize + CellPosInBrick;

	if (BrickRequests[BrickIndex].w == 1u << (2 * MipLevel))
	{
		int3 VoxelizeVolumeCoords = floor(CellPosInCurrentMip * InvCellSizeInCurrentMip);
		int BrickLinearAddress = VoxelizeVolume[VoxelizeVolumeCoords] - 1;

		if (BrickLinearAddress >= 0)
		{
			int3 BrickOrigin = ComputeBrickLayoutPosition(BrickLinearAddress, uint3(256, 256, 256));
			int3 BrickUVW = 5 * BrickOrigin + 4 * frac(CellPosInCurrentMip * InvCellSizeInCurrentMip);
		
			OutAmbientVector[VoxelPos] = OutAmbientVector[BrickUVW];
			OutSHCoefficients0R[VoxelPos] = OutSHCoefficients0R[BrickUVW];
			OutSHCoefficients1R[VoxelPos] = OutSHCoefficients1R[BrickUVW];
			OutSHCoefficients0G[VoxelPos] = OutSHCoefficients0G[BrickUVW];
			OutSHCoefficients1G[VoxelPos] = OutSHCoefficients1G[BrickUVW];
			OutSHCoefficients0B[VoxelPos] = OutSHCoefficients0B[BrickUVW];
			OutSHCoefficients1B[VoxelPos] = OutSHCoefficients1B[BrickUVW];
			OutSkyBentNormal[VoxelPos] = OutSkyBentNormal[BrickUVW];
			OutDirectionalLightShadowing[VoxelPos] = OutDirectionalLightShadowing[BrickUVW];
			ValidityMask[VoxelPos] = ValidityMask[BrickUVW];
		}
	}
}

[numthreads(5, 5, 5)]
void FillInvalidCellCS(uint3 BrickID : SV_GroupID, uint3 CellPosInBrick : SV_GroupThreadID)
{
	uint BrickIndex = BrickID.x + BrickBatchOffset;
	if (BrickIndex >= NumTotalBricks) return;	
	int3 CellPosInVLM = (BrickRequests[BrickIndex].xyz * 4 + CellPosInBrick) * BrickRequests[BrickIndex].w;	
	float3 InvCellSizeInVLM = float3(1, 1, 1) / 4.0f;
	float PaddedBrickSize = 4 + 1;
	uint3 VoxelPos = ComputeBrickLayoutPosition(BrickIndex, uint3(256, 256, 256)) * PaddedBrickSize + CellPosInBrick;

	if (ValidityMask[VoxelPos] == 0)
	{
		float3 IndirectionTextureUVW = CellPosInVLM * InvCellSizeInVLM;
		OutAmbientVector[VoxelPos] = DilateWithValidityMask(OutAmbientVector, ValidityMask, IndirectionTextureUVW);
		OutSHCoefficients0R[VoxelPos] = DilateWithValidityMask(OutSHCoefficients0R, ValidityMask, IndirectionTextureUVW);
		OutSHCoefficients1R[VoxelPos] = DilateWithValidityMask(OutSHCoefficients1R, ValidityMask, IndirectionTextureUVW);
		OutSHCoefficients0G[VoxelPos] = DilateWithValidityMask(OutSHCoefficients0G, ValidityMask, IndirectionTextureUVW);
		OutSHCoefficients1G[VoxelPos] = DilateWithValidityMask(OutSHCoefficients1G, ValidityMask, IndirectionTextureUVW);
		OutSHCoefficients0B[VoxelPos] = DilateWithValidityMask(OutSHCoefficients0B, ValidityMask, IndirectionTextureUVW);
		OutSHCoefficients1B[VoxelPos] = DilateWithValidityMask(OutSHCoefficients1B, ValidityMask, IndirectionTextureUVW);
		OutSkyBentNormal[VoxelPos] = DilateWithValidityMask(OutSkyBentNormal, ValidityMask, IndirectionTextureUVW);
		OutDirectionalLightShadowing[VoxelPos] = DilateWithValidityMask(OutDirectionalLightShadowing, ValidityMask, IndirectionTextureUVW);
	}
}

[numthreads(5, 5, 5)]
void StitchBorderCS(uint3 BrickID : SV_GroupID, uint3 CellPosInBrick : SV_GroupThreadID)
{
	uint BrickIndex = BrickID.x + BrickBatchOffset;
	if (BrickIndex >= NumTotalBricks) return;	
	int3 CellPosInVLM = (BrickRequests[BrickIndex].xyz * 4 + CellPosInBrick) * BrickRequests[BrickIndex].w;	
	float3 InvCellSizeInVLM = float3(1, 1, 1) / 4.0f;
	float PaddedBrickSize = 4 + 1;
	uint3 VoxelPos = ComputeBrickLayoutPosition(BrickIndex, uint3(256, 256, 256)) * PaddedBrickSize + CellPosInBrick;

	for (int dx = -1; dx <= 1; dx++)
	{
		for (int dy = -1; dy <= 1; dy++)
		{
			for (int dz = -1; dz <= 1; dz++)
			{
				int3 IndirectionTextureCoordinates = floor(CellPosInVLM * InvCellSizeInVLM + InvCellSizeInVLM * 0.5f * int3(dx, dy, dz));
				uint4 IndirectionTextureValue = IndirectionTexture[IndirectionTextureCoordinates];
				if (IndirectionTextureValue.w > BrickRequests[BrickIndex].w)
				{
					int3 BrickOrigin = (IndirectionTextureCoordinates / IndirectionTextureValue.w) * IndirectionTextureValue.w; // Integer Div
					float3 BrickUVW = (IndirectionTextureValue.xyz * PaddedBrickSize + saturate((CellPosInVLM * InvCellSizeInVLM - BrickOrigin) / IndirectionTextureValue.w) * 4);
					
					   OutAmbientVector[VoxelPos] = ManualTrilinearFilter(OutAmbientVector, BrickUVW);
					OutSHCoefficients0R[VoxelPos] = ManualTrilinearFilter(OutSHCoefficients0R, BrickUVW);
					OutSHCoefficients1R[VoxelPos] = ManualTrilinearFilter(OutSHCoefficients1R, BrickUVW);
					OutSHCoefficients0G[VoxelPos] = ManualTrilinearFilter(OutSHCoefficients0G, BrickUVW);
					OutSHCoefficients1G[VoxelPos] = ManualTrilinearFilter(OutSHCoefficients1G, BrickUVW);
					OutSHCoefficients0B[VoxelPos] = ManualTrilinearFilter(OutSHCoefficients0B, BrickUVW);
					OutSHCoefficients1B[VoxelPos] = ManualTrilinearFilter(OutSHCoefficients1B, BrickUVW);
					   OutSkyBentNormal[VoxelPos] = ManualTrilinearFilter(OutSkyBentNormal, BrickUVW);
					OutDirectionalLightShadowing[VoxelPos] = ManualTrilinearFilter(OutDirectionalLightShadowing, BrickUVW);
				}
			}
		}
	}
}

int NumTotalPassesToRender;

[numthreads(5, 5, 5)]
void FinalizeBrickResultsCS(uint3 BrickID : SV_GroupID, uint3 CellPosInBrick : SV_GroupThreadID)
{
	uint BrickIndex = BrickID.x + BrickBatchOffset;
	if (BrickIndex >= NumTotalBricks) return;	
	int3 CellPosInVLM = (BrickRequests[BrickIndex].xyz * 4 + CellPosInBrick) * BrickRequests[BrickIndex].w;	
	float3 InvCellSizeInVLM = float3(1, 1, 1) / 4.0f;
	float PaddedBrickSize = 4 + 1;
	uint3 VoxelPos = ComputeBrickLayoutPosition(BrickID.x, uint3(256, 256, 256)) * PaddedBrickSize + CellPosInBrick;
	uint3 OutVoxelPos = ComputeBrickLayoutPosition(BrickIndex, uint3(256, 256, 256)) * PaddedBrickSize + CellPosInBrick;

	uint SampleCount = asuint(AmbientVector[VoxelPos].w);
	if (SampleCount > 0)
	{
		OutAmbientVector[OutVoxelPos] = AmbientVector[VoxelPos].xyz / SampleCount;
		
		float3 InvAmbient = float3(1.0f, 1.0f, 1.0f) / max(AmbientVector[VoxelPos].xyz / SampleCount, float3(0.0001f, 0.0001f, 0.0001f));
		
		float4 CoefficientNormalizationScale0 = float4(
			0.282095f / 0.488603f,
			0.282095f / 0.488603f,
			0.282095f / 0.488603f,
			0.282095f / 1.092548f);

		float4 CoefficientNormalizationScale1 = float4(
			0.282095f / 1.092548f,
			0.282095f / (4.0f * 0.315392f),
			0.282095f / 1.092548f,
			0.282095f / (2.0f * 0.546274f));
	
		OutSHCoefficients0R[OutVoxelPos] = (SHCoefficients0R[VoxelPos] / SampleCount) * InvAmbient.r * CoefficientNormalizationScale0 * 0.5f + 0.5f;
		OutSHCoefficients1R[OutVoxelPos] = (SHCoefficients1R[VoxelPos] / SampleCount) * InvAmbient.r * CoefficientNormalizationScale1 * 0.5f + 0.5f;
		OutSHCoefficients0G[OutVoxelPos] = (SHCoefficients0G[VoxelPos] / SampleCount) * InvAmbient.g * CoefficientNormalizationScale0 * 0.5f + 0.5f;
		OutSHCoefficients1G[OutVoxelPos] = (SHCoefficients1G[VoxelPos] / SampleCount) * InvAmbient.g * CoefficientNormalizationScale1 * 0.5f + 0.5f;
		OutSHCoefficients0B[OutVoxelPos] = (SHCoefficients0B[VoxelPos] / SampleCount) * InvAmbient.b * CoefficientNormalizationScale0 * 0.5f + 0.5f;
		OutSHCoefficients1B[OutVoxelPos] = (SHCoefficients1B[VoxelPos] / SampleCount) * InvAmbient.b * CoefficientNormalizationScale1 * 0.5f + 0.5f;
		OutSkyBentNormal[OutVoxelPos] = SkyBentNormal[VoxelPos] / SampleCount * 0.5f + 0.5f;
		OutDirectionalLightShadowing[OutVoxelPos] = DirectionalLightShadowing[VoxelPos];
	}

	ValidityMask[OutVoxelPos] = SampleCount > NumTotalPassesToRender / 3;
}
