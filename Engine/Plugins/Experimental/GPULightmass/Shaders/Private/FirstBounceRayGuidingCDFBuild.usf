// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

#include "/Engine/Private/PathTracing/Material/FirstBounceRayGuidingCommon.ush"

#include "BatchedTiles.ush"

int RayGuidingEndPassIndex;

RWTexture2D<uint> RayGuidingLuminance;
RWTexture2D<float> RayGuidingCDFX;
RWTexture2D<float> RayGuidingCDFY;

groupshared float RowSum[DIRECTIONAL_BINS_ONE_DIM];

[numthreads(32, 16, 1)]
void FirstBounceRayGuidingCDFBuildCS(
	uint3 ThreadId : SV_GroupThreadID,
	uint3 GroupID : SV_GroupID
	) 
{
	int TileIndex = GroupID.x / NUM_CLUSTERS_PER_TILE;
	int ClusterLinearIndexInTile = GroupID.x % NUM_CLUSTERS_PER_TILE;
	int2 ClusterPosition = int2(ClusterLinearIndexInTile % NUM_CLUSTERS_PER_DIM, ClusterLinearIndexInTile / NUM_CLUSTERS_PER_DIM);
	
	if (BatchedTiles[TileIndex].RenderPassIndex != RayGuidingEndPassIndex) return;
	
	if (ThreadId.x < DIRECTIONAL_BINS_ONE_DIM)
	{
		float Value = 0;
		
		int FilterKernelSize = 1;
		for(int dx = -FilterKernelSize; dx <= FilterKernelSize; dx++)
		{
			for(int dy = -FilterKernelSize; dy <= FilterKernelSize; dy++)
			{
				int2 FinalPosition = ClusterPosition * DIRECTIONAL_BINS_ONE_DIM + clamp(ThreadId.xy + int2(dx, dy), int2(0, 0), int2(DIRECTIONAL_BINS_ONE_DIM - 1, DIRECTIONAL_BINS_ONE_DIM - 1));
				Value += max(asfloat(RayGuidingLuminance[BatchedTiles[TileIndex].WorkingSetPosition / GPreviewLightmapPhysicalTileSize * CDF_TILE_SIZE + FinalPosition]) * ((dx != 0 || dy != 0) ? 0.125 : 1), 0.00001f);
			}
		}

		float PrefixSum = WavePrefixSum(Value) + Value;
		if (ThreadId.x == DIRECTIONAL_BINS_ONE_DIM - 1)
		{
			RowSum[ThreadId.y] = PrefixSum;
		}
		
		PrefixSum /= WaveReadLaneAt(PrefixSum, DIRECTIONAL_BINS_ONE_DIM - 1);
		int2 FinalPosition = ClusterPosition * DIRECTIONAL_BINS_ONE_DIM + ThreadId.xy + int2(0, 0);
		RayGuidingCDFX[BatchedTiles[TileIndex].WorkingSetPosition / GPreviewLightmapPhysicalTileSize * CDF_TILE_SIZE + FinalPosition] = PrefixSum;
	}
	
	GroupMemoryBarrierWithGroupSync();

	if (ThreadId.y == 0)
	{
		if (ThreadId.x < DIRECTIONAL_BINS_ONE_DIM)
		{
			float Value = RowSum[ThreadId.x];
			float PrefixSum = WavePrefixSum(Value) + Value;
			PrefixSum /= WaveReadLaneAt(PrefixSum, DIRECTIONAL_BINS_ONE_DIM - 1);
			
			int2 WritePos = int2(ThreadId.x % 4, ThreadId.x / 4);
			int2 FinalPosition = ClusterPosition * DIRECTIONAL_BINS_ONE_DIM / 4 + WritePos;
			RayGuidingCDFY[BatchedTiles[TileIndex].WorkingSetPosition / GPreviewLightmapPhysicalTileSize * CDF_TILE_SIZE / 4 + FinalPosition] = PrefixSum;
		}
	}
}