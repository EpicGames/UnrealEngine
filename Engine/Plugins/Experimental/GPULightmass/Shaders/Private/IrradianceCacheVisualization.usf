// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IrradianceCacheVisualization.usf
=============================================================================*/

// Needs to be defined before the uniform buffer struct header is included, very unfortunately
#include "/Engine/Private/Common.ush"
#include "/Engine/Private/DeferredShadingCommon.ush"
#include "IrradianceCachingCommon.ush"

void VisualizeIrradianceCachePS(
	in float4 UVAndScreenPos : TEXCOORD0, 
	in float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 ScreenUV = SvPositionToBufferUV(SVPos);
	float2 ScreenPosition = (ScreenUV.xy - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
	float SceneDepth = CalcSceneDepth(ScreenUV);
	float3 WorldPosition = mul(float4(ScreenPosition * SceneDepth, SceneDepth, 1), DFHackToFloat(PrimaryView.ScreenToWorld)).xyz;
	float3 ShadingNormal = GetGBufferData(ScreenUV).WorldNormal;
	
	bool bIrradianceQuerySuccessful = false;
	bool bGeometryQuerySuccessful = false;
			
	uint NearestRecordIndex = 0;
	float3 RecordIrradiance;

	uint Index;
	if (ICHashTableFind(EncodeICHashKey(WorldPosition, ShadingNormal, IrradianceCachingParameters.Spacing), Index))
	{
		uint RecordIndex = IrradianceCachingParameters.RWHashToIndex[Index];
		uint4 RecordIrradianceAndSampleCount = IrradianceCachingParameters.IrradianceCacheRecords[RecordIndex];
		uint BackfaceHitsCount = IrradianceCachingParameters.IrradianceCacheRecordBackfaceHits[RecordIndex].a;
		if (RecordIrradianceAndSampleCount.w > 0)
		{
			bIrradianceQuerySuccessful = true;
			
			RecordIrradiance = asfloat(RecordIrradianceAndSampleCount.xyz) / RecordIrradianceAndSampleCount.w;

			if (BackfaceHitsCount > RecordIrradianceAndSampleCount.w * BackfaceThresholdRejection)
			{
				RecordIrradiance = float3(1, 1, 0) / View.PreExposure;
			}

			if (BackfaceHitsCount > RecordIrradianceAndSampleCount.w * BackfaceThresholdInsideGeometry)
			{
				RecordIrradiance = float3(1, 0, 0) / View.PreExposure;
			}
		}
		
		bGeometryQuerySuccessful = true;
		NearestRecordIndex = RecordIndex;
	}
	
	if (bIrradianceQuerySuccessful)
	{		
		OutColor = float4(RecordIrradiance * View.PreExposure, 0);
	}
	else
	{
		discard;
	}
}
