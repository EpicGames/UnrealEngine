// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Private/HashTable.ush"

#ifndef IC_BACKFACE_DETECTION
#define IC_BACKFACE_DETECTION 0
#endif

#define BackfaceThresholdRejection (1.0f / 64)
#define BackfaceThresholdInsideGeometry (2.0f)

struct FFinalGatherHitPoint
{
	float3 WorldPosition;
	float3 WorldNormal;
};

uint ICKeyMix(uint2 Key)
{
	return MurmurMix(MurmurAdd(Key.x, Key.y));
}

bool ICHashTableAdd(uint2 Key, out uint Index)
{
	// Zero is reserved as invalid
	Key++;

	LOOP
	[allow_uav_condition]
	for(Index = ICKeyMix(Key); ; Index++)
	{
		Index = Index % IrradianceCachingParameters.HashTableSize;
		
		uint2 StoredKey = 0;

		bool bAtomicReadSuccessful = false;
		uint SemaphoreState;
		InterlockedCompareExchange(IrradianceCachingParameters.HashTableSemaphore[Index], 0, 1, SemaphoreState);
		if (SemaphoreState == 0 || SemaphoreState == 1)
		{
			StoredKey = IrradianceCachingParameters.RWHashTable[Index];
			bAtomicReadSuccessful = true;
			
			if (SemaphoreState == 0)
			{
				InterlockedCompareExchange(IrradianceCachingParameters.HashTableSemaphore[Index], 1, 0, SemaphoreState);
			}
		}

		if (!bAtomicReadSuccessful)
		{
			return false;
		}
		
		if(any(StoredKey != Key))
		{
			if(any(StoredKey != 0))
				continue;

			uint2 PrevKey = 0xffffffff;
			
			bool bAtomicWriteSuccessful = false;
			InterlockedCompareExchange(IrradianceCachingParameters.HashTableSemaphore[Index], 0, 2, SemaphoreState);
			if (SemaphoreState == 0)
			{
				bAtomicWriteSuccessful = true;
				PrevKey = IrradianceCachingParameters.RWHashTable[ Index ];
				if (all(PrevKey == 0))
				{
					IrradianceCachingParameters.RWHashTable[Index] = Key;
				}
				InterlockedCompareExchange(IrradianceCachingParameters.HashTableSemaphore[Index], 2, 0, SemaphoreState);
			}

			if (bAtomicWriteSuccessful)
			{
				if(all(PrevKey == 0))
					return true;
				else if(any(PrevKey != Key))
					continue;
			}
			else
			{
				return false;
			}
		}

		break;
	}

	return false;
}

// Returns true if key is found.
// Index output is the hash table bucket this key is stored in if found.
bool ICHashTableFind(uint2 Key, out uint Index)
{
	// Zero is reserved as invalid
	Key++;

	LOOP
	[allow_uav_condition]
	for(Index = ICKeyMix(Key); ; Index++)
	{
		Index = Index % IrradianceCachingParameters.HashTableSize;

		uint2 StoredKey = 0;

		bool bAtomicReadSuccessful = false;
		uint SemaphoreState;
		InterlockedCompareExchange(IrradianceCachingParameters.HashTableSemaphore[Index], 0, 1, SemaphoreState);
		if (SemaphoreState == 0 || SemaphoreState == 1)
		{
			StoredKey = IrradianceCachingParameters.RWHashTable[Index];
			bAtomicReadSuccessful = true;
			
			if (SemaphoreState == 0)
			{
				InterlockedCompareExchange(IrradianceCachingParameters.HashTableSemaphore[Index], 1, 0, SemaphoreState);
			}
		}
		
		if (!bAtomicReadSuccessful)
		{
			return false;
		}
		
		if(any(StoredKey != Key))
		{
			if(any(StoredKey != 0))
				continue;
		}
		else
		{
			return true;
		}

		break;
	}

	return false;
}

uint3 EncodeVoxelKey(float3 VoxelPos)
{
	int3 Signed = int3( VoxelPos ) + 0x3ffff;
	uint3 Voxel = uint3( Signed ) & 0x7ffff;

	uint3 Key;
	Key.x = Voxel.x;
	Key.y = Voxel.y;
	Key.z = Voxel.z;
	return Key;
}

float3 EncodePositionKey(float3 WorldPosition, float3 WorldNormal, float Spacing)
{
	// Our cache entries are square shaped surfels. Along the primary axis of the normal the length is 1/4 of the specified cell size - which means 4x resolution.
	// This reduces leaks through thin walls and also serves as a fail-safe plan when the geometry normal is not reliable
	
	const int CompressionFactor = 4;
	
	if(abs(WorldNormal.x) >= abs(WorldNormal.y) && abs(WorldNormal.x) >= abs(WorldNormal.z))
	{
		WorldPosition.x *= CompressionFactor;
		return floor(WorldPosition / Spacing);
	}
	
	if(abs(WorldNormal.y) >= abs(WorldNormal.x) && abs(WorldNormal.y) >= abs(WorldNormal.z))
	{
		WorldPosition.y *= CompressionFactor;
		return floor(WorldPosition / Spacing);
	}
	
	if(abs(WorldNormal.z) >= abs(WorldNormal.x) && abs(WorldNormal.z) >= abs(WorldNormal.y))
	{				
		WorldPosition.z *= CompressionFactor;
		return floor(WorldPosition / Spacing);
	}
	
	return floor(WorldPosition / Spacing);
}

uint EncodeNormalBits(float3 WorldNormal)
{
	return	(WorldNormal.x >= 0.01 ? (1 << 5) : 0) |
			(WorldNormal.y >= 0.01 ? (1 << 4) : 0) |
			(WorldNormal.z >= 0.01 ? (1 << 3) : 0) |
			(abs(WorldNormal.x) > abs(WorldNormal.y) && abs(WorldNormal.x) > abs(WorldNormal.z) ? (1 << 2) : 0) |
			(abs(WorldNormal.y) > abs(WorldNormal.x) && abs(WorldNormal.y) > abs(WorldNormal.z) ? (1 << 1) : 0) |
			(abs(WorldNormal.z) > abs(WorldNormal.x) && abs(WorldNormal.z) > abs(WorldNormal.y) ? (1 << 0) : 0)
			;
}

uint2 EncodeICHashKey(float3 WorldPosition, float3 WorldNormal, float Spacing)
{
	float3 PosKey = EncodePositionKey(WorldPosition, WorldNormal, Spacing); // RT_LWC_TODO
	uint3 Key = EncodeVoxelKey(PosKey);
	uint NormalDirectionBits = EncodeNormalBits(WorldNormal);
	
	uint2 HashKey = uint2(0, 0);

	// We have 64 bits in total
	// x: 31 - 26: 06 bits for normal direction
	// x: 25 - 07: 19 bits for world position x
	// x: 06 - 00: 07 bits for world position y (19 bits in total)
	// y: 31 - 20: 12 bits for world position y (19 bits in total)
	// y: 19 - 01: 19 bits for world position z
	// y: 00: unused
	
	HashKey.x = HashKey.x | NormalDirectionBits << 26;
	HashKey.x = HashKey.x | ((Key.x & 0x7fff) << 7);
	HashKey.x = HashKey.x | ((Key.y & 0x7f) << 0);
	
	HashKey.y = HashKey.y | (Key.y & 0xfff) << 20;
	HashKey.y = HashKey.y | (Key.z & 0x7fff) << 1;
	
	return HashKey;
}

#define ATOMIC_ADD_FLOAT(Value, Increment) \
{ \
	uint NewValue = asuint(Increment); \
	uint CompareValue = 0; \
	uint OldValue; \
	[allow_uav_condition] \
	while (true) \
	{ \
		InterlockedCompareExchange(Value, CompareValue, NewValue, OldValue); \
		if (OldValue == CompareValue) \
			break; \
		CompareValue = OldValue; \
		NewValue = asuint(Increment + asfloat(OldValue)); \
	} \
}

void EmitGeometryHitPoint(FFinalGatherHitPoint HitPoint, uint Size)
{
	uint Index;
	if (ICHashTableAdd(EncodeICHashKey(HitPoint.WorldPosition, HitPoint.WorldNormal, Size), Index))
	{
		uint RecordIndex = 0;
		InterlockedAdd(IrradianceCachingParameters.RecordAllocator[0], 1, RecordIndex);
		RecordIndex %= IrradianceCachingParameters.CacheSize;
		IrradianceCachingParameters.RWHashToIndex[Index] = RecordIndex;
		if (IrradianceCachingParameters.RWIndexToHash[RecordIndex] != 0)
		{
			IrradianceCachingParameters.RWHashTable[IrradianceCachingParameters.RWIndexToHash[RecordIndex]] = 0;
		}
		IrradianceCachingParameters.RWIndexToHash[RecordIndex] = Index;
		IrradianceCachingParameters.IrradianceCacheRecords[RecordIndex] = uint4(0, 0, 0, 0);
		IrradianceCachingParameters.IrradianceCacheRecordBackfaceHits[RecordIndex] = uint4(0, 0, 0, 0);
	}
}

int GetIrradianceCachingQuality()
{
	return IrradianceCachingParameters.Quality;
}
