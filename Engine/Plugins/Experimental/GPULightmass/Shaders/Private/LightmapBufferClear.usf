// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightmapBufferClear.usf
=============================================================================*/

#include "/Engine/Private/Common.ush"

#include "LightmapEncoding.ush"
#include "BatchedTiles.ush"

int NumBatchedTiles;

RWTexture2D<float4> OutputTileAtlas;
RWTexture2D<float4> GBufferWorldPosition;
RWTexture2D<float4> GBufferWorldNormal;
RWTexture2D<float4> GBufferShadingNormal;
RWTexture2D<float4> IrradianceAndSampleCount;
RWTexture2D<float4> SHDirectionality;
RWTexture2D<float4> SHCorrectionAndStationarySkyLightBentNormal;
RWTexture2D<float4> ShadowMask;
RWTexture2D<float4> ShadowMaskSampleCount;

uint StagingPoolSizeX;
RWTexture2D<float4> StagingHQLayer0;
RWTexture2D<float4> StagingHQLayer1;
RWTexture2D<float4> StagingShadowMask;
RWTexture2D<float4> StagingSkyOcclusion;

[numthreads(8, 8, 1)]
void CopyConvergedLightmapTilesCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{	
	uint2 BatchedLaunchIndex = DispatchThreadId.xy;
	
	if (BatchedLaunchIndex.y >= GPreviewLightmapPhysicalTileSize) return;

	int TileIndex = BatchedLaunchIndex.x / GPreviewLightmapPhysicalTileSize;
	
	if (TileIndex >= NumBatchedTiles) return;
	
	int2 LaunchIndex = int2(BatchedLaunchIndex.x % GPreviewLightmapPhysicalTileSize, BatchedLaunchIndex.y);
	
	uint2 TexelIndexInPool = LaunchIndex + BatchedTiles[TileIndex].WorkingSetPosition;
	uint2 TexelIndexInStagingPool = LaunchIndex + uint2(TileIndex % StagingPoolSizeX, TileIndex / StagingPoolSizeX) * GPreviewLightmapPhysicalTileSize;
		
	{
		uint SampleCount = asuint(IrradianceAndSampleCount[TexelIndexInPool].w);
		
		if (SampleCount > 0)
		{
			float3 OutputColor = IrradianceAndSampleCount[TexelIndexInPool].rgb / SampleCount;
			FL2SHAndCorrection SH;
			SH.L2SHCoefficients = SHDirectionality[TexelIndexInPool] / SampleCount;
			SH.Correction = SHCorrectionAndStationarySkyLightBentNormal[TexelIndexInPool].x / SampleCount;
			
			FinalizeLightmapIrradiance(OutputColor, SH, StagingHQLayer0[TexelIndexInStagingPool]);
			FinalizeLightmapSH(OutputColor, SH, StagingHQLayer1[TexelIndexInStagingPool]);
		}
		else
		{
			StagingHQLayer0[TexelIndexInStagingPool] = float4(0, 0, 0, -1);
			StagingHQLayer1[TexelIndexInStagingPool] = float4(0, 0, 0, 0);
		}
	}
	
	{
		uint4 ShadowValue =  asuint(ShadowMask[TexelIndexInPool]);
		uint4 SampleCountValue = asuint(ShadowMaskSampleCount[TexelIndexInPool]);
		uint4 ValidityMask = saturate(SampleCountValue) * 2 - 1;

		StagingShadowMask[TexelIndexInStagingPool] = sqrt((float4)ShadowValue / max(uint4(1, 1, 1, 1), SampleCountValue)) * ValidityMask;
	}

	{
		uint SampleCount = asuint(IrradianceAndSampleCount[TexelIndexInPool].w);
		
		if (SampleCount > 0)
		{
			StagingSkyOcclusion[TexelIndexInStagingPool].rgb = saturate(normalize(SHCorrectionAndStationarySkyLightBentNormal[TexelIndexInPool].yzw / SampleCount) * 0.5 + 0.5);
			StagingSkyOcclusion[TexelIndexInStagingPool].a = saturate(sqrt(length(SHCorrectionAndStationarySkyLightBentNormal[TexelIndexInPool].yzw / SampleCount)));
		}
		else
		{
			StagingSkyOcclusion[TexelIndexInStagingPool] = float4(0, 0, 0, 0);
		}
	}
}

RWTexture2D<float4> SrcTexture;
RWTexture2D<float4> DstTexture;
StructuredBuffer<int2> SrcTilePositions;
StructuredBuffer<int2> DstTilePositions;

[numthreads(8, 8, 1)]
void UploadConvergedLightmapTilesCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint2 BatchedLaunchIndex = DispatchThreadId.xy;
	
	if (BatchedLaunchIndex.y >= GPreviewLightmapPhysicalTileSize) return;

	int TileIndex = BatchedLaunchIndex.x / GPreviewLightmapPhysicalTileSize;
	
	if (TileIndex >= NumBatchedTiles) return;
	
	int2 LaunchIndex = int2(BatchedLaunchIndex.x % GPreviewLightmapPhysicalTileSize, BatchedLaunchIndex.y);
	
	DstTexture[LaunchIndex + DstTilePositions[TileIndex]] = SrcTexture[LaunchIndex + SrcTilePositions[TileIndex]];
}

int NumTiles;
int TileSize;
Buffer<int2> TilePositions;
RWTexture2D<float4> TilePool;

[numthreads(8, 8, 1)]
void MultiTileClearCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint2 BatchedLaunchIndex = DispatchThreadId.xy;
	
	if (BatchedLaunchIndex.y >= TileSize) return;

	int TileIndex = BatchedLaunchIndex.x / TileSize;
	
	if (TileIndex >= NumTiles) return;
	
	int2 LaunchIndex = int2(BatchedLaunchIndex.x % TileSize, BatchedLaunchIndex.y);
	
	TilePool[LaunchIndex + TilePositions[TileIndex] * TileSize] = float4(0, 0, 0, 0);
}
