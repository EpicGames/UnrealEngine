// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/PathTracing/PathTracingCommon.ush"

FMinimalPayload TraceLightmapVisibilityRay(
	in RaytracingAccelerationStructure TLAS,
	in uint RayFlags,
	in FRayDesc Ray)
{
	// GPULightmass only binds path tracing shaders, so we need to use it to trace rays, even though this function only cares about the hit distance
	FPackedPathTracingPayload PackedPayload = InitPathTracingVisibilityPayload(1.0f);

	TraceRay(
		TLAS,
		RayFlags,
		PATHTRACER_MASK_SHADOW,
		RAY_TRACING_SHADER_SLOT_SHADOW,
		RAY_TRACING_NUM_SHADER_SLOTS,
		0,
		Ray.GetNativeDesc(),
		PackedPayload);

	// Unpack the payload

	FMinimalPayload MinimalPayload = (FMinimalPayload)0;
	MinimalPayload.HitT = PackedPayload.HitT;
	return MinimalPayload;
}