// Copyright Epic Games, Inc. All Rights Reserved.
float4 SHBasisFunctionFloat(float3 InputVector)
{
	float4 Result;
	// These are derived from simplifying SHBasisFunction in C++
	Result.x = 0.282095f; 
	Result.y = -0.488603f * InputVector.y;
	Result.z = 0.488603f * InputVector.z;
	Result.w = -0.488603f * InputVector.x;
	return Result;
}

struct FL2SHAndCorrection
{
	float4 L2SHCoefficients;
	float Correction;
	
	void AddIncomingRadiance(float Luminance, float3 WorldDirection, float TangentZ)
	{
		if (TangentZ <= 0) return;

		L2SHCoefficients += Luminance * SHBasisFunctionFloat(WorldDirection);
		
		float4 SHInTangentSpace = SHBasisFunctionFloat(float3(0, 0, TangentZ));
		// Evaluate SH after diffuse conv
		Correction += Luminance * dot(SHInTangentSpace, SHBasisFunctionFloat(float3(2.0f / 3, 2.0f / 3, 2.0f / 3)));
	}
};

void FinalizeLightmapSH(
	in float3 Irradiance,
	in FL2SHAndCorrection L2SHAndCorrection,
	inout float4 EncodedSH
)
{

#if 1
	float4 SH = L2SHAndCorrection.L2SHCoefficients;
	// Diffuse conv
	SH *= SHBasisFunctionFloat(float3(2.0f / 3, 2.0f / 3, 2.0f / 3));
	// Perform correction
	SH /= max(0.0001f, L2SHAndCorrection.Correction);
#else
	// Debug: disable SH directionality
	float4 SH = float4(1, 0, 0, 0);
#endif

	EncodedSH = SH.yzwx; // Extra swizzling to match the pixel shader
}

void FinalizeLightmapIrradiance(
	in float3 Irradiance,
	in FL2SHAndCorrection L2SHAndCorrection,
	inout float4 EncodedIrradiance
)
{
	const half LogBlackPoint = 0.01858136;
	
#if 1
	float4 SH = L2SHAndCorrection.L2SHCoefficients;
	// Diffuse conv
	SH *= SHBasisFunctionFloat(float3(2.0f / 3, 2.0f / 3, 2.0f / 3));
	// Perform correction
	SH /= max(0.0001f, L2SHAndCorrection.Correction);
#else
	// Debug: disable SH directionality
	float4 SH = float4(1, 0, 0, 0);
#endif

	float4 EncodedSH = SH.yzwx; // Extra swizzling to match the pixel shader		
	EncodedIrradiance = float4(sqrt(max(Irradiance, float3(0.00001, 0.00001, 0.00001))), log2( 1 + LogBlackPoint ) - (EncodedSH.w / 255 - 0.5 / 255));
}
