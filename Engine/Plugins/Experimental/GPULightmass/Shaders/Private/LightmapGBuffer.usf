// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightmapGBuffer.usf
=============================================================================*/

#include "/Engine/Private/Common.ush"

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

#include "/Engine/Private/PathTracing/Utilities/PathTracingRandomSequence.ush"

float4 VirtualTexturePhysicalTileCoordinateScaleAndBias;
int2 ScratchTilePoolOffset; 
int RenderPassIndex;
int bMaterialTwoSided;

struct FLightmapGBufferVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float4 Position : SV_POSITION;
	float4 SavedWorldPosition : POSITION1;
	// Support WPO in the far future
	// float4 SavedWorldPositionWithShaderOffsets : POSITION2;
}; 

#if VERTEXSHADER
void LightmapGBufferVS(
	FVertexFactoryInput Input,
	out FLightmapGBufferVSToPS Output
	)
{
	ResolvedView = ResolveView();

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float3 WorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates).xyz;
	float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition, TangentToLocal);

	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);

	float2 LightmapUV, UnusedLightmapUV1;
	uint UnusedLightmapDataIndex;
    GetLightMapCoordinates(Output.FactoryInterpolants, LightmapUV, UnusedLightmapUV1, UnusedLightmapDataIndex);
	// Unscale with float2(1, 2)
    LightmapUV *= float2(1, 2);
	// Transform to tile [-1, 1] uv space
    float2 LightmapUVYInverted = (LightmapUV * VirtualTexturePhysicalTileCoordinateScaleAndBias.xy + VirtualTexturePhysicalTileCoordinateScaleAndBias.zw) * 2.0f - 1.0f;
    LightmapUVYInverted.y = -LightmapUVYInverted.y;

    uint SampleIndex = 0;
	float2 RandSample = float2(
		Halton(RenderPassIndex, 2), 
		Halton(RenderPassIndex, 3)
	);

	const float FilterWidth = 2.0f;
    float2 RandOffset = FilterWidth * (- 1.0f + 2 * RandSample) / GPreviewLightmapPhysicalTileSize;

    Output.Position = float4(LightmapUVYInverted + RandOffset, 0.0f, 1.0f);
	Output.SavedWorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	// Support WPO in the far future
	// Output.SavedWorldPositionWithShaderOffsets = Output.SavedWorldPosition + float4(GetMaterialWorldPositionOffset(VertexParameters), 0.0f);
}
#endif

#define WorldPositionScalar 0.00001f

void LightmapGBufferPS(
	FLightmapGBufferVSToPS Input
	OPTIONAL_IsFrontFace
	)
{ 
	ResolvedView = ResolveView();
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Input.FactoryInterpolants, Input.SavedWorldPosition);
	FPixelMaterialInputs PixelMaterialInputs;
	// Support WPO in the far future
	// CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, float4(0, 0, 0, 0), float4(0, 0, 0, 0), bIsFrontFace, Input.SavedWorldPositionWithShaderOffsets, Input.SavedWorldPosition);
	CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, float4(0, 0, 0, 0), float4(0, 0, 0, 0), bIsFrontFace, Input.SavedWorldPosition.xyz, Input.SavedWorldPosition.xyz);
	
	float4 OutWorldPosition = float4(WSHackToFloat(GetWorldPosition(MaterialParameters)) * WorldPositionScalar, 1.0f);
	// Faceted triangle normal
    float4 OutWorldNormal = float4(normalize(cross(ddx(OutWorldPosition.xyz), ddy(OutWorldPosition.xyz))), 1.0f);
    OutWorldNormal *= CondMask(bIsFrontFace, 1.0f, -1.0f);
	OutWorldNormal *= GetPrimitive_DeterminantSign(MaterialParameters.PrimitiveId);
	// Interpolated vertex normal
	float4 OutShadingNormal = float4(MaterialParameters.TangentToWorld[2], bMaterialTwoSided);

	// GBuffer Outputs
	LightmapGBufferParams.ScratchTilePoolLayer0[ScratchTilePoolOffset + Input.Position.xy] = OutWorldPosition;
	LightmapGBufferParams.ScratchTilePoolLayer1[ScratchTilePoolOffset + Input.Position.xy] = OutWorldNormal;
	LightmapGBufferParams.ScratchTilePoolLayer2[ScratchTilePoolOffset + Input.Position.xy] = OutShadingNormal;
}
