// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Private/Common.ush"

#include "LightmapEncoding.ush"
#include "BatchedTiles.ush"

int NumBatchedTiles;

RWTexture2D<float4> OutputTileAtlas;
RWTexture2D<float4> GBufferWorldPosition;
RWTexture2D<float4> GBufferWorldNormal;
RWTexture2D<float4> GBufferShadingNormal;
RWTexture2D<float4> IrradianceAndSampleCount;
RWTexture2D<float4> SHDirectionality;
RWTexture2D<float4> SHCorrectionAndStationarySkyLightBentNormal;
RWTexture2D<float4> ShadowMask;
RWTexture2D<float4> ShadowMaskSampleCount;

int NumTotalSamples;
int NumIrradianceCachePasses;
int NumRayGuidingTrialSamples;

[numthreads(8, 8, 1)]
void SelectiveLightmapOutputCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{	
	uint2 BatchedLaunchIndex = DispatchThreadId.xy;
	
	if (BatchedLaunchIndex.y >= GPreviewLightmapPhysicalTileSize) return;

	int TileIndex = BatchedLaunchIndex.x / GPreviewLightmapPhysicalTileSize;
	
	if (TileIndex >= NumBatchedTiles) return;
	
	int2 LaunchIndex = int2(BatchedLaunchIndex.x % GPreviewLightmapPhysicalTileSize, BatchedLaunchIndex.y);
	
	uint2 TexelIndexInPool = LaunchIndex + BatchedTiles[TileIndex].WorkingSetPosition;
		
	uint SampleCount = asuint(IrradianceAndSampleCount[TexelIndexInPool].w);
					
	if (DIM_OUTPUT_LAYER >= 0 && DIM_OUTPUT_LAYER <= 1)
	{
		if (SampleCount > 0)
		{
			float3 OutputColor = IrradianceAndSampleCount[TexelIndexInPool].rgb / SampleCount;
			FL2SHAndCorrection SH;
			SH.L2SHCoefficients = SHDirectionality[TexelIndexInPool] / SampleCount;
			SH.Correction = SHCorrectionAndStationarySkyLightBentNormal[TexelIndexInPool].x / SampleCount;
			
			if (DIM_OUTPUT_LAYER == 0)
			{
				FinalizeLightmapIrradiance(OutputColor, SH, OutputTileAtlas[LaunchIndex + BatchedTiles[TileIndex].OutputLayer0Position]);
			}
			
			if (DIM_OUTPUT_LAYER == 1)
			{
				FinalizeLightmapSH(OutputColor, SH, OutputTileAtlas[LaunchIndex + BatchedTiles[TileIndex].OutputLayer1Position]);
			}
		}
		else
		{
			if (DIM_OUTPUT_LAYER == 0)
			{
				OutputTileAtlas[LaunchIndex + BatchedTiles[TileIndex].OutputLayer0Position] = float4(0, 0, 0, 0);
			}
			
			if (DIM_OUTPUT_LAYER == 1)
			{
				OutputTileAtlas[LaunchIndex + BatchedTiles[TileIndex].OutputLayer1Position] = float4(0, 0, 0, 0);
			}
		}

#if 0 // Debug: Overwrite with green to see which tiles got output
		if (DIM_OUTPUT_LAYER == 0)
		{
			OutputTileAtlas[LaunchIndex + BatchedTiles[TileIndex].OutputLayer0Position] = float4(0, 1, 0, 0);
		}
		
		if (DIM_OUTPUT_LAYER == 1)
		{
			OutputTileAtlas[LaunchIndex + BatchedTiles[TileIndex].OutputLayer1Position] = float4(0, 0, 0, 1);
		}
#endif
	
#if DRAW_PROGRESS_BARS
		if ((DIM_OUTPUT_LAYER == 0 || DIM_OUTPUT_LAYER == 1) && NumTotalSamples > 0 && BatchedTiles[TileIndex].RenderPassIndex < NumTotalSamples - 1)
		{
			int2 ProgressBarMin = int2(64, 64) / 2 - int2(40, 6) / 2;
			int2 ProgressBarMax = int2(64, 64) / 2 + int2(40, 6) / 2 - int2(1, 1);
			if (LaunchIndex.x >= ProgressBarMin.x && LaunchIndex.y >= ProgressBarMin.y && LaunchIndex.x <= ProgressBarMax.x && LaunchIndex.y <= ProgressBarMax.y)
			{
				if (LaunchIndex.x == ProgressBarMin.x || LaunchIndex.y == ProgressBarMin.y || LaunchIndex.x == ProgressBarMax.x || LaunchIndex.y == ProgressBarMax.y
					|| LaunchIndex.x <= ProgressBarMin.x + 40 * BatchedTiles[TileIndex].RenderPassIndex / (float)NumTotalSamples)
				{
					const half LogBlackPoint = 0.01858136;
					
					float3 Color = float3(0, 1, 0);
					
					if (BatchedTiles[TileIndex].RenderPassIndex < NumIrradianceCachePasses)
					{
						Color = float3(1, 0, 0);
					}
					
					if (BatchedTiles[TileIndex].RenderPassIndex >= NumIrradianceCachePasses && BatchedTiles[TileIndex].RenderPassIndex < NumIrradianceCachePasses + NumRayGuidingTrialSamples)
					{
						Color = float3(0.5, 0.5, 0);
					}
					
					if (DIM_OUTPUT_LAYER == 0)
					{
						OutputTileAtlas[LaunchIndex + BatchedTiles[TileIndex].OutputLayer0Position].rgba = float4(
						Color, log2( 1 + LogBlackPoint ) - (1.0f / 255 - 0.5 / 255));
					}
					
					if (DIM_OUTPUT_LAYER == 1)
					{
						OutputTileAtlas[LaunchIndex + BatchedTiles[TileIndex].OutputLayer1Position] = float4(0, 0, 0, 1);
					}
				}
			}
		}
#endif
	}
	else if (DIM_OUTPUT_LAYER == 2)
	{
		uint4 ShadowValue =  asuint(ShadowMask[TexelIndexInPool]);
		uint4 SampleCountValue = asuint(ShadowMaskSampleCount[TexelIndexInPool]);
		uint4 ValidityMask = saturate(SampleCountValue); // Different math from CopyConvergedLightmapTilesCS, as we need negative values to indicate coverage there

		OutputTileAtlas[LaunchIndex + BatchedTiles[TileIndex].OutputLayer2Position] = sqrt((float4)ShadowValue / max(uint4(1, 1, 1, 1), SampleCountValue)) * ValidityMask;
	}
	else if (DIM_OUTPUT_LAYER == 3)
	{
		if (SampleCount > 0)
		{
			OutputTileAtlas[LaunchIndex + BatchedTiles[TileIndex].OutputLayer3Position].rgb = saturate(normalize(SHCorrectionAndStationarySkyLightBentNormal[TexelIndexInPool].yzw / SampleCount) * 0.5 + 0.5);
			OutputTileAtlas[LaunchIndex + BatchedTiles[TileIndex].OutputLayer3Position].a = saturate(sqrt(length(SHCorrectionAndStationarySkyLightBentNormal[TexelIndexInPool].yzw / SampleCount)));
		}
		else
		{
			OutputTileAtlas[LaunchIndex + BatchedTiles[TileIndex].OutputLayer3Position] = float4(0, 0, 0, 0);
		}
	}
}
