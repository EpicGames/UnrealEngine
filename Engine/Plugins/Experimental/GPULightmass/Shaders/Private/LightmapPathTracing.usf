// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
PathTracingRayGenShader.usf: Reference path tracing
===============================================================================================*/

#define SIMPLIFIED_MATERIAL_SHADER 1 
#define RANDSEQ_UNROLL_SOBOL 0
#define USE_PATH_TRACING_LIGHT_GRID 1

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/PostProcessCommon.ush"
#include "/Engine/Private/RectLight.ush"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/RayTracing/RayTracingDirectionalLight.ush"
#include "/Engine/Private/RayTracing/RayTracingRectLight.ush"
#include "/Engine/Private/RayTracing/RayTracingSphereLight.ush"
#include "/Engine/Private/RayTracing/RayTracingCapsuleLight.ush"

#include "/Engine/Private/PathTracing/PathTracingCommon.ush"
#include "/Engine/Private/PathTracing/Light/PathTracingLightGrid.ush"
#include "/Engine/Private/RayTracing/RayTracingHitGroupCommon.ush"

#include "BatchedTiles.ush"
#include "/Engine/Private/PathTracing/Material/FirstBounceRayGuidingCommon.ush"

int NumRayGuidingTrialSamples;
RWTexture2D<uint> RayGuidingLuminance;
Texture2D<float> RayGuidingCDFX;
Texture2D<float> RayGuidingCDFY;

#include "IrradianceCachingCommon.ush"

#include "/Engine/Private/ShadingModels.ush"
#include "/Engine/Private/PathTracing/Utilities/PathTracingRandomSequence.ush"
#include "/Engine/Private/PathTracing/Light/PathTracingLightSampling.ush"
#include "/Engine/Private/PathTracing/Material/PathTracingMaterialSampling.ush"
#include "/Engine/Private/PathTracing/Material/PathTracingRadianceProbe.ush"

#include "/Engine/Shared/LightDefinitions.h"

#include "LightmapEncoding.ush"
#include "LightmapCommon.ush"

int LastInvalidationFrame;
int NumTotalSamples;

Texture2D<float4> GBufferWorldPosition;
Texture2D<float4> GBufferWorldNormal;
Texture2D<float4> GBufferShadingNormal;
RWTexture2D<float4> IrradianceAndSampleCount;
RWTexture2D<float4> SHDirectionality;
RWTexture2D<float4> SHCorrectionAndStationarySkyLightBentNormal;

RWTexture2D<float4> OutputTileAtlas;

void GenerateCosineNormalRay(
	float3 TranslatedWorldPosition,
	float3 WorldNormal,
	inout FRandomSequence RandSequence,
	out float3 RayOrigin,
	out float3 RayDirection,
	out float3 TangentDirection,
	out float RayTMin,
	out float RayTMax,
	out float RayPdf
)
{
	// Draw random variable
	float2 BufferSize = View.BufferSizeAndInvSize.xy;
	float2 RandSample = RandSequence.Get2D();

	// Perform cosine-hemispherical sampling and convert to world-space
	float4 Direction_Tangent = CosineSampleHemisphere(RandSample);
	TangentDirection = Direction_Tangent.xyz;
	float3 Direction_World = TangentToWorld(Direction_Tangent.xyz, WorldNormal);

	RayOrigin = TranslatedWorldPosition;
	RayDirection = Direction_World;
	RayTMin = 0.01;
	RayTMax = 1e20;
	RayPdf = 1.0f;
}

struct FThreeBandSHVectorFloat
{
	float4 V0;
	float4 V1;
	float V2;
};

struct FThreeBandSHVectorRGBFloat
{
	FThreeBandSHVectorFloat R;
	FThreeBandSHVectorFloat G;
	FThreeBandSHVectorFloat B;
};

FThreeBandSHVectorFloat SHBasisFunction3Float(float3 InputVector)
{
	FThreeBandSHVectorFloat Result;
	// These are derived from simplifying SHBasisFunction in C++
	Result.V0.x = 0.282095f; 
	Result.V0.y = -0.488603f * InputVector.y;
	Result.V0.z = 0.488603f * InputVector.z;
	Result.V0.w = -0.488603f * InputVector.x;

	half3 VectorSquared = InputVector * InputVector;
	Result.V1.x = 1.092548f * InputVector.x * InputVector.y;
	Result.V1.y = -1.092548f * InputVector.y * InputVector.z;
	Result.V1.z = 0.315392f * (3.0f * VectorSquared.z - 1.0f);
	Result.V1.w = -1.092548f * InputVector.x * InputVector.z;
	Result.V2 = 0.546274f * (VectorSquared.x - VectorSquared.y);

	return Result;
}

#define WorldPositionScalar 0.00001f

#ifndef USE_IRRADIANCE_CACHING
	#define USE_IRRADIANCE_CACHING 0
#endif

static const uint MaxBounces = 32;

// 0: only Material sampling
// 1: only Light sampling
// 2: both Material and Light
static const uint MISMode = 2;

static const uint ApproximateCaustics = 1;
static const uint EnableEmissive = 1;
static const float MaxNormalBias = 0.1;

RaytracingAccelerationStructure TLAS;
uint SceneVisibleLightCount;

void AccumulateRadiance(inout float3 TotalRadiance, float3 PathRadiance)
{
	float MaxPathIntensity = 0;
	if (MaxPathIntensity > 0)
	{
		// User asked for path contributions to be clamped to reduce fireflies.
		// Depending on how aggressive this value is, the image could be quite biased
		TotalRadiance += min(PathRadiance, MaxPathIntensity);
	}
	else
	{
		// Just average values directly
		TotalRadiance += PathRadiance;
	}
}

bool IsInvalidSurfaceHit(FPathTracingPayload Payload)
{
	return !Payload.IsFrontFace() && !Payload.IsMaterialTransmissive() && !Payload.IsMaterialTwoSided();
}

FPathTracingPayload TraceTransparentRay(FRayDesc Ray, bool IsCameraRay, bool LastBounce, bool IncludeEmission, uint2 LaunchIndex, uint NumLights, inout RandomSequence RandSequence, inout float3 PathThroughput, inout float3 Radiance)
{
	const uint RayFlags = 0;
	const uint MissShaderIndex = 0;
	float SelectionWeightSum = 0;
	float3 PayloadThroughput = PathThroughput;
	FPathTracingPayload Payload;
	if (!IncludeEmission && LastBounce)
	{
		Payload.SetMiss();
		PathThroughput = 0;
		return Payload;
	}
	for (;;)
	{
		FPackedPathTracingPayload PackedPayload = InitPathTracingPayload(IsCameraRay ? PATHTRACER_SCATTER_CAMERA : PATHTRACER_SCATTER_DIFFUSE, 1.0);
		// Trace the ray
		TraceRay(
			TLAS,
			RayFlags,
			PATHTRACER_MASK_ALL,
			RAY_TRACING_SHADER_SLOT_MATERIAL,
			RAY_TRACING_NUM_SHADER_SLOTS,
			MissShaderIndex,
			Ray.GetNativeDesc(),
			PackedPayload);

		// Loop over lights to capture their contribution
		// #dxr_todo: if we have lots of lights, having some hierarchical structure would be better ....
		for (uint LightId = 0; LightId < NumLights; ++LightId)
		{
			FRayDesc LightRay = Ray;
			LightRay.TMax = PackedPayload.IsMiss() ? Ray.TMax : PackedPayload.HitT;
			float3 LightRadiance = TraceLight(LightRay, LightId).Radiance;
			AccumulateRadiance(Radiance, PathThroughput * LightRadiance);
		}

		if (PackedPayload.IsMiss())
		{
			// Ray didn't hit any real geometry, so nothing left to do
			break;
		}

		// Unpack the payload
		FPathTracingPayload HitPayload = UnpackPathTracingPayload(PackedPayload, Ray);

		// add in surface emission
		if (IncludeEmission)
		{
			AccumulateRadiance(Radiance, PathThroughput * HitPayload.Radiance);
		}

		if (!LastBounce)
		{
			float3 Contrib = PathThroughput * EstimateMaterialAlbedo(HitPayload);

			float SelectionWeight = max3(Contrib.x, Contrib.y, Contrib.z);
			SelectionWeightSum += SelectionWeight;
			bool AcceptHit = false;

			// weighted reservoir sampling
			if (SelectionWeight > 0)
			{
				if (SelectionWeight < SelectionWeightSum)
				{
					// the acceptance probability is not 1.0
					// generate a random number to see if we should accept this hit
					float RandValue = RandSequence.Get1D();
					AcceptHit = RandValue * SelectionWeightSum < SelectionWeight;
				}
				else
				{
					// accept automatically on the first hit
					AcceptHit = true;
				}
			}

			if (AcceptHit)
			{
				// stash this hit for next time
				Payload = HitPayload;
				PayloadThroughput = PathThroughput / SelectionWeight;
			}
		}

		if (IsInvalidSurfaceHit(HitPayload))
		{
			// Propagate to Payload and terminate immediately upon invalid surface hits
			Payload = HitPayload;
			break;
		}

		// prepare next step around the loop
		// retrace the exact same ray with TMin one ulp past the hit we just found
		PathThroughput *= HitPayload.TransparencyColor;
		Ray.TMin = asfloat(asuint(HitPayload.HitT) + 1);

		if (all(PathThroughput == 0))
		{
			break;
		}
	}

	if (SelectionWeightSum > 0)
	{
		// if we stored a valid hit in the payload, reset the path throughput to this point
		PathThroughput = PayloadThroughput * SelectionWeightSum;
	}
	else
	{
		PathThroughput = 0;
		Payload.SetMiss();
	}
	return Payload;
}

float3 TraceTransparentVisibilityRay(FRayDesc Ray, float PathRoughness)
{
	const uint RayFlags = RAY_FLAG_ACCEPT_FIRST_HIT_AND_END_SEARCH | RAY_FLAG_SKIP_CLOSEST_HIT_SHADER;
	const uint InstanceInclusionMask = PATHTRACER_MASK_SHADOW;
	const uint RayContributionToHitGroupIndex = RAY_TRACING_SHADER_SLOT_SHADOW;
	const uint MultiplierForGeometryContributionToShaderIndex = RAY_TRACING_NUM_SHADER_SLOTS;
	const uint MissShaderIndex = 0;

	FPackedPathTracingPayload PackedPayload = InitPathTracingVisibilityPayload(PathRoughness);
	TraceRay(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		RayContributionToHitGroupIndex,
		MultiplierForGeometryContributionToShaderIndex,
		MissShaderIndex,
		Ray.GetNativeDesc(),
		PackedPayload);
	if (PackedPayload.IsHit())
	{
		// We didn't run the miss shader, therefore we must have hit something opaque (or reached full opacity)
		return 0.0;
	}
	return PackedPayload.GetRayThroughput();
}

void PathTracingKernel(
	in uint RenderPassIndex,
	float3 TranslatedWorldPosition,
	float3 ShadingNormal,
	inout RandomSequence RandSequence,
	inout uint SampleIndex,
	inout bool bIsValidSample,
	inout float3 RadianceValue,
	inout float3 RadianceDirection,
	inout float3 DirectLightingIrradiance,
#ifdef LIGHTMAP_PATH_TRACING_MAIN_RG
	inout FL2SHAndCorrection DirectLightingSH,
#else
	inout FThreeBandSHVectorRGBFloat DirectLightingSH,
#endif
	inout float LuminanceForFirstBounceRayGuiding,
	inout float3 SkyLightBentNormal,
	inout float4 PrimaryRandSample)
{
	uint2 LaunchIndex = uint2(0, 0);
	
#if USE_IRRADIANCE_CACHING
	bool bShouldEmitGeometryHitPoint = false;
	
	FFinalGatherHitPoint IrradianceCacheFinalGatherHitPoint = (FFinalGatherHitPoint)0;	
	float RadianceProbe_Pdf = 1.0f / (2 * PI);
	
	uint NewEntrySize = 32;
	int NearestCacheEntryID = -1;
#endif

	float3 Radiance = 0;
	
	// GPULightmass's SampleEmitter(): generate a fake camera ray hitting the texel
	FRayDesc Ray;
	Ray.Origin = TranslatedWorldPosition + ShadingNormal;
	Ray.Direction = -ShadingNormal;
	Ray.TMin = -0.01f;
	Ray.TMax = 1.01f;
	
	// This array will hold a CDF for light picking
	// Seed the array with a uniform CDF at first so that we always have a valid CDF
	float LightPickingCdf[RAY_TRACING_LIGHT_COUNT_MAXIMUM];
	
	Ray.Direction = normalize(Ray.Direction);
	
	// path state variables (these cary information between bounces)
	float3 PathThroughput = 1.0;
	float PathRoughness = 0;

	// number of directly visible lights for the first bounce
	uint NumVisibleLights = SceneVisibleLightCount;
	
	for (int Bounce = 0; Bounce <= MaxBounces; Bounce++)
	{		
		const bool bIsCameraRay = Bounce == 0;
		const bool bIsLastBounce = Bounce == MaxBounces;
		const bool bIncludeEmissive = EnableEmissive;
		FPathTracingPayload Payload = (FPathTracingPayload)0;
		
		if (bIsCameraRay)
		{
			// GPULightmass fakes a 'camera' ray that hits the lightmap texel perpendicularly
			Payload.TranslatedWorldPos = TranslatedWorldPosition;
			Payload.WorldNormal = ShadingNormal;
			Payload.WorldGeoNormal = ShadingNormal;
			Payload.Radiance = 0;
			Payload.BaseColor = 1;
			Payload.SubsurfaceColor = 0;
			Payload.BSDFOpacity = 1;
			Payload.TransparencyColor = 0;
			Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_BASIC;
			Payload.PrimitiveLightingChannelMask = 0b111;
			Payload.HitT = 1.0f;
			Payload.SetFrontFace();
		}
		else
		{
			Payload = TraceTransparentRay(Ray, false , bIsLastBounce, bIncludeEmissive, LaunchIndex, NumVisibleLights, RandSequence, PathThroughput, Radiance);
		}
		
		if (Payload.IsMiss())
		{
			// we didn't hit anything selectable for further shading, we are done
			break;
		}
		
		if (IsInvalidSurfaceHit(Payload))
		{
			bIsValidSample = false;
			break;
		}

	#if USE_IRRADIANCE_CACHING
		float3 TexelPosition = TranslatedWorldPosition - DFHackToFloat(PrimaryView.PreViewTranslation);
		float3 HitPosition = Payload.TranslatedWorldPos - DFHackToFloat(PrimaryView.PreViewTranslation);
		float DistanceFromTexelToHit = length(HitPosition - TexelPosition);
		bool bRejectedByCornerRejection = RenderPassIndex >= GetIrradianceCachingQuality() && DistanceFromTexelToHit < IrradianceCachingParameters.Spacing * IrradianceCachingParameters.CornerRejection;
		
		// Compute an overestimated upper bound for the number of samples we need for this texel by assuming
		// all the rays from all IC building passes are sent from this single texel
		float CellArea = IrradianceCachingParameters.Spacing * IrradianceCachingParameters.Spacing;
		float TotalSamplesOverAllICPasses = GPreviewLightmapPhysicalTileSize * GPreviewLightmapPhysicalTileSize * GetIrradianceCachingQuality();
		float DistanceFraction = CellArea / (2 * PI * DistanceFromTexelToHit * DistanceFromTexelToHit);

		// Cache entries too far away have a very low chance of being reused
		// Don't bother placing or looking them up
		bool bRejectedByMaxLookupDistace = false;
		if (TotalSamplesOverAllICPasses * DistanceFraction < 1)
		{
			bRejectedByMaxLookupDistace = true;
		}
		
		if (NearestCacheEntryID == -1 && Bounce >= 1 && !bRejectedByCornerRejection && !bRejectedByMaxLookupDistace)
		{
			bool bIrradianceQuerySuccessful = false;
			bool bGeometryQuerySuccessful = false;
			
			uint NearestRecordIndex = 0;
			float3 RecordIrradiance;

			uint Index;
			if (ICHashTableFind(EncodeICHashKey(HitPosition, Payload.WorldNormal, IrradianceCachingParameters.Spacing), Index))
			{
				uint RecordIndex = IrradianceCachingParameters.RWHashToIndex[Index];

				uint4 RecordIrradianceAndSampleCount = IrradianceCachingParameters.IrradianceCacheRecords[RecordIndex];
				uint BackfaceHitsCount = IrradianceCachingParameters.IrradianceCacheRecordBackfaceHits[RecordIndex].a;
			
				// We accept a cache entry, if:
				// 1) The entry has been built by other lightmap tiles and contains more samples than we will ever need.
				// 2) We're outside of IC building passes. An entry with low sample count means we just need that much - still, we're going to require a minimum of 4 samples to remove outliers from potential cache thrashing

				bool bShouldAcceptCacheEntry = false;

				uint SampleCountThreshold;

				if (RenderPassIndex < GetIrradianceCachingQuality())
				{
					SampleCountThreshold = TotalSamplesOverAllICPasses * DistanceFraction;
				}
				else
				{
					SampleCountThreshold = 4;
				}

				bShouldAcceptCacheEntry = RecordIrradianceAndSampleCount.w >= SampleCountThreshold;
				
			#if IC_BACKFACE_DETECTION
				// Reject the entry if it can see back faces
				if (RenderPassIndex >= GetIrradianceCachingQuality() && BackfaceHitsCount > RecordIrradianceAndSampleCount.w * BackfaceThresholdRejection)
				{
					bShouldAcceptCacheEntry = false;
				}

				// Terminate the path if the entry contains too many back face hits which means it is very likely to be fully inside geometry
				if (RenderPassIndex >= GetIrradianceCachingQuality() && BackfaceHitsCount > RecordIrradianceAndSampleCount.w * BackfaceThresholdInsideGeometry)
				{
					bIsValidSample = false;
					break;
				}
			#endif
			
				if (bShouldAcceptCacheEntry)
				{
					bIrradianceQuerySuccessful = true;
		
					RecordIrradiance = asfloat(RecordIrradianceAndSampleCount.xyz) / RecordIrradianceAndSampleCount.w;
				}
				
				bGeometryQuerySuccessful = true;
				NearestRecordIndex = RecordIndex;
			}
	
			if (bIrradianceQuerySuccessful)
			{
				// Successful query. Terminate path immediately.
				Radiance += RecordIrradiance * PathThroughput;
				LuminanceForFirstBounceRayGuiding += Luminance(RecordIrradiance * PathThroughput);
				break;
			}
			else
			{
				// Failed query.
				if (Bounce == 1)
				{
					// Prevent modification to the cache outside IC building passes
					if (RenderPassIndex < GetIrradianceCachingQuality())
					{
						if (!bGeometryQuerySuccessful)
						{
							// Only use the very first few passes to place new entries to reduce cache thrashing
							bool bShouldPlaceNewEntry = RenderPassIndex < 16;

							if (bShouldPlaceNewEntry)
							{
								bShouldEmitGeometryHitPoint = true;
								IrradianceCacheFinalGatherHitPoint.WorldPosition = HitPosition;
								IrradianceCacheFinalGatherHitPoint.WorldNormal = Payload.WorldNormal;
								NewEntrySize = IrradianceCachingParameters.Spacing;
								break;
							}
						}
						else
						{
							NearestCacheEntryID = NearestRecordIndex;
						}
					}
				}
			}
		}
	#endif
		
		FLightLoopCount LightLoopCount = LightGridLookup(Payload.TranslatedWorldPos);

		// Choose a random number for both Light sampling and BxDF sampling
		float4 RandSample = RandSequence.Get4D();

		float LightPickingCdfSum = 0;

		// If we are using Light sampling and the material can use it ...
		if (MISMode != 0 && SceneLightCount > 0)
		{
			// Choose a light and sample it
			float3 TranslatedWorldPos = Payload.TranslatedWorldPos;
			float3 WorldNormal = Payload.WorldNormal;
			uint PrimitiveLightingChannelMask = Payload.PrimitiveLightingChannelMask;

			bool IsTransmissiveMaterial = ENABLE_TRANSMISSION && Payload.IsMaterialTransmissive();

			for (uint Index = 0, Num = LightLoopCount.NumLights; Index < Num; ++Index)
			{
				uint LightIndex = GetLightId(Index, LightLoopCount);
				LightPickingCdfSum += EstimateLight(LightIndex, TranslatedWorldPos, WorldNormal, PrimitiveLightingChannelMask, IsTransmissiveMaterial);
				LightPickingCdf[Index] = LightPickingCdfSum;
			}

			if (LightPickingCdfSum > 0)
			{
				// init worked
				int LightId;
				float LightPickPdf = 0;

				SelectLight(RandSample.x * LightPickingCdfSum, LightLoopCount.NumLights, LightPickingCdf, LightId, LightPickPdf);

				LightId = GetLightId(LightId, LightLoopCount);

				FLightSample LightSample = SampleLight(LightId, RandSample.yz, TranslatedWorldPos, WorldNormal);

				LightPickPdf /= LightPickingCdfSum;

				LightSample.RadianceOverPdf /= LightPickPdf;
				LightSample.Pdf *= LightPickPdf;
				if (LightSample.Pdf > 0)
				{
					float3 Visibility = float3(1, 1, 1);
					
					if (CastsShadow(LightId))
					{
						// for transmissive materials, bias the position to the other side of the surface if the light is coming from behind
						const float SignedPositionBias = IsTransmissiveMaterial ? sign(dot(Payload.WorldNormal, LightSample.Direction)) : 1.0;

						FRayDesc LightRay;
						LightRay.Origin = TranslatedWorldPos;
						LightRay.TMin = 0;
						LightRay.Direction = LightSample.Direction;
						LightRay.TMax = LightSample.Distance;
						ApplyRayBias(LightRay.Origin, Payload.HitT, SignedPositionBias * Payload.WorldGeoNormal);
					
						float AvgRoughness = ApproximateCaustics ? GetAverageRoughness(Payload) : 0.0;

						Visibility = TraceTransparentVisibilityRay(LightRay, AvgRoughness);
						LightSample.RadianceOverPdf *= Visibility;
					}

					// #dxr_todo: Is it cheaper to fire the ray first? Or eval the material first?
					if (any(LightSample.RadianceOverPdf > 0))
					{
						// Evaluate material
						FMaterialEval MaterialEval;
						if (Bounce == 0)
						{
							MaterialEval = RadianceProbe_EvalMaterial(LightSample.Direction, Payload);
						}
						else
						{
							MaterialEval = EvalMaterial(-Ray.Direction, LightSample.Direction, Payload, float2(1.0, 0.0));
						}
							

						// Record the contribution
						float3 LightContrib = PathThroughput * LightSample.RadianceOverPdf * MaterialEval.Weight * MaterialEval.Pdf;						
						float3 BentNormalVector = LightSample.Direction / LightSample.Pdf / PI * Visibility;
						if (MISMode == 2)
						{
							LightContrib *= MISWeightPower(LightSample.Pdf, MaterialEval.Pdf);
							BentNormalVector *= MISWeightPower(LightSample.Pdf, MaterialEval.Pdf);
						}

						// Record the contribution
						if (Bounce > 0)
						{
							AccumulateRadiance(Radiance, LightContrib);
							LuminanceForFirstBounceRayGuiding += Luminance(LightContrib);
						}
						else
						{
							// GPU Lightmass records contribution of direct lighting separately, and only for static lights
							if (!IsStationary(LightId))
							{
							#ifdef LIGHTMAP_PATH_TRACING_MAIN_RG
								float TangentZ = saturate(dot(LightSample.Direction, Payload.WorldNormal));
								DirectLightingSH.AddIncomingRadiance(Luminance(LightContrib), LightSample.Direction, TangentZ);
								DirectLightingIrradiance += LightContrib * TangentZ;
							#else								
								DirectLightingSH.R.V0 = SHBasisFunction3Float(LightSample.Direction).V0 * LightContrib.r;
								DirectLightingSH.R.V1 = SHBasisFunction3Float(LightSample.Direction).V1 * LightContrib.r;
								DirectLightingSH.R.V2 = SHBasisFunction3Float(LightSample.Direction).V2 * LightContrib.r;
								DirectLightingSH.G.V0 = SHBasisFunction3Float(LightSample.Direction).V0 * LightContrib.g;
								DirectLightingSH.G.V1 = SHBasisFunction3Float(LightSample.Direction).V1 * LightContrib.g;
								DirectLightingSH.G.V2 = SHBasisFunction3Float(LightSample.Direction).V2 * LightContrib.g;
								DirectLightingSH.B.V0 = SHBasisFunction3Float(LightSample.Direction).V0 * LightContrib.b;
								DirectLightingSH.B.V1 = SHBasisFunction3Float(LightSample.Direction).V1 * LightContrib.b;
								DirectLightingSH.B.V2 = SHBasisFunction3Float(LightSample.Direction).V2 * LightContrib.b;
							#endif
							}

							if (IsStationary(LightId) && IsEnvironmentLight(LightId) && CastsShadow(LightId))
							{
								SkyLightBentNormal += BentNormalVector;
							}
						}
					}
				}
			}
		}
	
		// Sample material
		FMaterialSample MaterialSample;
		if (Bounce == 0)
		{
			MaterialSample = RadianceProbe_SampleMaterial(Payload, RandSample.xyz);
		}
		else
		{
			MaterialSample = SampleMaterial(-Ray.Direction, Payload, RandSample.xyz);
		}

		if (MaterialSample.Pdf < 0 || asuint(MaterialSample.Pdf) > 0x7F800000)
		{
			// Pdf became invalid (either negative or NaN)
			Radiance = float3(1, 0, 1);
			break;
		}

		if (!(MaterialSample.Pdf > 0))
		{
			// No valid direction -- we are done
			break;
		}

		float3 NextPathThroughput = PathThroughput * MaterialSample.Weight;
		if (!any(NextPathThroughput > 0))
		{
			// no energy left in this path
			break;
		}

		// Russian roulette:
		//   The probability of keeping the path should be roughly proportional to the weight at the current shade point,
		//  but just using MaterialWeight would miss out on cases where the path throughput changes color (like in a cornell
		//  box when bouncing between walls of different colors). So use the ratio of the brightest color channel in the
		//  previous and next throughput.
		//   The second tweak is to add a sqrt() around the probability to soften the termination probability (paths will last
		//  a little longer). This allows paths to go a bit deeper than the naive heuristic while still allowing them to terminate
		//  early. This makes RR effective from the very first bounce without needing to delay it.
		float ContinuationProb = sqrt(saturate(max(NextPathThroughput.x, max(NextPathThroughput.y, NextPathThroughput.z)) / max(PathThroughput.x, max(PathThroughput.y, PathThroughput.z))));
		if (ContinuationProb < 1)
		{
			// If there is some chance we should terminate the ray, draw an extra random value
			float RussianRouletteRand = RandSample.w;
			if (RussianRouletteRand >= ContinuationProb)
			{
				// stochastically terminate the path
				break;
			}
			PathThroughput = NextPathThroughput / ContinuationProb;
		}
		else
		{
			PathThroughput = NextPathThroughput;
		}

		// Update ray according to material sample
		Ray.Origin = Payload.TranslatedWorldPos;
		Ray.Direction = MaterialSample.Direction;
		Ray.TMin = 0;
		Ray.TMax = RAY_DEFAULT_T_MAX;
		ApplyRayBias(Ray.Origin, Payload.HitT, MaterialSample.PositionBiasSign * Payload.WorldGeoNormal);

		if (ApproximateCaustics)
		{
			// enlarge roughness based on the chosen lobe roughness
			PathRoughness = max(PathRoughness, MaterialSample.Roughness);
		}
		
		if (Bounce == 0)
		{
			RadianceDirection = Ray.Direction;
		#if USE_IRRADIANCE_CACHING
			RadianceProbe_Pdf = MaterialSample.Pdf;
		#endif
			PrimaryRandSample = RandSample;
		}
		
		// If we are using Material sampling for lights
		if (MISMode != 1)
		{
			// Check which lights can be seen by the material ray and trace a dedicated shadow ray
			// While it would be possible to just loop around and use the indirect ray to do this, it would prevent the application
			// of shadow ray specific logic for transparent shadows or various per light tricks like shadow casting
			const bool bUseMIS = MISMode == 2 && LightPickingCdfSum > 0;
			for (uint Index = 0, Num = LightLoopCount.NumMISLights; Index < Num; ++Index)
			{
				uint LightId = GetLightId(Index, LightLoopCount);
				if ((Payload.PrimitiveLightingChannelMask & GetLightingChannelMask(LightId)) == 0)
				{
					// light does not affect the current ray
					continue;
				}

				FLightHit LightResult = TraceLight(Ray, LightId);

				if (LightResult.IsMiss())
				{
					continue;
				}

				float3 LightContrib = PathThroughput * LightResult.Radiance;
				float3 BentNormalVector = PathThroughput * MaterialSample.Direction / PI;
				
				if (bUseMIS)
				{
					float PreviousCdfValue = 0.0;
					BRANCH if (Index > 0)
					{
						PreviousCdfValue = LightPickingCdf[Index - 1];
					}
					float LightPickPdf = (LightPickingCdf[Index] - PreviousCdfValue) / LightPickingCdfSum;

					LightContrib *= MISWeightPower(MaterialSample.Pdf, LightResult.Pdf * LightPickPdf);

					// Separate direct lighting (bounce 0) for GPU Lightmass
					BentNormalVector *= MISWeightPower(MaterialSample.Pdf, LightResult.Pdf * LightPickPdf);
				}

				if (any(LightContrib > 0))
				{
					if (CastsShadow(LightId))
					{
						FRayDesc LightRay = Ray;
						LightRay.TMax = LightResult.HitT;
						float3 Visibility = TraceTransparentVisibilityRay(LightRay, PathRoughness);
						LightContrib *= Visibility;
						
						// Separate direct lighting (bounce 0) for GPU Lightmass
						BentNormalVector *= Visibility;
					}

					if (Bounce > 0)
					{
						// the light made some contribution, and there was nothing along the shadow ray
						AccumulateRadiance(Radiance, LightContrib);
					}
					else // Bounce == 0
					{
						if (!IsStationary(LightId))
						{
						#ifdef LIGHTMAP_PATH_TRACING_MAIN_RG
							float TangentZ = saturate(dot(MaterialSample.Direction, Payload.WorldNormal));
							DirectLightingSH.AddIncomingRadiance(Luminance(LightContrib), MaterialSample.Direction, TangentZ);
							DirectLightingIrradiance += LightContrib * TangentZ;
						#else								
							DirectLightingSH.R.V0 = SHBasisFunction3Float(MaterialSample.Direction).V0 * LightContrib.r;
							DirectLightingSH.R.V1 = SHBasisFunction3Float(MaterialSample.Direction).V1 * LightContrib.r;
							DirectLightingSH.R.V2 = SHBasisFunction3Float(MaterialSample.Direction).V2 * LightContrib.r;
							DirectLightingSH.G.V0 = SHBasisFunction3Float(MaterialSample.Direction).V0 * LightContrib.g;
							DirectLightingSH.G.V1 = SHBasisFunction3Float(MaterialSample.Direction).V1 * LightContrib.g;
							DirectLightingSH.G.V2 = SHBasisFunction3Float(MaterialSample.Direction).V2 * LightContrib.g;
							DirectLightingSH.B.V0 = SHBasisFunction3Float(MaterialSample.Direction).V0 * LightContrib.b;
							DirectLightingSH.B.V1 = SHBasisFunction3Float(MaterialSample.Direction).V1 * LightContrib.b;
							DirectLightingSH.B.V2 = SHBasisFunction3Float(MaterialSample.Direction).V2 * LightContrib.b;
						#endif
						}

						if (IsStationary(LightId) && IsEnvironmentLight(LightId) && CastsShadow(LightId))
						{
							SkyLightBentNormal += BentNormalVector;
						}
					}
					
					LuminanceForFirstBounceRayGuiding += Luminance(LightContrib);
				}
			}
			
		}

		// from this point on, we don't need to include lights in the trace call
		// because NEE handled it for us
		NumVisibleLights = 0;
	}
	
#if USE_IRRADIANCE_CACHING
	if(bIsValidSample)
	{
		if(bShouldEmitGeometryHitPoint)
		{
			EmitGeometryHitPoint(IrradianceCacheFinalGatherHitPoint, NewEntrySize);
		}
		
		if(NearestCacheEntryID != -1)
		{
			float3 IrradianceToAccumulate = Radiance * RadianceProbe_Pdf; // Revert the radiance probe pdf as we're caching the first bounce outgoing radiance
			ATOMIC_ADD_FLOAT(IrradianceCachingParameters.IrradianceCacheRecords[NearestCacheEntryID].r, IrradianceToAccumulate.r);
			ATOMIC_ADD_FLOAT(IrradianceCachingParameters.IrradianceCacheRecords[NearestCacheEntryID].g, IrradianceToAccumulate.g);
			ATOMIC_ADD_FLOAT(IrradianceCachingParameters.IrradianceCacheRecords[NearestCacheEntryID].b, IrradianceToAccumulate.b);
			InterlockedAdd(IrradianceCachingParameters.IrradianceCacheRecords[NearestCacheEntryID].a, 1);
		}
	}
	else
	{
	#if IC_BACKFACE_DETECTION
		if(NearestCacheEntryID != -1)
		{
			InterlockedAdd(IrradianceCachingParameters.IrradianceCacheRecordBackfaceHits[NearestCacheEntryID].a, 1);
		}
	#endif
	}
#endif
	
	RadianceValue = Radiance;
}

[shader("raygeneration")]
void LightmapPathTracingMainRG()
{
	uint2 BatchedLaunchIndex = DispatchRaysIndex().xy;

	uint2 LaunchIndex = uint2(BatchedLaunchIndex.x % GPreviewLightmapPhysicalTileSize, BatchedLaunchIndex.y);
	int TileIndex = BatchedLaunchIndex.x / GPreviewLightmapPhysicalTileSize;
	uint2 TexelIndexInPool = LaunchIndex + BatchedTiles[TileIndex].WorkingSetPosition;
	uint2 TexelIndexInScratch = LaunchIndex + BatchedTiles[TileIndex].ScratchPosition;

	float3 WorldNormal = GBufferWorldNormal[TexelIndexInScratch].xyz;
	float3 ShadingNormal = GBufferShadingNormal[TexelIndexInScratch].xyz;
	bool bMaterialTwoSided = GBufferShadingNormal[TexelIndexInScratch].w;

	float4 EncodedGBufferWorldPosition = GBufferWorldPosition[TexelIndexInScratch];

	bool bGBufferSampleValid = true;
	if (EncodedGBufferWorldPosition.w == 0.0f)
	{
		bGBufferSampleValid = false;
	}

	float3 TranslatedWorldPosition = EncodedGBufferWorldPosition.xyz / WorldPositionScalar + DFHackToFloat(PrimaryView.PreViewTranslation); // RT_LWC_TODO
	
	int EffectiveRenderPassIndex = BatchedTiles[TileIndex].RenderPassIndex;
#if USE_IRRADIANCE_CACHING
	if (EffectiveRenderPassIndex >= GetIrradianceCachingQuality())
	{
		EffectiveRenderPassIndex -= GetIrradianceCachingQuality();

	#if USE_FIRST_BOUNCE_RAY_GUIDING
		if (EffectiveRenderPassIndex >= NumRayGuidingTrialSamples)
		{
			EffectiveRenderPassIndex -= NumRayGuidingTrialSamples;
		}
	#endif
	}
#endif
					
	if (EffectiveRenderPassIndex == 0) 
	{
		IrradianceAndSampleCount[TexelIndexInPool]= float4(0, 0, 0, 0);
		SHDirectionality[TexelIndexInPool] = float4(0, 0, 0, 0);
		SHCorrectionAndStationarySkyLightBentNormal[TexelIndexInPool] = float4(0, 0, 0, 0);
	}
	
	if (NumTotalSamples <= 0 || (NumTotalSamples > 0 && BatchedTiles[TileIndex].RenderPassIndex < NumTotalSamples - 1))
	{
		bool bAnyHemispherePathSampleValid = false;
	
		if (bGBufferSampleValid)
		{			
			// Needs a seed that is only related to position in virtual space to avoid seams due to per-tile calculation
			uint2 VirtualTextureSpacePosition = BatchedTiles[TileIndex].VirtualTilePosition + LaunchIndex - uint2(2, 2);
			uint Seed = VirtualTextureSpacePosition.y * BatchedTiles[TileIndex].LightmapSize.x + VirtualTextureSpacePosition.x;
					
			float4 PrimaryRandSample = float4(-1, -1, -1, -1);
			
			// Evaluate both hemispheres for two sided materials and add the lighting up naively
			UNROLL_N(2)
			for (int HemisphereIndex = 0; HemisphereIndex < (bMaterialTwoSided ? 2 : 1); HemisphereIndex++)
			{
				FRandomSequence RandSequence = RandomSequenceCreate(Seed, EffectiveRenderPassIndex);
				
				uint SampleIndex = 2;

				float3 RadianceValue = 0;
				float3 RadianceDirection = 0;				
				float3 DirectLightingIrradiance = 0;
			#ifdef LIGHTMAP_PATH_TRACING_MAIN_RG
				FL2SHAndCorrection DirectLightingSH = (FL2SHAndCorrection)0;
			#else
				FThreeBandSHVectorRGBFloat DirectLightingSH = (FThreeBandSHVectorRGBFloat)0;
			#endif
				float LuminanceForFirstBounceRayGuiding = 0;
				float3 SkyLightBentNormal = 0;

				float3 EffectiveShadingNormal = HemisphereIndex == 0 ? ShadingNormal : -ShadingNormal;

				bool bPathSampleValid = true;
				
				PathTracingKernel(
					BatchedTiles[TileIndex].RenderPassIndex,
					TranslatedWorldPosition,
					EffectiveShadingNormal,
					RandSequence,
					SampleIndex,
					bPathSampleValid,
					RadianceValue,
					RadianceDirection,
					DirectLightingIrradiance,
					DirectLightingSH,
					LuminanceForFirstBounceRayGuiding,
					SkyLightBentNormal,
					PrimaryRandSample
				);
				
				if (any(isnan(RadianceValue)) || any(RadianceValue < 0) || any(isinf(RadianceValue)))
				{
					bPathSampleValid = false;
				}
				
				if (bPathSampleValid)
				{
				#if USE_FIRST_BOUNCE_RAY_GUIDING
					int MinRenderPassIndex = 0;
					int MaxRenderPassIndex = NumRayGuidingTrialSamples;
				
				#if USE_IRRADIANCE_CACHING
					MinRenderPassIndex += GetIrradianceCachingQuality();
					MaxRenderPassIndex += GetIrradianceCachingQuality();
				#endif
					
					if (BatchedTiles[TileIndex].RenderPassIndex >= MinRenderPassIndex && BatchedTiles[TileIndex].RenderPassIndex < MaxRenderPassIndex)
					{
						float2 Point = PrimaryRandSample.yx;						
						int2 ClusterPosition = clamp((int2)LaunchIndex - int2(2, 2), int2(0, 0), int2(63, 63)) / TEXEL_CLUSTER_SIZE;
						float2 JitteredBin = Point * DIRECTIONAL_BINS_ONE_DIM;// - float2(0.5f, 0.5f) + PrimaryRandSample.xy;
						int2 PositionInBin = clamp(JitteredBin, float2(0, 0), float2(DIRECTIONAL_BINS_ONE_DIM - 1, DIRECTIONAL_BINS_ONE_DIM - 1));
						int2 FinalPosition = ClusterPosition * DIRECTIONAL_BINS_ONE_DIM + PositionInBin;

						{
							float Illuminance = LuminanceForFirstBounceRayGuiding * saturate(dot(RadianceDirection, EffectiveShadingNormal));
							// For positive floats we can cast them to uint and use atomic max directly
							InterlockedMax(RayGuidingLuminance[BatchedTiles[TileIndex].WorkingSetPosition / GPreviewLightmapPhysicalTileSize * CDF_TILE_SIZE + FinalPosition], asuint(max(Illuminance, 0)));
						}
					}
				#endif
					
					{
						FL2SHAndCorrection SH = (FL2SHAndCorrection)0;
						
						float TangentZ = saturate(dot(RadianceDirection, EffectiveShadingNormal));
						SH.AddIncomingRadiance(Luminance(RadianceValue), RadianceDirection, TangentZ);
						
						IrradianceAndSampleCount[TexelIndexInPool].rgb += float3(RadianceValue * TangentZ / PI);
						SHDirectionality[TexelIndexInPool] += SH.L2SHCoefficients;					
						SHCorrectionAndStationarySkyLightBentNormal[TexelIndexInPool].x += SH.Correction;
					}

					#ifdef LIGHTMAP_PATH_TRACING_MAIN_RG
					{
						IrradianceAndSampleCount[TexelIndexInPool].rgb += DirectLightingIrradiance / PI;
						SHDirectionality[TexelIndexInPool] += DirectLightingSH.L2SHCoefficients;					
						SHCorrectionAndStationarySkyLightBentNormal[TexelIndexInPool].x += DirectLightingSH.Correction;
					}
					#endif

					SHCorrectionAndStationarySkyLightBentNormal[TexelIndexInPool].yzw += SkyLightBentNormal;
				}

				bAnyHemispherePathSampleValid = bAnyHemispherePathSampleValid || bPathSampleValid;
			}
		}

		uint SampleCount = asuint(IrradianceAndSampleCount[TexelIndexInPool].w);

#if 0 // Smooth invalidation. This path is currently disabled to avoid inconsistency when saving baked results.
		if (BatchedTiles[TileIndex].FrameIndex < LastInvalidationFrame)
		{
			if (SampleCount > 0)
			{
				float L = Luminance(IrradianceAndSampleCount[TexelIndexInPool].rgb / SampleCount);
				int HistoryWeight = 2.0f / (1.0f + L * L);
				IrradianceAndSampleCount[TexelIndexInPool].rgb = IrradianceAndSampleCount[TexelIndexInPool].rgb / SampleCount * HistoryWeight;
				SHDirectionality[TexelIndexInPool] = SHDirectionality[TexelIndexInPool] / SampleCount * HistoryWeight;
				SampleCount = HistoryWeight;
			}
		}
#endif
		
		if (bAnyHemispherePathSampleValid)
			SampleCount++;
		
		IrradianceAndSampleCount[TexelIndexInPool].a = asfloat(SampleCount);
	}
			
	#if 0 // Debug: Ray guiding PDF visualization
		{
			int2 ClusterPosition = clamp((int2)LaunchIndex - int2(2, 2), int2(0, 0), int2(63, 63)) / TEXEL_CLUSTER_SIZE;
			int2 PositionInBin = clamp((int2)LaunchIndex - int2(2, 2), int2(0, 0), int2(63, 63)) % DIRECTIONAL_BINS_ONE_DIM;
			int2 FinalPosition = ClusterPosition * DIRECTIONAL_BINS_ONE_DIM + PositionInBin;

			IrradianceAndSampleCount[TexelIndexInPool].rgb = (
				asfloat(RayGuidingCDF[BatchedTiles[TileIndex].WorkingSetPosition / GPreviewLightmapPhysicalTileSize * CDF_TILE_SIZE + LaunchIndex]).xxx / (1 + asuint(IrradianceAndSampleCount[TexelIndexInPool].a))
			);
			
			SHDirectionality[TexelIndexInPool] = float4(1, 0, 0, 0);					
			SHCorrectionAndStationarySkyLightBentNormal[TexelIndexInPool].x = 1;
		}
	#endif

#if 0 // Debug: GBuffer shading normal visualization	
	{
		const half LogBlackPoint = 0.01858136;
	
		float3 ShadingNormal = GBufferShadingNormal[TexelIndexInScratch].xyz * 0.5f + 0.5f;
		
		IrradianceAndSampleCount[TexelIndexInPool].rgb = ShadingNormal;
		IrradianceAndSampleCount[TexelIndexInPool].a = asfloat(1);
		
		SHDirectionality[TexelIndexInPool] = float4(1, 0, 0, 0);					
		SHCorrectionAndStationarySkyLightBentNormal[TexelIndexInPool].x = 1;
	}
#endif
}

#include "BrickAllocationDefs.ush"

uint FrameNumber;
float4 VolumeMin;
float4 VolumeSize;
int3 IndirectionTextureDim;
Texture3D<uint4> IndirectionTexture;
Buffer<uint4> BrickRequests;
int NumTotalBricks;
int BrickBatchOffset;

RWTexture3D<float4> AmbientVector;
RWTexture3D<float4> SHCoefficients0R;
RWTexture3D<float4> SHCoefficients1R;
RWTexture3D<float4> SHCoefficients0G;
RWTexture3D<float4> SHCoefficients1G;
RWTexture3D<float4> SHCoefficients0B;
RWTexture3D<float4> SHCoefficients1B;
RWTexture3D<float4> SkyBentNormal;
RWTexture3D<UNORM float> DirectionalLightShadowing;

uint MortonEncode3(uint3 Pixel)
{
	uint Morton = MortonCode3(Pixel.x & 0xFF) | (MortonCode3(Pixel.y & 0xFF) << 1) | (MortonCode3(Pixel.z & 0xFF) << 1);
	return Morton;
}

// Must match C++
struct FLightShaderConstants
{
	float3 PositionHigh;
	float  InvRadius;
	float3 Color;
	float  FalloffExponent;
	float3 Direction;
	float  SpecularScale;
	float  DiffuseScale;
	float3 Tangent;
	float  SourceRadius;
	float3 PositionLow;
	float  SourceLength;
	float2 SpotAngles;
	float  SoftSourceRadius;
	float  RectLightBarnCosAngle;
	float  RectLightBarnLength;
	
	void FillLightShaderParameters(inout FLightShaderParameters LightShaderParameters)
	{
		const FDFVector3 WorldPosition = MakeDFVector3(PositionHigh, PositionLow);

		LightShaderParameters.TranslatedWorldPosition = DFFastAddDemote(WorldPosition, PrimaryView.PreViewTranslation);
		LightShaderParameters.InvRadius = InvRadius;
		LightShaderParameters.Color = Color;
		LightShaderParameters.FalloffExponent = FalloffExponent;
		LightShaderParameters.Direction = Direction;
		LightShaderParameters.SpecularScale = SpecularScale;
		LightShaderParameters.DiffuseScale = DiffuseScale;
		LightShaderParameters.Tangent = Tangent;
		LightShaderParameters.SourceRadius = SourceRadius;
		LightShaderParameters.SpotAngles = SpotAngles;
		LightShaderParameters.SoftSourceRadius = SoftSourceRadius;
		LightShaderParameters.SourceLength = SourceLength;
		LightShaderParameters.RectLightBarnCosAngle = RectLightBarnCosAngle;
		LightShaderParameters.RectLightBarnLength = RectLightBarnLength;
	}
};

bool GenerateOcclusionRay(
	int LightType,
	FLightShaderParameters LightParameters,
	float3 TranslatedWorldPosition,
	float3 WorldNormal,
	float2 RandSample,
	inout float3 RayOrigin,
	inout float3 RayDirection,
	inout float RayTMin,
	inout float RayTMax
)
{
	if (LightType == LIGHT_TYPE_DIRECTIONAL)
	{
		GenerateDirectionalLightOcclusionRay(
			LightParameters,
			TranslatedWorldPosition, WorldNormal,
			RandSample,
			/* out */ RayOrigin,
			/* out */ RayDirection,
			/* out */ RayTMin,
			/* out */ RayTMax);
	}
	else if (LightType == LIGHT_TYPE_POINT || LightType == LIGHT_TYPE_SPOT)
	{
		if (LightType == LIGHT_TYPE_SPOT)
		{
			// before generating a shadow ray, make sure we are inside the cone of the light
			float3 LightDirection = normalize(LightParameters.TranslatedWorldPosition - TranslatedWorldPosition);
			float CosAngle = LightParameters.SpotAngles.x;
			if (!(dot(LightDirection, LightParameters.Direction) >= CosAngle))
			{
				// outside the cone of the light, skip all work
				return false;
			}
		}
		float RayPdf;
		if (LightParameters.SourceLength > 0.0)
		{
			return GenerateCapsuleLightOcclusionRayWithSolidAngleSampling(
				LightParameters,
				TranslatedWorldPosition, WorldNormal,
				RandSample,
				/* out */ RayOrigin,
				/* out */ RayDirection,
				/* out */ RayTMin,
				/* out */ RayTMax,
				/* out */ RayPdf);
		}
		return GenerateSphereLightOcclusionRayWithSolidAngleSampling(
			LightParameters,
			TranslatedWorldPosition, WorldNormal,
			RandSample,
			/* out */ RayOrigin,
			/* out */ RayDirection,
			/* out */ RayTMin,
			/* out */ RayTMax,
			/* out */ RayPdf);
	}
	else if (LightType == LIGHT_TYPE_RECT)
	{
		float RayPdf = 0.0;
		return GenerateRectLightOcclusionRay(
			LightParameters,
			TranslatedWorldPosition, WorldNormal,
			RandSample,
			/* out */ RayOrigin,
			/* out */ RayDirection,
			/* out */ RayTMin,
			/* out */ RayTMax,
			/* out */ RayPdf);
	}

	return true;
}

StructuredBuffer<FLightShaderConstants> LightShaderParametersArray;

[shader("raygeneration")]
void VolumetricLightmapPathTracingMainRG()
{
	uint BrickVolumeSize = 4 * 4 * 4;
	uint BrickIndex = DispatchRaysIndex().x / BrickVolumeSize + BrickBatchOffset;
	if (BrickIndex >= NumTotalBricks) return;
	uint CellIndex = DispatchRaysIndex().x % BrickVolumeSize;
	uint3 CellPosInBrick = ComputeBrickLayoutPosition(CellIndex, uint3(4, 4, 4));
	uint PaddedBrickSize = 4 + 1;
	uint3 VoxelPos = ComputeBrickLayoutPosition(DispatchRaysIndex().x / BrickVolumeSize, uint3(256, 256, 256)) * PaddedBrickSize + CellPosInBrick;
	
	bool bIsValidSample = true;	
	
	int EffectiveRenderPassIndex = FrameNumber;
#if USE_IRRADIANCE_CACHING
	if (EffectiveRenderPassIndex >= GetIrradianceCachingQuality())
	{
		EffectiveRenderPassIndex -= GetIrradianceCachingQuality();
	}
#endif
	
	if (EffectiveRenderPassIndex == 0) 
	{
		AmbientVector[VoxelPos] = float4(0, 0, 0, 0);
		
		SHCoefficients0R[VoxelPos] = float4(0, 0, 0, 0);
		SHCoefficients1R[VoxelPos] = float4(0, 0, 0, 0);
		
		SHCoefficients0G[VoxelPos] = float4(0, 0, 0, 0);
		SHCoefficients1G[VoxelPos] = float4(0, 0, 0, 0);		
		
		SHCoefficients0B[VoxelPos] = float4(0, 0, 0, 0);
		SHCoefficients1B[VoxelPos] = float4(0, 0, 0, 0);

		SkyBentNormal[VoxelPos] = float4(0, 0, 0, 0);
	}
	
	int3 CellPosInVLM = (BrickRequests[BrickIndex].xyz * 4 + CellPosInBrick) * BrickRequests[BrickIndex].w;	
	uint Seed = ComputeBrickLinearAddress(CellPosInVLM, IndirectionTextureDim * 4);
	
	float3 RandSample = float3(
		Halton(MortonEncode3(CellPosInVLM) + EffectiveRenderPassIndex, 2), 
		Halton(MortonEncode3(CellPosInVLM) + EffectiveRenderPassIndex, 3),
		Halton(MortonEncode3(CellPosInVLM) + EffectiveRenderPassIndex, 5)
	);
	float3 Jitter = RandSample;
	
	float3 JitteredCellPosInVLM = (BrickRequests[BrickIndex].xyz * 4 + CellPosInBrick + Jitter - float3(0.5, 0.5, 0.5)) * BrickRequests[BrickIndex].w;	
	float3 DetailCellSize = VolumeSize.xyz / IndirectionTextureDim / 4;
	float3 WorldPosition = VolumeMin.xyz + DetailCellSize * JitteredCellPosInVLM; // RT_LWC_TODO
	float3 TranslatedWorldPosition = WorldPosition + DFHackToFloat(PrimaryView.PreViewTranslation); // RT_LWC_TODO

	int NumValidHalfSamples = 0;
	
	UNROLL_N(2)
	for (int HalfSampleIndex = 0; HalfSampleIndex < 2; HalfSampleIndex++)
	{
		float3 ShadingNormal = float3(0, 0, (HalfSampleIndex == 0) ? 1 : -1);

		float3 RadianceValue = 0;
		float3 RadianceDirection = 0;				
		float3 DirectLightingIrradiance = 0;
#ifdef LIGHTMAP_PATH_TRACING_MAIN_RG
		FL2SHAndCorrection DirectLightingSH = (FL2SHAndCorrection)0;
#else
		FThreeBandSHVectorRGBFloat DirectLightingSH = (FThreeBandSHVectorRGBFloat)0;
#endif
		float LuminanceForFirstBounceRayGuiding = 0;
		float3 SkyLightBentNormal = 0;
		float4 PrimaryRandSample;

		if (bIsValidSample)
		{
			RandomSequence RandSequence = RandomSequenceCreate(Seed, EffectiveRenderPassIndex);
			uint SampleIndex = 4;
		
			PathTracingKernel(
				FrameNumber,
				TranslatedWorldPosition,
				ShadingNormal,
				RandSequence,
				SampleIndex,
				bIsValidSample,
				RadianceValue,
				RadianceDirection,
				DirectLightingIrradiance,
				DirectLightingSH,
				LuminanceForFirstBounceRayGuiding,
				SkyLightBentNormal,
				PrimaryRandSample);
		}

		if (any(isnan(RadianceValue)) || any(RadianceValue < 0))
		{
			bIsValidSample = false;
		}

		if (bIsValidSample)
		{
			NumValidHalfSamples++;

			FThreeBandSHVectorRGBFloat SH;
			SH.R.V0 = SHBasisFunction3Float(RadianceDirection).V0 * RadianceValue.r;
			SH.R.V1 = SHBasisFunction3Float(RadianceDirection).V1 * RadianceValue.r;
			SH.R.V2 = SHBasisFunction3Float(RadianceDirection).V2 * RadianceValue.r;
			SH.G.V0 = SHBasisFunction3Float(RadianceDirection).V0 * RadianceValue.g;
			SH.G.V1 = SHBasisFunction3Float(RadianceDirection).V1 * RadianceValue.g;
			SH.G.V2 = SHBasisFunction3Float(RadianceDirection).V2 * RadianceValue.g;
			SH.B.V0 = SHBasisFunction3Float(RadianceDirection).V0 * RadianceValue.b;
			SH.B.V1 = SHBasisFunction3Float(RadianceDirection).V1 * RadianceValue.b;
			SH.B.V2 = SHBasisFunction3Float(RadianceDirection).V2 * RadianceValue.b;

			AmbientVector[VoxelPos].x += SH.R.V0.x;
			AmbientVector[VoxelPos].y += SH.G.V0.x;
			AmbientVector[VoxelPos].z += SH.B.V0.x;
			
			SHCoefficients0R[VoxelPos] += float4(SH.R.V0.yzw, SH.R.V1.x);
			SHCoefficients1R[VoxelPos] += float4(SH.R.V1.yzw, SH.R.V2);
			
			SHCoefficients0G[VoxelPos] += float4(SH.G.V0.yzw, SH.G.V1.x);
			SHCoefficients1G[VoxelPos] += float4(SH.G.V1.yzw, SH.G.V2);		
			
			SHCoefficients0B[VoxelPos] += float4(SH.B.V0.yzw, SH.B.V1.x);
			SHCoefficients1B[VoxelPos] += float4(SH.B.V1.yzw, SH.B.V2);
				
			if (HalfSampleIndex == 0)
			{
				SkyBentNormal[VoxelPos].rgb += SkyLightBentNormal;
			}
		
		#ifndef LIGHTMAP_PATH_TRACING_MAIN_RG
			AmbientVector[VoxelPos].x += DirectLightingSH.R.V0.x;
			AmbientVector[VoxelPos].y += DirectLightingSH.G.V0.x;
			AmbientVector[VoxelPos].z += DirectLightingSH.B.V0.x;
				
			SHCoefficients0R[VoxelPos] += float4(DirectLightingSH.R.V0.yzw, DirectLightingSH.R.V1.x);
			SHCoefficients1R[VoxelPos] += float4(DirectLightingSH.R.V1.yzw, DirectLightingSH.R.V2);
				
			SHCoefficients0G[VoxelPos] += float4(DirectLightingSH.G.V0.yzw, DirectLightingSH.G.V1.x);
			SHCoefficients1G[VoxelPos] += float4(DirectLightingSH.G.V1.yzw, DirectLightingSH.G.V2);		
				
			SHCoefficients0B[VoxelPos] += float4(DirectLightingSH.B.V0.yzw, DirectLightingSH.B.V1.x);
			SHCoefficients1B[VoxelPos] += float4(DirectLightingSH.B.V1.yzw, DirectLightingSH.B.V2);
		#endif
		}
	}

	if (NumValidHalfSamples > 0)
	{
		uint SampleCount = asuint(AmbientVector[VoxelPos].w);
		AmbientVector[VoxelPos].w = asfloat(SampleCount + 1);
	}
	
	// 32 samples for single sample stationary directional light shadowing
	if (FrameNumber < 32 && length(LightShaderParametersArray[0].Direction) > 0)
	{
		if (FrameNumber == 0)
		{
			DirectionalLightShadowing[VoxelPos] = 0;
		}
		
		FLightShaderParameters LightParameters;
		LightShaderParametersArray[0].FillLightShaderParameters(LightParameters);

		float2 RandSample = float2(
			Halton(StrongIntegerHash(Seed) + FrameNumber, 5), 
			Halton(StrongIntegerHash(Seed) + FrameNumber, 7)
		);
		
		FRayDesc Ray;
		bool bIsValidRay = GenerateOcclusionRay(
			LIGHT_TYPE_DIRECTIONAL,
			LightParameters,
			TranslatedWorldPosition, float3(0, 0, 1),
			RandSample,
			/* out */ Ray.Origin,
			/* out */ Ray.Direction,
			/* out */ Ray.TMin,
			/* out */ Ray.TMax);
			
		if (bIsValidRay)
		{
			uint RayFlags = RAY_FLAG_ACCEPT_FIRST_HIT_AND_END_SEARCH;
	
			FMinimalPayload MinimalPayload = TraceLightmapVisibilityRay(
				TLAS,
				RayFlags,
				Ray);
			
			DirectionalLightShadowing[VoxelPos] += (MinimalPayload.IsMiss() ? 1 : 0) / 32.0f;
		}
	}
}

Buffer<int> LightTypeArray;
Buffer<int> ChannelIndexArray;
Buffer<int> LightSampleIndexArray;
RWTexture2D<float4> ShadowMask;
RWTexture2D<float4> ShadowMaskSampleCount;

[shader("raygeneration")]
void StationaryLightShadowTracingMainRG()
{
	uint2 BatchedLaunchIndex = DispatchRaysIndex().xy;

	uint2 LaunchIndex = uint2(BatchedLaunchIndex.x % GPreviewLightmapPhysicalTileSize, BatchedLaunchIndex.y);
	int TileIndex = BatchedLaunchIndex.x / GPreviewLightmapPhysicalTileSize;
	uint2 TexelIndexInPool = LaunchIndex + BatchedTiles[TileIndex].WorkingSetPosition;
	uint2 TexelIndexInScratch = LaunchIndex + BatchedTiles[TileIndex].ScratchPosition;

	int ChannelIndex = ChannelIndexArray[TileIndex];

	bool bGBufferSampleValid = true;

	float4 EncodedGBufferWorldPosition = GBufferWorldPosition[TexelIndexInScratch];
	
	if (EncodedGBufferWorldPosition.w == 0.0f)
	{
		bGBufferSampleValid = false;
	}

	float3 TranslatedWorldPosition = EncodedGBufferWorldPosition.xyz / WorldPositionScalar + DFHackToFloat(PrimaryView.PreViewTranslation); // RT_LWC_TODO
	
	if (bGBufferSampleValid)
	{
		float3 WorldNormal = GBufferWorldNormal[TexelIndexInScratch].xyz;
		float3 ShadingNormal = GBufferShadingNormal[TexelIndexInScratch].xyz;
		bool bMaterialTwoSided = GBufferShadingNormal[TexelIndexInScratch].w;
		
		// Needs a seed that is only related to position in virtual space to avoid seams due to per-tile calculation
		uint2 VirtualTextureSpacePosition = BatchedTiles[TileIndex].VirtualTilePosition + LaunchIndex - uint2(2, 2);
		uint Seed = VirtualTextureSpacePosition.y * BatchedTiles[TileIndex].LightmapSize.x + VirtualTextureSpacePosition.x;
		
		int LightType = LightTypeArray[TileIndex];
		FLightShaderParameters LightParameters;
		LightShaderParametersArray[TileIndex].FillLightShaderParameters(LightParameters);

		bool bAnyHemisphereRaySampleValid = false;
		uint Visibility = 0;
		
		UNROLL_N(2)
		for (int HemisphereIndex = 0; HemisphereIndex < (bMaterialTwoSided ? 2 : 1); HemisphereIndex++)
		{
			float2 RandSample = float2(
				Halton(StrongIntegerHash(Seed) + LightSampleIndexArray[TileIndex], 5), 
				Halton(StrongIntegerHash(Seed) + LightSampleIndexArray[TileIndex], 7)
			);

			float3 EffectiveWorldNormal = HemisphereIndex == 0 ? WorldNormal : -WorldNormal;

			bool bRaySampleValid = true;
			
			FRayDesc Ray;
			bRaySampleValid &= GenerateOcclusionRay(
				LightType,
				LightParameters,
				TranslatedWorldPosition, EffectiveWorldNormal,
				RandSample,
				/* out */ Ray.Origin,
				/* out */ Ray.Direction,
				/* out */ Ray.TMin,
				/* out */ Ray.TMax);
				
			ApplyRayBias(Ray.Origin, LightType == LIGHT_TYPE_DIRECTIONAL ? 1.0f : max(Ray.TMax - Ray.TMin, 0.0f), EffectiveWorldNormal);
			
			if (bRaySampleValid)
			{
				uint RayFlags = RAY_FLAG_ACCEPT_FIRST_HIT_AND_END_SEARCH;

				FMinimalPayload MinimalPayload = TraceLightmapVisibilityRay(
					TLAS,
					RayFlags,
					Ray);
				
				Visibility = max(Visibility, MinimalPayload.IsMiss() ? 1 : 0);
			}

			bAnyHemisphereRaySampleValid = bAnyHemisphereRaySampleValid || bRaySampleValid;
		}

		if (bAnyHemisphereRaySampleValid)
		{
			ShadowMask[TexelIndexInPool][ChannelIndex] = asfloat(asuint(ShadowMask[TexelIndexInPool][ChannelIndex]) + Visibility);
			uint SampleCount = asuint(ShadowMaskSampleCount[TexelIndexInPool][ChannelIndex]);
			SampleCount++;
			ShadowMaskSampleCount[TexelIndexInPool][ChannelIndex] = asfloat(SampleCount);
		}
	}
}
