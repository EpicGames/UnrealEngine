// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "LightmapCommon.ush"

RaytracingAccelerationStructure TLAS;

int StaticShadowDepthMapSuperSampleFactor;
int2 ShadowMapSize;
float4x4 LightToWorld;
float4x4 WorldToLight;
float3 LightSpaceImportanceBoundsMin;
float3 LightSpaceImportanceBoundsMax;
float MaxPossibleDistance;
RWTexture2D<half> DepthMapTexture;

[shader("raygeneration")]
void StaticShadowDepthMapTracingRG()
{
	const float2 Fraction = float2(DispatchRaysIndex().xy) / ((ShadowMapSize - 1) * StaticShadowDepthMapSuperSampleFactor);
#if LIGHT_TYPE == 0
	const float4 LightSpaceStartPosition = float4(
		LightSpaceImportanceBoundsMin.xy + Fraction * (LightSpaceImportanceBoundsMax.xy - LightSpaceImportanceBoundsMin.xy),
		LightSpaceImportanceBoundsMin.z,
		1);
	const float4 LightSpaceEndPosition = float4(LightSpaceStartPosition.x, LightSpaceStartPosition.y, LightSpaceImportanceBoundsMax.z, 1);
#elif LIGHT_TYPE == 1
	const float4 LightSpaceStartPosition = float4(0, 0, 0, 1);
	const float4 LightSpaceEndPosition = float4(
		LightSpaceImportanceBoundsMin.xy + Fraction * (LightSpaceImportanceBoundsMax.xy - LightSpaceImportanceBoundsMin.xy),
		LightSpaceImportanceBoundsMax.z,
		1);
#else
	const float Phi = Fraction.y * PI;
	const float SinPhi = sin(Phi);
	const float Theta = Fraction.x * 2 * PI;
	const float3 Direction = normalize(float3(cos(Theta) * SinPhi, sin(Theta) * SinPhi, cos(Phi)));
	
	const float4 LightSpaceStartPosition = float4(0, 0, 0, 1);
	const float4 LightSpaceEndPosition = float4(Direction * MaxPossibleDistance, 1);
#endif
	
	const float3 WorldSpaceStartPosition = mul(LightSpaceStartPosition, LightToWorld).xyz;
	const float3 WorldSpaceEndPosition = mul(LightSpaceEndPosition, LightToWorld).xyz;

	FRayDesc Ray;
	Ray.Origin = WorldSpaceStartPosition;
	Ray.Direction = normalize(WorldSpaceEndPosition - WorldSpaceStartPosition);
	Ray.TMin = 0;
	Ray.TMax = length(WorldSpaceEndPosition - WorldSpaceStartPosition);

	const uint RayFlags = 0;
	
	FMinimalPayload MinimalPayload = TraceLightmapVisibilityRay(
		TLAS,
		RayFlags,
		Ray);

	if (MinimalPayload.IsHit())
	{
	#if LIGHT_TYPE == 0 || LIGHT_TYPE == 2
		DepthMapTexture[DispatchRaysIndex().xy] = MinimalPayload.HitT / MaxPossibleDistance;
	#elif LIGHT_TYPE == 1
		const float4 WorldSpaceHitPosition = float4(Ray.Origin + Ray.Direction * MinimalPayload.HitT, 1);
		const float4 LightSpaceHitPosition = mul(WorldSpaceHitPosition, WorldToLight);
		DepthMapTexture[DispatchRaysIndex().xy] = LightSpaceHitPosition.z / MaxPossibleDistance;
	#endif
	}
	else
	{
		DepthMapTexture[DispatchRaysIndex().xy] = 1;
	}
}
