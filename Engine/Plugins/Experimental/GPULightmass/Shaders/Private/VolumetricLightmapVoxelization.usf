// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricLightmapVoxelization.usf
=============================================================================*/

#include "/Engine/Private/Common.ush"  

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

#include "BrickAllocationDefs.ush"

#define SOFTWARE_CONSERVATIVE_RASTERIZATION 1
#define SUPER_RESOLUTION_FACTOR 4 // Must match VolumetricLightmap.cpp

struct FVLMVoxelizationVSToGS
{
	float4 WorldPosition : SV_POSITION;
};

void VLMVoxelizationVS(
	FVertexFactoryInput Input,
	out FVLMVoxelizationVSToGS Output
	)
{
	ResolvedView = ResolveView();

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	Output.WorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates) - float4(DFHackToFloat(ResolvedView.PreViewTranslation), 0.0f);
	Output.WorldPosition = float4((Output.WorldPosition.xyz - VLMVoxelizationParams.VolumeCenter.xyz) / max(VLMVoxelizationParams.VolumeExtent.xyz, float3(0.0001, 0.0001, 0.0001)), 1.0f);
}

struct FVLMVoxelizationVertex
{
	float4 ScreenPosition : SV_POSITION;
	uint DominantAxis : DOMAXIS;
#if SOFTWARE_CONSERVATIVE_RASTERIZATION
	float4 AABB : AABB;
#endif
};

[maxvertexcount(3)]
void VLMVoxelizationGS(triangle FVLMVoxelizationVSToGS Inputs[3], inout TriangleStream<FVLMVoxelizationVertex> OutStream)
{
	// Dominant axis selection
	float3 TriangleNormal = normalize(cross(Inputs[1].WorldPosition.xyz - Inputs[0].WorldPosition.xyz, Inputs[2].WorldPosition.xyz - Inputs[0].WorldPosition.xyz));
	float3 AxisProjection = abs(TriangleNormal);
	
	float MaxAxisProjection = -1;
	uint DominantAxis;
	if (AxisProjection.x > MaxAxisProjection)
	{
		MaxAxisProjection = AxisProjection.x;
		DominantAxis = 0;
	}
	if (AxisProjection.y > MaxAxisProjection)
	{
		MaxAxisProjection = AxisProjection.y;
		DominantAxis = 1;
	}
	if (AxisProjection.z > MaxAxisProjection)
	{
		MaxAxisProjection = AxisProjection.z;
		DominantAxis = 2;
	}
	
	// Axis permutation
	FVLMVoxelizationVertex Vertices[3];
	
	for (int i = 0; i < 3; i++)
	{
		FVLMVoxelizationVSToGS Input = Inputs[i];
		
		FVLMVoxelizationVertex Vertex = (FVLMVoxelizationVertex)0;
		if (DominantAxis == 0)
		{
			Vertex.ScreenPosition.xyz = Input.WorldPosition.yzx;
		}
		else if (DominantAxis == 1)
		{
			Vertex.ScreenPosition.xyz = Input.WorldPosition.zxy;
		}
		else
		{
			Vertex.ScreenPosition.xyz = Input.WorldPosition.xyz;
		}
		Vertex.ScreenPosition.w = 1.0f;

		Vertex.DominantAxis = DominantAxis;
		
		Vertices[i] = Vertex;
	}

#if SOFTWARE_CONSERVATIVE_RASTERIZATION
	// Calculate AABB to clip excessive pixels caused by edge extension
	float4 AABB;
	AABB.xy = Vertices[0].ScreenPosition.xy;
	AABB.zw = Vertices[0].ScreenPosition.xy;
	AABB.xy = min(AABB.xy, Vertices[1].ScreenPosition.xy);
	AABB.zw = max(AABB.zw, Vertices[1].ScreenPosition.xy);
	AABB.xy = min(AABB.xy, Vertices[2].ScreenPosition.xy);
	AABB.zw = max(AABB.zw, Vertices[2].ScreenPosition.xy);
	AABB.xy -= float2(1.0f / VLMVoxelizationParams.VolumeMaxDim / SUPER_RESOLUTION_FACTOR, 1.0f / VLMVoxelizationParams.VolumeMaxDim / SUPER_RESOLUTION_FACTOR);
	AABB.zw += float2(1.0f / VLMVoxelizationParams.VolumeMaxDim / SUPER_RESOLUTION_FACTOR, 1.0f / VLMVoxelizationParams.VolumeMaxDim / SUPER_RESOLUTION_FACTOR);
	
	Vertices[0].AABB = AABB;
	Vertices[1].AABB = AABB;
	Vertices[2].AABB = AABB;

    float3 Edge[3] = {
		Vertices[1].ScreenPosition.xyz - Vertices[0].ScreenPosition.xyz,
		Vertices[2].ScreenPosition.xyz - Vertices[1].ScreenPosition.xyz,
		Vertices[0].ScreenPosition.xyz - Vertices[2].ScreenPosition.xyz
	};
	
	float3 SemiDiagonalVectors[4] = {
		{ sqrt(2.0f) / VLMVoxelizationParams.VolumeMaxDim / SUPER_RESOLUTION_FACTOR,  sqrt(2.0f) / VLMVoxelizationParams.VolumeMaxDim / SUPER_RESOLUTION_FACTOR, 0.0f},
		{ sqrt(2.0f) / VLMVoxelizationParams.VolumeMaxDim / SUPER_RESOLUTION_FACTOR, -sqrt(2.0f) / VLMVoxelizationParams.VolumeMaxDim / SUPER_RESOLUTION_FACTOR, 0.0f},
		{-sqrt(2.0f) / VLMVoxelizationParams.VolumeMaxDim / SUPER_RESOLUTION_FACTOR,  sqrt(2.0f) / VLMVoxelizationParams.VolumeMaxDim / SUPER_RESOLUTION_FACTOR, 0.0f},
		{-sqrt(2.0f) / VLMVoxelizationParams.VolumeMaxDim / SUPER_RESOLUTION_FACTOR, -sqrt(2.0f) / VLMVoxelizationParams.VolumeMaxDim / SUPER_RESOLUTION_FACTOR, 0.0f}
	};
	
	float MaxProj[3] = {0, 0, 0};
	float3 EdgeNormal[3];
	{
		for (int i = 0; i < 3; i++)
		{
			EdgeNormal[i] = cross(Edge[i], float3(0, 0, TriangleNormal[DominantAxis] > 0 ? 1 : -1)); // Unnormalized, doesn't matter
			for (int Dir = 0; Dir < 4; Dir++)
			{
				MaxProj[i] = max(dot(SemiDiagonalVectors[Dir], EdgeNormal[i]), MaxProj[i]);
			}
		}
	}

	Vertices[0].ScreenPosition.xyz += ( MaxProj[0] * Edge[2]/dot(Edge[2].xy,EdgeNormal[0].xy) + MaxProj[2] * Edge[0]/dot(Edge[0].xy,EdgeNormal[2].xy) );
	Vertices[1].ScreenPosition.xyz += ( MaxProj[1] * Edge[0]/dot(Edge[0].xy,EdgeNormal[1].xy) + MaxProj[0] * Edge[1]/dot(Edge[1].xy,EdgeNormal[0].xy) );
	Vertices[2].ScreenPosition.xyz += ( MaxProj[2] * Edge[1]/dot(Edge[1].xy,EdgeNormal[2].xy) + MaxProj[1] * Edge[2]/dot(Edge[2].xy,EdgeNormal[1].xy) );
#endif

	{
		for (int i = 0; i < 3; i++)
		{
			Vertices[i].ScreenPosition.z += 1;
			Vertices[i].ScreenPosition.z /= 2;
	
			OutStream.Append(Vertices[i]);
		}
	}
}

void VLMVoxelizationPS(
	FVLMVoxelizationVertex Input
	)
{
	// Recover {[-1, 1], [-1, 1], [-1, 1]} bounding box normalized coordinates from SV_Position (which is in {[0, ResX], [0, ResY], [0, 1]}
	Input.ScreenPosition.xyz /= 0.5f;
	Input.ScreenPosition.xy /= VLMVoxelizationParams.VolumeMaxDim * SUPER_RESOLUTION_FACTOR;
	Input.ScreenPosition.x -= 1;
	Input.ScreenPosition.y = 1 - Input.ScreenPosition.y;
	Input.ScreenPosition.z -= 1;

#if SOFTWARE_CONSERVATIVE_RASTERIZATION
	if (any(Input.ScreenPosition.xy < Input.AABB.xy) || any(Input.ScreenPosition.xy > Input.AABB.zw)) discard;
#endif

	float3 WorldPosition;
	
	if (Input.DominantAxis == 0)
	{
		WorldPosition.yzx = Input.ScreenPosition.xyz;
	}
	else if (Input.DominantAxis == 1)
	{
		WorldPosition.zxy = Input.ScreenPosition.xyz;
	}
	else
	{
		WorldPosition.xyz = Input.ScreenPosition.xyz;
	}

	float3 VoxelPos = (WorldPosition + 1.0f) * 0.5f * VLMVoxelizationParams.VolumeMaxDim;

	// Expand by 1 cell to avoid losing details from stitching to a higher mip
	for (int dx = -1; dx <= 1; dx++)
	{
		for (int dy = -1; dy <= 1; dy++)
		{
			for (int dz = -1; dz <= 1; dz++)
			{
				if (VLMVoxelizationParams.VoxelizeVolume[VoxelPos + float3(dx, dy, dz) / SUPER_RESOLUTION_FACTOR] == BRICK_IN_IMPORTANCE_VOLUME)
				{
					VLMVoxelizationParams.VoxelizeVolume[VoxelPos + float3(dx, dy, dz) / SUPER_RESOLUTION_FACTOR] = BRICK_ALLOCATED;
				}
			}
		}
	}
}
