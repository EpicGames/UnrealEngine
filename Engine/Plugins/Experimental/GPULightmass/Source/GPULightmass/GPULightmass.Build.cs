// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GPULightmass : ModuleRules
	{
		public GPULightmass(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"Shaders"
				}
			);

			PrivateIncludePaths.AddRange(
				new string[]
				{
					System.IO.Path.Combine(GetModuleDirectory("Renderer"), "Private"),
				}
			);
			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					// ... add other public dependencies that you statically link with here ...
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"Engine",
					"Landscape",
					"RenderCore",
					"Renderer",
					"RHI",
					"Projects",
					"EditorFramework",
					"UnrealEd",
					"SlateCore",
					"Slate",
					"IntelOIDN",
				}
				);

			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
					// ... add any modules that your module loads dynamically here ...
				}
				);
		}
	}
}
