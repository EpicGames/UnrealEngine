// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Containers/SparseArray.h"

// A handle to an object in a pool which moves and compacts its content.
class FCompactingObjectHandleBase
{
protected:
	// To allow objects contaning handles to be moved freely (relocatable outside of the handle tracking system),
	// we use an extra level of indirection so that handles are referenced by their indices ('address') in the handle allocator array.
	TSparseArray<int32>* HandleAllocator = nullptr;
	int32 HandleAddress = INDEX_NONE;

	// Pointer to the corresponding tracking table so that we can unregister the handle from the tracking table when it is destructed.
	TSet<int32>* HandleTrackingTable = nullptr;

public:
	// Default constructor which takes the default values defined below
	FCompactingObjectHandleBase() = default;
	
	FCompactingObjectHandleBase(int32 ObjectAddress, TSet<int32>* HandleTrackingTable, TSparseArray<int32>* HandleAllocator)
		: HandleAllocator(HandleAllocator)
		, HandleTrackingTable(HandleTrackingTable)
		
	{
		// Allocate space for the 'content' of the handle ('content' is the address to the real object)
		HandleAddress = HandleAllocator->Add(ObjectAddress);
		if (ObjectAddress != INDEX_NONE)
		{
			// Check for double registration
			check(!HandleTrackingTable->Contains(HandleAddress));
			// Register this handle in the tracking table
			HandleTrackingTable->Add(HandleAddress);
		}
	}

	void Unregister()
	{
		if (GetObjectAddress() != INDEX_NONE)
		{
			HandleTrackingTable->Remove(HandleAddress);
		}

		// The allocator doesn't recycle the memory immediately so we manually mark as invalid here for easier debugging
		(*HandleAllocator)[HandleAddress] = INDEX_NONE;
		
		HandleAllocator->RemoveAt(HandleAddress);
	}

	~FCompactingObjectHandleBase()
	{
		if (HandleAddress != INDEX_NONE)
		{
			Unregister();
		}
	}

	FCompactingObjectHandleBase& operator=(const FCompactingObjectHandleBase& Other)
	{
		// Destroy this handle and create a new one in place
		FCompactingObjectHandleBase::~FCompactingObjectHandleBase();
		new (this) FCompactingObjectHandleBase(Other.GetObjectAddress(), Other.HandleTrackingTable, Other.HandleAllocator);
		return *this;
	}

	FCompactingObjectHandleBase(const FCompactingObjectHandleBase& Other)
	{
		// Destroy this handle and create a new one in place
		FCompactingObjectHandleBase::~FCompactingObjectHandleBase();
		new (this) FCompactingObjectHandleBase(Other.GetObjectAddress(), Other.HandleTrackingTable, Other.HandleAllocator);
	}

	// This operator checks if two handles point to the same object (including invalid object), not for comparing handle addresses
	bool operator==(const FCompactingObjectHandleBase& Other) const
	{
		return HandleAllocator == Other.HandleAllocator && GetObjectAddress() == Other.GetObjectAddress();
	}

	int32 GetObjectAddress() const
	{
		return (*HandleAllocator)[HandleAddress];
	}

	bool IsInitialized() const
	{
		return HandleAddress != INDEX_NONE && HandleAllocator != nullptr && HandleTrackingTable != nullptr;
	}

	int32 GetObjectAddressChecked() const
	{
		check(IsInitialized());
		return GetObjectAddress();
	}

	bool PointsToValidObject() const
	{
		return IsInitialized() && GetObjectAddress() != INDEX_NONE;
	}
	
	int32 GetValidObjectAddress() const
	{
		check(PointsToValidObject());
		return GetObjectAddress();
	}
};

// A tag to prevent user from putting arbitrary containers into TObjectHandle's ObjectStorage ptr (which is practically void*)
class FDynamicallyIndexableContainer {};

template<typename ObjectType>
class TObjectHandle : public FCompactingObjectHandleBase
{
	using ObjectAccessorFuncType = ObjectType& (*)(FDynamicallyIndexableContainer* ObjectStorage, int32 ObjectAddress);
	
private:
	FDynamicallyIndexableContainer* ObjectStorage = nullptr;
	ObjectAccessorFuncType ObjectAccessor = nullptr;
		
public:
	TObjectHandle() = default;
	TObjectHandle(int32 ObjectAddress, TSet<int32>* HandleTrackingTable, TSparseArray<int32>* HandleAllocator, FDynamicallyIndexableContainer* ObjectStorage, ObjectAccessorFuncType ObjectAccessor)
	: FCompactingObjectHandleBase(ObjectAddress, HandleTrackingTable, HandleAllocator), ObjectStorage(ObjectStorage), ObjectAccessor(ObjectAccessor) {}

	ObjectType& Get() const { return ObjectAccessor(ObjectStorage, GetValidObjectAddress()); }
	ObjectType& operator*() const { return ObjectAccessor(ObjectStorage, GetValidObjectAddress()); }
	ObjectType* operator->() const { return &Get(); }
	operator ObjectType& () const { return Get(); }
};

template<typename ObjectType>
class TCompactingObjectPool : public FDynamicallyIndexableContainer
{
protected:
	TArray<TSet<int32>> HandleTrackingTablePerObject;
	TSparseArray<int32> HandleAllocator;
	TArray<ObjectType> Objects;
	
public:
	template <typename... ArgsType>
	TObjectHandle<ObjectType> Emplace(ArgsType&&... Args)
	{
		Objects.Emplace(Forward<ArgsType>(Args)...);
		HandleTrackingTablePerObject.AddDefaulted();
		return GetHandleForObject(Objects.Last());
	}

	// Core function for implementing dynamic polymorphic indexing behavior
	// An array of Base and an array of Derived are inherently different types: their indexing behavior is totally different
	// The first one indexes each element at i * sizeof(Base) while the second one does so at i * sizeof(Derived)
	// To properly access the Base of each element out of an array of Derived, the indexing must be done as Derived but
	// then the indexed element is accessed as Base
	// This means the concrete type of the element stored in this container (in other words, the type of the container)
	// must be known at indexing time, prior to the element being accessed - that means dynamic dispatches happen here
	// before the vtable of the element is accessed. A side effect of that is we don't even need the object's vtable to achieve dynamic polymorphism
	template<typename InterfaceType = ObjectType>
	static InterfaceType& ObjectAccessor(FDynamicallyIndexableContainer* ObjectStorage, int32 ObjectAddress)
	{
		// Implicit conversion from ObjectType to InterfaceType
		// Will only compile when InterfaceType is covariant with ObjectType (eg. base classes, or other convertible types)
		return static_cast<TCompactingObjectPool<ObjectType>*>(ObjectStorage)->Objects[ObjectAddress];
	}
	
	template<typename InterfaceType = ObjectType>
	TObjectHandle<InterfaceType> GetHandleForObject(ObjectType& Object)
	{
		check(&Object >= Objects.GetData() && &Object < Objects.GetData() + Objects.Num());
		int32 ObjectAddress = &Object - Objects.GetData();
		return TObjectHandle<InterfaceType>(ObjectAddress, &HandleTrackingTablePerObject[ObjectAddress], &HandleAllocator, this, ObjectAccessor);
	}
	
	// Move the last object in the pool to the address being removed to achieve compaction
	void RemoveAt(int32 ObjectAddress, bool bCheckNoActiveHandle = false)
	{
		Objects.RangeCheck(ObjectAddress);

		const bool bRemovingLastObject = ObjectAddress == Objects.Num() - 1;

		// Redirect handles pointing to the last object to the address being removed
		if (!bRemovingLastObject)
		{
			TSet<int32>& HandleTrackingTableForTheLastObject = HandleTrackingTablePerObject[Objects.Num() - 1];
			for (int32 HandleAddress : HandleTrackingTableForTheLastObject)
			{
				HandleAllocator[HandleAddress] = ObjectAddress;
			}
		}
	
		if (bCheckNoActiveHandle)
		{
			check(HandleTrackingTablePerObject[ObjectAddress].Num() == 0);
		}

		TSet<int32>& HandleTrackingTable = HandleTrackingTablePerObject[ObjectAddress];
		TArray<int32> AddressesOfHandlesToInvalidate = HandleTrackingTable.Array();
		for (int32 HandleAddress : AddressesOfHandlesToInvalidate)
		{
			HandleAllocator[HandleAddress] = INDEX_NONE;
		}

		// Overwrite the handle tracking table of the object being removed with the one for the last object
		if (!bRemovingLastObject)
		{
			HandleTrackingTablePerObject[ObjectAddress] = HandleTrackingTablePerObject[Objects.Num() - 1];
		}

		// Release the table that is no longer used
		HandleTrackingTablePerObject.Pop();

		// Move pooled object over
		// This requires the object to be movable (well, since the underlying TArray requires elements to be bitwise-relocatable, this is always satisfied)
		if (!bRemovingLastObject)
		{
			new (&Objects[ObjectAddress]) ObjectType(MoveTemp(Objects.Last()));
		}

		Objects.Pop();
	}
	
	void Remove(TObjectHandle<ObjectType>& Handle)
	{
		RemoveAt(Handle.GetObjectAddress());
	
		check(HandleAllocator[Handle.GetObjectAddress()] == INDEX_NONE);
	}

	ObjectType& operator[](int32 Index)
	{
		return Objects[Index];
	}

	const ObjectType& operator[](int32 Index) const
	{
		return Objects[Index];
	}

	int32 Num()
	{
		return Objects.Num();
	}

	struct TIterator
	{
		TArray<ObjectType>& ObjectStorage;
		int32 Index;

		TIterator& operator++()
		{
			++Index;
			return *this;
		}

		bool operator!=(const TIterator& Other)
		{
			// We don't check the list of arrays
			return Index != Other.Index;
		}

		ObjectType& operator*() const
		{
			return ObjectStorage[Index];
		}
		
		ObjectType& operator->() const
		{
			return ObjectStorage[Index];
		}
	};

	TIterator begin() { return TIterator {Objects, 0};}
	TIterator end() { return TIterator {Objects, Objects.Num()};}
};
