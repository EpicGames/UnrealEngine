// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "CompactingObjectPool.h"

struct Base
{
	int a;

	operator int&() { return a; }
};

struct Derived : Base
{
	int b;
};

void RunCompactingObjectPoolTests()
{
	TCompactingObjectPool<int32> MyPoolForInts;
	MyPoolForInts.Emplace(99);
	MyPoolForInts.Emplace(100);
	MyPoolForInts.Emplace(101);

	for (int32& myint : MyPoolForInts)
	{
		TObjectHandle<int32> MyHandleToAPooledInt = MyPoolForInts.GetHandleForObject(myint);
		ensure(*MyHandleToAPooledInt == myint);
	}

	for (auto It = MyPoolForInts.begin(); It != MyPoolForInts.end(); ++It)
	{
		int32 Index = It.Index;
		int32 myint = *It;
		int32& myint2 = *It;
	}

	for (int32 Index = 0; Index < MyPoolForInts.Num(); Index++)
	{
		int32 myint = MyPoolForInts[Index];
		int32& myint2 = MyPoolForInts[Index];
	}

	TObjectHandle<int32> MyHandleToAPooledInt2 = MyPoolForInts.GetHandleForObject(MyPoolForInts[2]);
	MyPoolForInts.RemoveAt(1);
	ensure(MyPoolForInts[1] == 101);
	ensure(MyHandleToAPooledInt2 == 101);

	TCompactingObjectPool<Base> MyPoolForBase;
	TCompactingObjectPool<Derived> MyPoolForDerived;

	MyPoolForBase.Emplace(Base {0});
	MyPoolForBase.Emplace(Base {10});
	MyPoolForBase.Emplace(Base {100});
	MyPoolForDerived.Emplace(Derived {{1}, 2});
	MyPoolForDerived.Emplace(Derived {{10}, 20});
	MyPoolForDerived.Emplace(Derived {{100}, 200});

	static_cast<int&>(MyPoolForDerived[0]) = 3;
	
	TObjectHandle<Base> MyDynamicHandle;
	MyDynamicHandle = MyPoolForBase.GetHandleForObject(MyPoolForBase[0]);
	ensure(MyDynamicHandle->a == 0);
	MyDynamicHandle = MyPoolForDerived.GetHandleForObject<Base>(MyPoolForDerived[0]);
	ensure(MyDynamicHandle->a == 3);
	
	// Generate a handle that manipulates the Base::a part of a Derived. Needs operator int&() to be defined.
	// Evil.
	TObjectHandle<int> MyDynamicHandleToIntPart = MyPoolForDerived.GetHandleForObject<int>(MyPoolForDerived[0]);
	// TObjectHandle<Derived> dhb = MyPoolForBase.GetHandleForObject<Derived>(MyPoolForBase[0]); // Shouldn't compile
	
	*MyDynamicHandleToIntPart = 4;
	ensure(MyDynamicHandle->a == 4);

	MyPoolForBase.RemoveAt(0);
	MyPoolForDerived.RemoveAt(0);
	
	ensure(!MyDynamicHandle.PointsToValidObject());
	ensure(!MyDynamicHandleToIntPart.PointsToValidObject());
}

