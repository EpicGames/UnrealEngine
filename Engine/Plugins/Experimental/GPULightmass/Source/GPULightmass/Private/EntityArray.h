// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if 0
// Very unfortunately, this version doesn't work with our T[Sparse]Array implementations because they don't call copy/move constructors on ResizeGrow (which uses Realloc directly)
template<typename T>
class TEntityArray
{
public:
	class EntityRefType
	{
	public:
		EntityRefType(const T& Element, TEntityArray<T>& Container);
		EntityRefType(const EntityRefType& Other);
		~EntityRefType();

		T* operator->() { return &Container.Elements[Id]; }
		operator T& () { return Container.Elements[Id]; }
	private:
		int32 Id;
		TEntityArray<T>& Container;
		friend TEntityArray<T>;
	};

	void RemoveAt(int32 Index)
	{
		Elements.RangeCheck(Index);

		if (Index != Elements.Num() - 1)
		{
			for (EntityRefType* Ref : Refs[Elements.Num() - 1])
			{
				Ref->Id = Index;
			}
		}

		auto RefsToInvalidate = Refs[Index].Array();
		for (EntityRefType* Ref : RefsToInvalidate)
		{
			Ref->Id = INDEX_NONE;
		}

		if (Index != Elements.Num() - 1)
		{
			Refs[Index] = Refs[Elements.Num() - 1];
		}

		Refs.Remove(Elements.Num() - 1);

		if (Index != Elements.Num() - 1)
		{
			Elements[Index].~T();
			new (&Elements[Index]) T(MoveTemp(Elements.Last()));
		}

		Elements.RemoveAt(Elements.Num() - 1);
	}

	void Remove(EntityRefType& Ref)
	{
		RemoveAt(Ref.Id);

		check(Ref.Id == INDEX_NONE);
	}

	template <typename... ArgsType>
	EntityRefType Emplace(ArgsType&&... Args)
	{
		Elements.Emplace(Forward<ArgsType>(Args)...);
		return EntityRefType(Elements.Last(), *this);
	}

private:
	TArray<T> Elements;
	TMap<int32, TSet<EntityRefType*>> Refs;
};

template<typename T>
TEntityArray<T>::EntityRefType::EntityRefType(const T& Element, TEntityArray<T>& Container)
	: Container(Container)
{
	check(Container.Elements.GetData() <= &Element && Container.Elements.GetData() + Container.Elements.Num() > & Element);
	Id = &Element - Container.Elements.GetData();
	if (!Container.Refs.Contains(Id))
	{
		Container.Refs.Add(Id, TSet<EntityRefType*>{});
	}
	check(!Container.Refs[Id].Contains(this));
	Container.Refs[Id].Add(this);
}

template<typename T>
TEntityArray<T>::EntityRefType::EntityRefType(const EntityRefType& Other)
	: Id(Other.Id)
	, Container(Other.Container)
{
	if (!Container.Refs.Contains(Id))
	{
		Container.Refs.Add(Id, TSet<EntityRefType*>{});
	}
	check(!Container.Refs[Id].Contains(this));
	Container.Refs[Id].Add(this);
}

template<typename T>
TEntityArray<T>::EntityRefType::~EntityRefType()
{
	if (Id != INDEX_NONE)
	{
		Container.Refs[Id].Remove(this);
		if (Container.Refs[Id].Num() == 0)
		{
			Container.Refs.Remove(Id);
		}
	}
}

#else

using RefAddr = int32;

class FGenericEntityRef
{
public:
	FGenericEntityRef() : Addr(-1), RefsArrayPtr(nullptr), RefAllocatorPtr(nullptr)
	{
	}

	FGenericEntityRef(int32 ElementId, TArray<TSet<RefAddr>>& Refs, TSparseArray<int32>& RefAllocator)
		: RefsArrayPtr(&Refs)
		, RefAllocatorPtr(&RefAllocator)
	{
		Addr = RefAllocator.Add(ElementId);
		if (ElementId != INDEX_NONE)
		{
			check(!Refs[ElementId].Contains(Addr));
			Refs[ElementId].Add(Addr);
		}
	}

	void Unregister()
	{
		int32 ElementId = GetElementId();
		if (ElementId != INDEX_NONE)
		{
			(*RefsArrayPtr)[ElementId].Remove(Addr);
		}

		(*RefAllocatorPtr)[Addr] = INDEX_NONE;
		RefAllocatorPtr->RemoveAt(Addr);
	}

	FGenericEntityRef& operator=(const FGenericEntityRef& Other)
	{
		Unregister();

		RefsArrayPtr = Other.RefsArrayPtr;
		RefAllocatorPtr = Other.RefAllocatorPtr;
		int32 ElementId = (*RefAllocatorPtr)[Other.Addr];
		check(ElementId < RefsArrayPtr->Num());
		Addr = RefAllocatorPtr->Add(ElementId);
		if (ElementId != INDEX_NONE)
		{
			(*RefsArrayPtr)[ElementId].Add(Addr);
		}

		return *this;
	}

	FGenericEntityRef(const FGenericEntityRef& Other)
	{
		RefsArrayPtr = Other.RefsArrayPtr;
		RefAllocatorPtr = Other.RefAllocatorPtr;
		int32 ElementId = (*RefAllocatorPtr)[Other.Addr];
		check(ElementId < RefsArrayPtr->Num());
		Addr = RefAllocatorPtr->Add(ElementId);
		if (ElementId != INDEX_NONE)
		{
			(*RefsArrayPtr)[ElementId].Add(Addr);
		}
	}

	~FGenericEntityRef()
	{
		Unregister();
	}

	int32 GetElementId() const
	{
		return (*RefAllocatorPtr)[Addr];
	}

	bool IsValid() const
	{
		return Addr != INDEX_NONE && RefsArrayPtr != nullptr && RefAllocatorPtr != nullptr && GetElementId() != INDEX_NONE;
	}

	int32 GetElementIdChecked() const
	{
		check(IsValid());
		return (*RefAllocatorPtr)[Addr];
	}

	bool operator==(const FGenericEntityRef& Other) const
	{
		return RefsArrayPtr == Other.RefsArrayPtr && GetElementId() == Other.GetElementId();
	}

protected:
	RefAddr Addr;
	TArray<TSet<RefAddr>>* RefsArrayPtr;
	TSparseArray<int32>* RefAllocatorPtr;
};

template<typename T>
class TEntityArray
{
public:
	class EntityRefType : public FGenericEntityRef
	{
	public:
		EntityRefType();
		EntityRefType(const EntityRefType& Other);
		EntityRefType(const T& Element, TEntityArray<T>& Container);
		EntityRefType(TEntityArray<T>& Container);
		~EntityRefType() {}

		T* GetUnderlyingAddress_Unsafe() const { return &(*ElementsPtr)[GetElementIdChecked()]; }
		T* operator->() const { return &(*ElementsPtr)[GetElementIdChecked()]; }
		T& GetReference_Unsafe() const { return (*ElementsPtr)[GetElementIdChecked()]; }
		operator T& () const { return (*ElementsPtr)[GetElementIdChecked()]; }
		bool IsValid() const
		{
			return FGenericEntityRef::IsValid() && ElementsPtr != nullptr && GetElementId() < ElementsPtr->Num();
		}
		bool operator==(const EntityRefType& Other) const
		{
			return FGenericEntityRef::operator==(Other) && ElementsPtr == Other.ElementsPtr;
		}
	private:
		TArray<T>* ElementsPtr;
		friend TEntityArray<T>;
	};

	void RemoveAt(int32 Index, bool bCheckNoActiveRef = false)
	{
		Elements.RangeCheck(Index);

		if (Index != Elements.Num() - 1)
		{
			for (int32 RefAddr : Refs[Elements.Num() - 1])
			{
				RefAllocator[RefAddr] = Index;
			}
		}

		if (bCheckNoActiveRef)
		{
			check(Refs[Index].Num() == 0);
		}

		auto RefsToInvalidate = Refs[Index].Array();
		for (int32 RefAddr : RefsToInvalidate)
		{
			RefAllocator[RefAddr] = INDEX_NONE;
		}

		if (Index != Elements.Num() - 1)
		{
			Refs[Index] = Refs[Elements.Num() - 1];
		}

		Refs.RemoveAt(Elements.Num() - 1);

		if (Index != Elements.Num() - 1)
		{
			Elements[Index].~T();
			new (&Elements[Index]) T(MoveTemp(Elements.Last()));
		}

		Elements.RemoveAt(Elements.Num() - 1);
	}

	void Remove(EntityRefType& Ref)
	{
		RemoveAt(RefAllocator[Ref.Addr]);

		check(RefAllocator[Ref.Addr] == INDEX_NONE);
	}

	template <typename... ArgsType>
	EntityRefType Emplace(ArgsType&&... Args)
	{
		Elements.Emplace(Forward<ArgsType>(Args)...);
		Refs.AddDefaulted();
		return EntityRefType(Elements.Last(), *this);
	}

	EntityRefType CreateNullRef()
	{
		return EntityRefType(*this);
	}

	TArray<T> Elements;
protected:
	TArray<TSet<RefAddr>> Refs;
	TSparseArray<int32> RefAllocator;
};

template<typename T>
TEntityArray<T>::EntityRefType::EntityRefType()
	: FGenericEntityRef()
	, ElementsPtr(nullptr)
{
}

template<typename T>
TEntityArray<T>::EntityRefType::EntityRefType(const EntityRefType& Other)
	: FGenericEntityRef(Other)
	, ElementsPtr(Other.ElementsPtr)
{
}

template<typename T>
TEntityArray<T>::EntityRefType::EntityRefType(const T& Element, TEntityArray<T>& Container)
	: FGenericEntityRef(&Element - Container.Elements.GetData(), Container.Refs, Container.RefAllocator)
	, ElementsPtr(&Container.Elements)
{
	check(GetElementId() < ElementsPtr->Num());
}

template<typename T>
TEntityArray<T>::EntityRefType::EntityRefType(TEntityArray<T>& Container)
	: FGenericEntityRef(INDEX_NONE, Container.Refs, Container.RefAllocator)
	, ElementsPtr(&Container.Elements)
{
}

template<typename T>
uint32 GetTypeHash(const typename TEntityArray<T>::EntityRefType& Ref)
{
	return 0;
}

#endif