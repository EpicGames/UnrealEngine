// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPULightmass.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Misc/ScopedSlowTask.h"
#include "Components/SkyLightComponent.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Async/Async.h"
#include "LightmapRenderer.h"
#include "GPULightmassSettings.h"

#define LOCTEXT_NAMESPACE "StaticLightingSystem"

FGPULightmass::FGPULightmass(UWorld* InWorld, FGPULightmassModule* InGPULightmassModule, UGPULightmassSettings* InSettings)
	: World(InWorld)
	, GPULightmassModule(InGPULightmassModule)
	, Settings(InSettings)
	, Scene(this)
	, LightBuildPercentage(0)
{
	check(IsInGameThread());
	check(Settings);

	InstallGameThreadEventHooks();

	SettingsGuard = MakeUnique<FGCObjectScopeGuard>(Settings);

	StartTime = FPlatformTime::Seconds();
}

void FGPULightmass::ShowLightBuildNotification()
{
	if (!IsEngineExitRequested() && !LightBuildNotification.IsValid())
	{
		// Start the lightmass 'progress' notification
		FNotificationInfo Info(LOCTEXT("LightBuildMessage", "Building lighting"));
		Info.bFireAndForget = false;

		if (Settings->Mode == EGPULightmassMode::BakeWhatYouSee)
		{
			Info.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("Save", "Save and Stop"),
			FText::GetEmpty(),
			FSimpleDelegate::CreateLambda([WorldPtr = World, this]() { 
				this->Scene.ApplyFinishedLightmapsToWorld(); 
				WorldPtr->GetSubsystem<UGPULightmassSubsystem>()->Stop();
			})));
		}
	
		Info.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("LightBuildCancel", "Cancel"),
			LOCTEXT("LightBuildCancelToolTip", "Cancels the lighting build in progress."),
			FSimpleDelegate::CreateLambda([WorldPtr = World]() {
				WorldPtr->GetSubsystem<UGPULightmassSubsystem>()->Stop();
				WorldPtr->GetSubsystem<UGPULightmassSubsystem>()->OnLightBuildEnded().Broadcast();
				})));

		LightBuildNotification = FSlateNotificationManager::Get().AddNotification(Info);
		if (LightBuildNotification.IsValid())
		{
			LightBuildNotification->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}
}

void FGPULightmass::RemoveLightBuildNotification()
{
	if (LightBuildNotification.IsValid())
	{
		// Shows a notification that fades out slowly
		FText CompletedText = LOCTEXT("LightBuildDoneMessage", "Lighting build completed");
		LightBuildNotification->SetText(CompletedText);
		LightBuildNotification->SetCompletionState(SNotificationItem::CS_Success);
		LightBuildNotification->ExpireAndFadeout();
		LightBuildNotification.Reset();
	}	
}

void FGPULightmass::GameThreadDestroy()
{
	check(IsInGameThread());

	UE_LOGF(LogGPULightmass, Log, "Total lighting time: %ls", *FPlatformTime::PrettyTime(FPlatformTime::Seconds() - StartTime));

	RemoveGameThreadEventHooks();
	RemoveLightBuildNotification();

	Scene.RemoveAllComponents();
}

FGPULightmass::~FGPULightmass()
{
	// RenderThreadDestroy
	check(IsInRenderingThread());
}

void FGPULightmass::InstallGameThreadEventHooks()
{
	FWorldDelegates::OnPreWorldFinishDestroy.AddRaw(this, &FGPULightmass::OnPreWorldFinishDestroy);

	FStaticLightingSystemInterface::OnPrimitiveComponentRegistered.AddRaw(this, &FGPULightmass::OnPrimitiveComponentRegistered);
	FStaticLightingSystemInterface::OnPrimitiveComponentUnregistered.AddRaw(this, &FGPULightmass::OnPrimitiveComponentUnregistered);
	FStaticLightingSystemInterface::OnLightComponentRegistered.AddRaw(this, &FGPULightmass::OnLightComponentRegistered);
	FStaticLightingSystemInterface::OnLightComponentUnregistered.AddRaw(this, &FGPULightmass::OnLightComponentUnregistered);
	FStaticLightingSystemInterface::OnStationaryLightChannelReassigned.AddRaw(this, &FGPULightmass::OnStationaryLightChannelReassigned);
	FStaticLightingSystemInterface::OnLightmassImportanceVolumeModified.AddRaw(this, &FGPULightmass::OnLightmassImportanceVolumeModified);
	FStaticLightingSystemInterface::OnMaterialInvalidated.AddRaw(this, &FGPULightmass::OnMaterialInvalidated);
	FStaticLightingSystemInterface::OnSkyAtmosphereModified.AddRaw(this, &FGPULightmass::OnSkyAtmosphereModified);
}

void FGPULightmass::RemoveGameThreadEventHooks()
{
	FWorldDelegates::OnPreWorldFinishDestroy.RemoveAll(this);

	FStaticLightingSystemInterface::OnPrimitiveComponentRegistered.RemoveAll(this);
	FStaticLightingSystemInterface::OnPrimitiveComponentUnregistered.RemoveAll(this);
	FStaticLightingSystemInterface::OnLightComponentRegistered.RemoveAll(this);
	FStaticLightingSystemInterface::OnLightComponentUnregistered.RemoveAll(this);
	FStaticLightingSystemInterface::OnStationaryLightChannelReassigned.RemoveAll(this);
	FStaticLightingSystemInterface::OnLightmassImportanceVolumeModified.RemoveAll(this);
	FStaticLightingSystemInterface::OnMaterialInvalidated.RemoveAll(this);
	FStaticLightingSystemInterface::OnSkyAtmosphereModified.RemoveAll(this);
}

void FGPULightmass::OnPrimitiveComponentRegistered(UPrimitiveComponent* InComponent)
{
	if (InComponent->GetWorld() != World) return;

	if (!InComponent->IsRegistered()) return;
	if (!InComponent->IsVisible()) return;

	check(InComponent->HasValidSettingsForStaticLighting(false));

	if (ULandscapeComponent* LandscapeComponent = Cast<ULandscapeComponent>(InComponent))
	{
		Scene.AddGeometryInstanceFromComponent(LandscapeComponent);
	}
	else if (UInstancedStaticMeshComponent* InstancedStaticMeshComponent = Cast<UInstancedStaticMeshComponent>(InComponent))
	{
		Scene.AddGeometryInstanceFromComponent(InstancedStaticMeshComponent);
	}
	else if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(InComponent))
	{
		Scene.AddGeometryInstanceFromComponent(StaticMeshComponent);
	}
}

void FGPULightmass::OnPrimitiveComponentUnregistered(UPrimitiveComponent* InComponent)
{
	if (InComponent->GetWorld() != World) return;

	if (ULandscapeComponent* LandscapeComponent = Cast<ULandscapeComponent>(InComponent))
	{
		Scene.RemoveGeometryInstanceFromComponent(LandscapeComponent);
	}
	if (UInstancedStaticMeshComponent* InstancedStaticMeshComponent = Cast<UInstancedStaticMeshComponent>(InComponent))
	{
		Scene.RemoveGeometryInstanceFromComponent(InstancedStaticMeshComponent);
	}
	else if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(InComponent))
	{
		Scene.RemoveGeometryInstanceFromComponent(StaticMeshComponent);
	}
}

void FGPULightmass::OnLightComponentRegistered(ULightComponentBase* InComponent)
{
	if (InComponent->GetWorld() != World) return;

	if (!InComponent->IsRegistered()) return;
	if (!InComponent->IsVisible()) return;

	if (UDirectionalLightComponent* DirectionalLight = Cast<UDirectionalLightComponent>(InComponent))
	{
		Scene.AddLight(DirectionalLight);
	}
	else if (URectLightComponent* RectLight = Cast<URectLightComponent>(InComponent))
	{
		Scene.AddLight(RectLight);
	}
	else if (USpotLightComponent* SpotLight = Cast<USpotLightComponent>(InComponent))
	{
		Scene.AddLight(SpotLight);
	}
	else if (UPointLightComponent* PointLight = Cast<UPointLightComponent>(InComponent))
	{
		Scene.AddLight(PointLight);
	}
	else if (USkyLightComponent* SkyLight = Cast<USkyLightComponent>(InComponent))
	{
		Scene.AddLight(SkyLight);
	}
}

void FGPULightmass::OnLightComponentUnregistered(ULightComponentBase* InComponent)
{
	if (InComponent->GetWorld() != World) return;

	if (UDirectionalLightComponent* DirectionalLight = Cast<UDirectionalLightComponent>(InComponent))
	{
		Scene.RemoveLight(DirectionalLight);
	}
	else if (URectLightComponent* RectLight = Cast<URectLightComponent>(InComponent))
	{
		Scene.RemoveLight(RectLight);
	}
	else if (USpotLightComponent* SpotLight = Cast<USpotLightComponent>(InComponent))
	{
		Scene.RemoveLight(SpotLight);
	}
	else if (UPointLightComponent* PointLight = Cast<UPointLightComponent>(InComponent))
	{
		Scene.RemoveLight(PointLight);
	}
	else if (USkyLightComponent* SkyLight = Cast<USkyLightComponent>(InComponent))
	{
		Scene.RemoveLight(SkyLight);
	}
}

void FGPULightmass::OnStationaryLightChannelReassigned(ULightComponentBase* InComponent, int32 NewShadowMapChannel)
{
	if (InComponent->GetWorld() != World) return;

	if (UDirectionalLightComponent* DirectionalLight = Cast<UDirectionalLightComponent>(InComponent))
	{
		if (Scene.HasLight(DirectionalLight))
		{
			Scene.RemoveLight(DirectionalLight);
			Scene.AddLight(DirectionalLight);
		}
	}
	else if (URectLightComponent* RectLight = Cast<URectLightComponent>(InComponent))
	{
		if (Scene.HasLight(RectLight))
		{
			Scene.RemoveLight(RectLight);
			Scene.AddLight(RectLight);
		}
	}
	else if (USpotLightComponent* SpotLight = Cast<USpotLightComponent>(InComponent))
	{
		if (Scene.HasLight(SpotLight))
		{
			Scene.RemoveLight(SpotLight);
			Scene.AddLight(SpotLight);
		}
	}
	else if (UPointLightComponent* PointLight = Cast<UPointLightComponent>(InComponent))
	{
		if (Scene.HasLight(PointLight))
		{
			Scene.RemoveLight(PointLight);
			Scene.AddLight(PointLight);
		}
	}
}

void FGPULightmass::OnSkyAtmosphereModified()
{
	Scene.OnSkyAtmosphereModified();
}

void FGPULightmass::OnPreWorldFinishDestroy(UWorld* InWorld)
{
	if (InWorld != World) return;

	// This calls destructor of FGPULightmass
	GPULightmassModule->RemoveGPULightmassFromWorld(World);
}

void FGPULightmass::EditorTick()
{
	Scene.BackgroundTick();
}

const FMeshMapBuildData* FGPULightmass::GetPrimitiveMeshMapBuildData(const UPrimitiveComponent* Component, int32 LODIndex)
{
	if (Component->GetWorld() != World) return nullptr;

	return Scene.GetComponentLightmapData(Component, LODIndex);
}

const FLightComponentMapBuildData* FGPULightmass::GetLightComponentMapBuildData(const ULightComponent* Component)
{
	if (Component->GetWorld() != World) return nullptr;

	return Scene.GetComponentLightmapData(Component);
}

const FPrecomputedVolumetricLightmap* FGPULightmass::GetPrecomputedVolumetricLightmap()
{
	check(IsInParallelRenderingThread());

	return Scene.RenderState.VolumetricLightmapRenderer->GetPrecomputedVolumetricLightmapForPreview();
}

void FGPULightmass::OnLightmassImportanceVolumeModified()
{
	Scene.bNeedsVoxelization = true;
}

void FGPULightmass::OnMaterialInvalidated(FMaterialRenderProxy* Material)
{
	ENQUEUE_RENDER_COMMAND(OnMaterialInvalidatedRenderThread)([&RenderState = Scene.RenderState](FRHICommandListImmediate&) mutable
	{
		if (RenderState.CachedRayTracingScene.IsValid())
		{
			RenderState.CachedRayTracingScene.Reset();
			UE_LOGF(LogGPULightmass, Log, "Cached ray tracing scene is invalidated due to material changes");
		}
	});
}

void FGPULightmass::StartRecordingVisibleTiles() 
{
	ENQUEUE_RENDER_COMMAND(BackgroundTickRenderThread)([&RenderState = Scene.RenderState](FRHICommandListImmediate&) mutable {
		RenderState.LightmapRenderer->RecordedTileRequests.Empty();
		RenderState.LightmapRenderer->bIsRecordingTileRequests = true;
	});
}
void FGPULightmass::EndRecordingVisibleTiles() 
{
	ENQUEUE_RENDER_COMMAND(BackgroundTickRenderThread)([&RenderState = Scene.RenderState](FRHICommandListImmediate&) mutable {
		RenderState.LightmapRenderer->bIsRecordingTileRequests = false;
		RenderState.LightmapRenderer->DeduplicateRecordedTileRequests();
	});
}


#undef LOCTEXT_NAMESPACE
