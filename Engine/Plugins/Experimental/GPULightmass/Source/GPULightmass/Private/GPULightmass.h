// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VolumetricLightmap.h"
#include "GPULightmassModule.h"
#include "GPULightmassSettings.h"
#include "Scene/Scene.h"

class SNotificationItem;

class FGPULightmass : public IStaticLightingSystem
{
public:
	FGPULightmass(UWorld* InWorld, FGPULightmassModule* GPULightmassModule, UGPULightmassSettings* InSettings = nullptr);
	virtual ~FGPULightmass();
	void GameThreadDestroy();

	virtual const FMeshMapBuildData* GetPrimitiveMeshMapBuildData(const UPrimitiveComponent* Component, int32 LODIndex) override;
	virtual const FLightComponentMapBuildData* GetLightComponentMapBuildData(const ULightComponent* Component) override;
	virtual const FPrecomputedVolumetricLightmap* GetPrecomputedVolumetricLightmap() override;

	UWorld* World;
	FGPULightmassModule* GPULightmassModule;
	UGPULightmassSettings* Settings;
	TUniquePtr<FGCObjectScopeGuard> SettingsGuard;

	GPULightmass::FScene Scene;

	void InstallGameThreadEventHooks();
	void RemoveGameThreadEventHooks();

	void ShowLightBuildNotification();
	void RemoveLightBuildNotification();

	void StartRecordingVisibleTiles();
	void EndRecordingVisibleTiles();

	// Game thread objects. Must be destroyed in GameThreadDestroy()
	TSharedPtr<SNotificationItem> LightBuildNotification;
	int32 LightBuildPercentage;
	
	double StartTime = 0;

	void EditorTick();

private:
	// Game thread event hooks
	void OnPreWorldFinishDestroy(UWorld* World);

	void OnPrimitiveComponentRegistered(UPrimitiveComponent* InComponent);
	void OnPrimitiveComponentUnregistered(UPrimitiveComponent* InComponent);
	void OnLightComponentRegistered(ULightComponentBase* InComponent);
	void OnLightComponentUnregistered(ULightComponentBase* InComponent);
	void OnStationaryLightChannelReassigned(ULightComponentBase* InComponent, int32 NewShadowMapChannel);
	void OnLightmassImportanceVolumeModified();
	void OnMaterialInvalidated(FMaterialRenderProxy* Material);
	void OnSkyAtmosphereModified();
};
