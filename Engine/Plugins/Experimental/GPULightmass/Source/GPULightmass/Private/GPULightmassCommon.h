// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

static const int32 GPreviewLightmapVirtualTileSize = 64;
static const int32 GPreviewLightmapTileBorderSize = 2;
static const int32 GPreviewLightmapPhysicalTileSize = GPreviewLightmapVirtualTileSize + 2 * GPreviewLightmapTileBorderSize;
static const int32 GPreviewLightmapMipmapMaxLevel = 14;

struct FTileVirtualCoordinates
{
	FIntPoint Position{ EForceInit::ForceInitToZero };
	int32 MipLevel = -1;

	FTileVirtualCoordinates() = default;

	FTileVirtualCoordinates(uint32 vAddress, uint8 vLevel) : Position((int32)FMath::ReverseMortonCode2(vAddress), (int32)FMath::ReverseMortonCode2(vAddress >> 1)), MipLevel(vLevel) {}
	FTileVirtualCoordinates(FIntPoint InPosition, uint8 vLevel) : Position(InPosition), MipLevel(vLevel) {}

	uint32 GetVirtualAddress() const { return FMath::MortonCode2(Position.X) | (FMath::MortonCode2(Position.Y) << 1); }

	bool operator==(const FTileVirtualCoordinates& Rhs) const
	{
		return Position == Rhs.Position && MipLevel == Rhs.MipLevel;
	}
};

FORCEINLINE uint32 GetTypeHash(const FTileVirtualCoordinates& Tile)
{
	return HashCombine(GetTypeHash(Tile.Position), GetTypeHash(Tile.MipLevel));
}
