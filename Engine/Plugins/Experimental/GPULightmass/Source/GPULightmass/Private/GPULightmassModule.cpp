// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPULightmassModule.h"
#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "RenderingThread.h"
#include "Interfaces/IPluginManager.h"
#include "ShaderCore.h"
#include "GPULightmass.h"
#include "UnrealEdGlobals.h"
#include "LevelEditorViewport.h"

#define LOCTEXT_NAMESPACE "StaticLightingSystem"

DEFINE_LOG_CATEGORY(LogGPULightmass);

IMPLEMENT_MODULE( FGPULightmassModule, GPULightmass )

void FGPULightmassModule::RunSelfTests()
{
	extern void RunCompactingObjectPoolTests();
	RunCompactingObjectPoolTests();
}

void FGPULightmassModule::StartupModule()
{
	RunSelfTests();
	
	UE_LOGF(LogGPULightmass, Log, "GPULightmass module is loaded");
	
	// Maps virtual shader source directory /Plugin/GPULightmass to the plugin's actual Shaders directory.
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("GPULightmass"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/GPULightmass"), PluginShaderDir);

	FStaticLightingSystemInterface::Get()->RegisterImplementation(FName(TEXT("GPULightmass")), this);
}

void FGPULightmassModule::ShutdownModule()
{
	FStaticLightingSystemInterface::Get()->UnregisterImplementation(FName(TEXT("GPULightmass")));

	{
		check(StaticLightingSystems.Num() == 0);
	}
}

FGPULightmass* FGPULightmassModule::CreateGPULightmassForWorld(UWorld* InWorld, UGPULightmassSettings* Settings)
{
	check(StaticLightingSystems.Find(InWorld) == nullptr);

	FGPULightmass* GPULightmass = new FGPULightmass(InWorld, this, Settings);

	StaticLightingSystems.Add(InWorld, GPULightmass);

	FlushRenderingCommands();

	OnStaticLightingSystemsChanged.Broadcast();

	return GPULightmass;
}

void FGPULightmassModule::RemoveGPULightmassFromWorld(UWorld* InWorld)
{
	if (StaticLightingSystems.Find(InWorld) != nullptr)
	{
		FGPULightmass* GPULightmass = StaticLightingSystems[InWorld];

		StaticLightingSystems.Remove(InWorld);

		GPULightmass->GameThreadDestroy();

		ENQUEUE_RENDER_COMMAND(DeleteGPULightmassCmd)([GPULightmass](FRHICommandListImmediate& RHICmdList) { delete GPULightmass; });

		FlushRenderingCommands();

		OnStaticLightingSystemsChanged.Broadcast();
	}
}

IStaticLightingSystem* FGPULightmassModule::GetStaticLightingSystemForWorld(UWorld* InWorld)
{
	return StaticLightingSystems.Find(InWorld) != nullptr ? *StaticLightingSystems.Find(InWorld) : nullptr;
}

void FGPULightmassModule::EditorTick()
{
	TArray<FGPULightmass*> FinishedStaticLightingSystems;

	for (auto& StaticLightingSystem : StaticLightingSystems)
	{
		FGPULightmass* GPULightmass = StaticLightingSystem.Value;
		GPULightmass->EditorTick();
		if (GPULightmass->LightBuildPercentage >= 100 && GPULightmass->Settings->Mode != EGPULightmassMode::BakeWhatYouSee)
		{
			FinishedStaticLightingSystems.Add(GPULightmass);
		}
	}

	for (FGPULightmass* StaticLightingSystem : FinishedStaticLightingSystems)
	{
		StaticLightingSystem->World->GetSubsystem<UGPULightmassSubsystem>()->Stop();
	}
}

bool FGPULightmassModule::IsStaticLightingSystemRunning()
{
	return StaticLightingSystems.Num() > 0;
}

bool FGPULightmassModule::IsRealtimeOn()
{
	return GCurrentLevelEditingViewportClient && GCurrentLevelEditingViewportClient->IsRealtime();
}

#undef LOCTEXT_NAMESPACE