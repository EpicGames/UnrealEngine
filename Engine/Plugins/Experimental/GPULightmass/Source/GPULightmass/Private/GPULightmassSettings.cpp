// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPULightmassSettings.h"
#include "EngineUtils.h"
#include "GPULightmassModule.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Misc/ScopedSlowTask.h"
#include "LandscapeComponent.h"
#include "GPULightmass.h"
#include "Editor.h"
#include "LevelEditorViewport.h"
#include "AssetCompilingManager.h"
#include "MeshUtilitiesCommon.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MessageDialog.h"
#include "EngineAnalytics.h"


#define LOCTEXT_NAMESPACE "StaticLightingSystem"

AGPULightmassSettingsActor::AGPULightmassSettingsActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.DoNotCreateDefaultSubobject(TEXT("Sprite")))
{
#if WITH_EDITORONLY_DATA
	bActorLabelEditable = false;
#endif // WITH_EDITORONLY_DATA
	bIsEditorOnlyActor = true;

	Settings = ObjectInitializer.CreateDefaultSubobject<UGPULightmassSettings>(this, TEXT("GPULightmassSettings"));
}

void UGPULightmassSettings::ApplyImmediateSettingsToRunningInstances()
{
	// Replicate value to any running instances
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (World)
	{
		FGPULightmassModule& GPULightmassModule = FModuleManager::LoadModuleChecked<FGPULightmassModule>(TEXT("GPULightmass"));
		FGPULightmass* GPULightmass = (FGPULightmass*)GPULightmassModule.GetStaticLightingSystemForWorld(World);
		if (GPULightmass)
		{
			GPULightmass->Settings->bShowProgressBars = bShowProgressBars;
			GPULightmass->Settings->TilePassesInSlowMode = FMath::Min(TilePassesInSlowMode, GISamples - 1);
			GPULightmass->Settings->TilePassesInFullSpeedMode = FMath::Min(TilePassesInFullSpeedMode, GISamples - 1);
			GPULightmass->Settings->bVisualizeIrradianceCache = bVisualizeIrradianceCache;
		}
	}
}

#if WITH_EDITOR
void UGPULightmassSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged)
	{
		FName PropertyName = PropertyThatChanged->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UGPULightmassSettings, bUseIrradianceCaching))
		{
			if (!bUseIrradianceCaching)
			{
				bUseFirstBounceRayGuiding = false;
			}
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UGPULightmassSettings, bShowProgressBars))
		{
			ApplyImmediateSettingsToRunningInstances();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UGPULightmassSettings, TilePassesInSlowMode))
		{
			ApplyImmediateSettingsToRunningInstances();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UGPULightmassSettings, TilePassesInFullSpeedMode))
		{
			ApplyImmediateSettingsToRunningInstances();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UGPULightmassSettings, bVisualizeIrradianceCache))
		{
			ApplyImmediateSettingsToRunningInstances();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UGPULightmassSettings::CanEditChange(const FProperty* InProperty) const
{
	FName PropertyName = InProperty->GetFName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UGPULightmassSettings, bShowProgressBars))
	{
		return true;
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UGPULightmassSettings, TilePassesInSlowMode))
	{
		return true;
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UGPULightmassSettings, TilePassesInFullSpeedMode))
	{
		return true;
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UGPULightmassSettings, bVisualizeIrradianceCache))
	{
		return true;
	}

	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (World)
	{
		if (World->GetSubsystem<UGPULightmassSubsystem>()->IsRunning())
		{
			return false;
		}
	}

	return true;
}
#endif

void UGPULightmassSubsystem::PostInitialize()
{
	Super::PostInitialize();

	UWorld* World = Cast<UWorld>(GetOuter());
	
	if (!World) return;

	AGPULightmassSettingsActor* SettingsActor = GetSettingsActor();

	if (SettingsActor == nullptr)
	{
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		SpawnInfo.Name = AGPULightmassSettingsActor::StaticClass()->GetFName();
		SpawnInfo.bHideFromSceneOutliner = true;
		SettingsActor = World->SpawnActor<AGPULightmassSettingsActor>(AGPULightmassSettingsActor::StaticClass(), SpawnInfo);
	}

	if (SettingsActor == nullptr)
	{
		UE_LOGF(LogGPULightmass, Warning, "Failed to spawn settings actor in World: %ls", *World->GetName());
	}
}

bool CheckStaticMeshesForLightmapUVVersionUpgrade(const UWorld* World)
{
	TSet<UStaticMesh*> StaticMeshesToUpgradeLightmapUVVersion;
			
	for (const UStaticMeshComponent* Component : TObjectRange<UStaticMeshComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
	{
		if (Component->GetWorld() == World &&
			Component->HasValidSettingsForStaticLighting(false) &&
			Component->GetStaticMesh() &&
			Component->GetStaticMesh()->GetLightmapUVVersion() != (int32)ELightmapUVVersion::Latest)
		{
			StaticMeshesToUpgradeLightmapUVVersion.Add(Component->GetStaticMesh().Get());
		}
	}

	if (!StaticMeshesToUpgradeLightmapUVVersion.IsEmpty())
	{
		const EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo,
			LOCTEXT("MeshLMUVNotLatestMessage", "Some meshes in your scene are not using the latest lightmap UV generation algorithm. Would you like to upgrade them? This can break existing lightmaps."));

		if (Response == EAppReturnType::Yes)
		{
			if (GCurrentLevelEditingViewportClient)
			{
				GCurrentLevelEditingViewportClient->SetShowStats(true);
			}
					
			UStaticMesh::BatchBuild(StaticMeshesToUpgradeLightmapUVVersion.Array());
					
			for (UStaticMesh* Mesh : StaticMeshesToUpgradeLightmapUVVersion)
			{
				FMessageLog("LightingResults").Info()
					->AddToken(FUObjectToken::Create(Mesh))
					->AddToken(FTextToken::Create(LOCTEXT("LightingResults_LMUVVersionUpdated", "lightmap UV version has been upgraded and need to be resaved.")));
						
				Mesh->MarkPackageDirty();
				Mesh->OnMeshChanged.Broadcast();
			}

			FMessageLog("LightingResults").Open();
					
			return true;
		}
	}

	return false;
}

void UGPULightmassSubsystem::Launch()
{
	UWorld* World = Cast<UWorld>(GetOuter());

	if (!World) return;

	FGPULightmassModule& GPULightmassModule = FModuleManager::LoadModuleChecked<FGPULightmassModule>(TEXT("GPULightmass"));

	if (!GPULightmassModule.GetStaticLightingSystemForWorld(World))
	{
		UGPULightmassSettings* SettingsCopy = DuplicateObject(GetSettings(), GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UGPULightmassSettings::StaticClass()));

		if (CheckStaticMeshesForLightmapUVVersionUpgrade(World))
		{
			return;
		}
		
		FScopedSlowTask SlowTask(1);
		SlowTask.MakeDialog();
		SlowTask.EnterProgressFrame(1, LOCTEXT("StartingStaticLightingSystem", "Starting static lighting system"));

		World->UpdateAllSkyCaptures();
		
		{
			FGlobalComponentRecreateRenderStateContext RecreateRenderStateContext; // Implicit FlushRenderingCommands();

			FlushRenderingCommands(); // Flush again to execute commands generated by DestroyRenderState_Concurrent()

			FGPULightmass* StaticLightingSystem = GPULightmassModule.CreateGPULightmassForWorld(World, SettingsCopy);

			if (StaticLightingSystem)
			{
				UE_LOGF(LogTemp, Log, "Static lighting system is created for world %ls.", *World->GetPathName(World->GetOuter()));

				ULightComponent::ReassignStationaryLightChannels(World, false, NULL);
#if WITH_EDITOR
				if (!GIsEditor)
				{
					if (GEngine)
					{
						GEngine->OnPostEditorTick().AddStatic(&FStaticLightingSystemInterface::GameTick);
					}
				}
#endif // WITH_EDITOR

				int32 NumPrimitiveComponents = 0;
				int32 NumLightComponents = 0;

				for (UPrimitiveComponent* Component : TObjectRange<UPrimitiveComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
				{
					if (Component->HasValidSettingsForStaticLighting(false))
					{
						NumPrimitiveComponents++;
					}
				}

				for (ULightComponentBase* Component : TObjectRange<ULightComponentBase>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
				{
					if (Component->bAffectsWorld && Component->HasStaticShadowing())
					{
						NumLightComponents++;
					}
				}

				FScopedSlowTask SubSlowTask(NumPrimitiveComponents + NumLightComponents, LOCTEXT("RegisteringComponentsWithStaticLightingSystem", "Registering components with static lighting system"));
				SubSlowTask.MakeDialog();

				for (UPrimitiveComponent* Component : TObjectRange<UPrimitiveComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
				{
					if (Component->HasValidSettingsForStaticLighting(false))
					{
						FStaticLightingSystemInterface::OnPrimitiveComponentRegistered.Broadcast(Component);

						SubSlowTask.EnterProgressFrame(1, LOCTEXT("RegisteringComponentsWithStaticLightingSystem", "Registering components with static lighting system"));
					}
				}

				for (ULightComponentBase* Component : TObjectRange<ULightComponentBase>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
				{
					if (Component->bAffectsWorld && Component->HasStaticShadowing())
					{
						FStaticLightingSystemInterface::OnLightComponentRegistered.Broadcast(Component);

						SubSlowTask.EnterProgressFrame(1, LOCTEXT("RegisteringComponentsWithStaticLightingSystem", "Registering components with static lighting system"));
					}
				}

				if (FEngineAnalytics::IsAvailable())
				{
					FEngineAnalytics::GetProvider().RecordEvent(TEXT("GPULightmass.Capture"));
				}
			}
			else
			{
				UE_LOGF(LogTemp, Log, "Tried to create static lighting system for world %ls, but failed", *World->GetPathName(World->GetOuter()));
			}
		}

		FlushRenderingCommands(); // Flush commands generated by ~FGlobalComponentRecreateRenderStateContext();
	}
}

void UGPULightmassSubsystem::Stop()
{
	UWorld* World = Cast<UWorld>(GetOuter());

	if (!World) return;

	FGPULightmassModule& GPULightmassModule = FModuleManager::LoadModuleChecked<FGPULightmassModule>(TEXT("GPULightmass"));

	if (GPULightmassModule.GetStaticLightingSystemForWorld(World))
	{
		FScopedSlowTask SlowTask(1);
		SlowTask.MakeDialog();
		SlowTask.EnterProgressFrame(1, LOCTEXT("RemovingStaticLightingSystem", "Removing static lighting system"));

		{
			FGlobalComponentRecreateRenderStateContext RecreateRenderStateContext; // Implicit FlushRenderingCommands();

			FlushRenderingCommands(); // Flush again to execute commands generated by DestroyRenderState_Concurrent()

			int32 NumPrimitiveComponents = 0;
			int32 NumLightComponents = 0;

			for (UPrimitiveComponent* Component : TObjectRange<UPrimitiveComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
			{
				NumPrimitiveComponents++;
			}

			for (ULightComponentBase* Component : TObjectRange<ULightComponentBase>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
			{
				NumLightComponents++;
			}

			FScopedSlowTask SubSlowTask(NumPrimitiveComponents + NumLightComponents, LOCTEXT("UnregisteringComponentsWithStaticLightingSystem", "Unregistering components with static lighting system"));

			// Unregister all landscapes first to prevent grass picking up landscape lightmaps
			for (ULandscapeComponent* Component : TObjectRange<ULandscapeComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
			{
				FStaticLightingSystemInterface::OnPrimitiveComponentUnregistered.Broadcast(Component);
			}

			for (UPrimitiveComponent* Component : TObjectRange<UPrimitiveComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
			{
				FStaticLightingSystemInterface::OnPrimitiveComponentUnregistered.Broadcast(Component);

				SubSlowTask.EnterProgressFrame(1, LOCTEXT("UnregisteringComponentsWithStaticLightingSystem", "Unregistering components with static lighting system"));
			}

			for (ULightComponentBase* Component : TObjectRange<ULightComponentBase>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
			{
				FStaticLightingSystemInterface::OnLightComponentUnregistered.Broadcast(Component);

				SubSlowTask.EnterProgressFrame(1, LOCTEXT("UnregisteringComponentsWithStaticLightingSystem", "Unregistering components with static lighting system"));
			}

			GPULightmassModule.RemoveGPULightmassFromWorld(World);

			UE_LOGF(LogTemp, Log, "Static lighting system is removed for world %ls.", *World->GetPathName(World->GetOuter()));
		}

		FlushRenderingCommands(); // Flush commands generated by ~FGlobalComponentRecreateRenderStateContext();
	}

	// Always turn Realtime on after building lighting
	SetRealtime(true);
}

bool UGPULightmassSubsystem::IsRunning()
{
	UWorld* World = Cast<UWorld>(GetOuter());

	if (!World) return false;

	FGPULightmassModule& GPULightmassModule = FModuleManager::LoadModuleChecked<FGPULightmassModule>(TEXT("GPULightmass"));

	return GPULightmassModule.GetStaticLightingSystemForWorld(World) != nullptr;
}

AGPULightmassSettingsActor* UGPULightmassSubsystem::GetSettingsActor()
{
	UWorld* World = Cast<UWorld>(GetOuter());

	if (!World) return nullptr;

	AGPULightmassSettingsActor* SettingsActor = nullptr;

	if (TActorIterator<AGPULightmassSettingsActor> It(World, AGPULightmassSettingsActor::StaticClass(), EActorIteratorFlags::SkipPendingKill); It)
	{
		SettingsActor = *It;
	}

	return SettingsActor;
}

UGPULightmassSettings* UGPULightmassSubsystem::GetSettings()
{
	return GetSettingsActor() ? GetSettingsActor()->Settings : nullptr;
}

void UGPULightmassSubsystem::StartRecordingVisibleTiles()
{
	// Replicate value to any running instances
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (World)
	{
		FGPULightmassModule& GPULightmassModule = FModuleManager::LoadModuleChecked<FGPULightmassModule>(TEXT("GPULightmass"));
		FGPULightmass* GPULightmass = (FGPULightmass*)GPULightmassModule.GetStaticLightingSystemForWorld(World);
		if (GPULightmass)
		{
			GPULightmass->StartRecordingVisibleTiles();
		}
	}
}

void UGPULightmassSubsystem::EndRecordingVisibleTiles()
{
	// Replicate value to any running instances
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (World)
	{
		FGPULightmassModule& GPULightmassModule = FModuleManager::LoadModuleChecked<FGPULightmassModule>(TEXT("GPULightmass"));
		FGPULightmass* GPULightmass = (FGPULightmass*)GPULightmassModule.GetStaticLightingSystemForWorld(World);
		if (GPULightmass)
		{
			GPULightmass->EndRecordingVisibleTiles();
		}
	}
}

int32 UGPULightmassSubsystem::GetPercentage()
{
	UWorld* World = Cast<UWorld>(GetOuter());

	if (!World) return 0;

	FGPULightmassModule& GPULightmassModule = FModuleManager::LoadModuleChecked<FGPULightmassModule>(TEXT("GPULightmass"));

	if (GPULightmassModule.StaticLightingSystems.Find(World) != nullptr)
	{
		FGPULightmass* GPULightmass = GPULightmassModule.StaticLightingSystems[World];
		return GPULightmass->LightBuildPercentage;
	}

	return 0;
}

void UGPULightmassSubsystem::SetRealtime(bool bInRealtime)
{
	if (GCurrentLevelEditingViewportClient != nullptr)
	{
		GCurrentLevelEditingViewportClient->SetRealtime(bInRealtime);
	}
	else
	{
		UE_LOGF(LogGPULightmass, Warning, "CurrentLevelEditingViewportClient is NULL!");
	}
}

void UGPULightmassSubsystem::Save()
{
	UWorld* World = Cast<UWorld>(GetOuter());

	if (!World) return;

	FGPULightmassModule& GPULightmassModule = FModuleManager::LoadModuleChecked<FGPULightmassModule>(TEXT("GPULightmass"));

	if (GPULightmassModule.StaticLightingSystems.Find(World) != nullptr)
	{
		FGPULightmass* GPULightmass = GPULightmassModule.StaticLightingSystems[World];
		GPULightmass->Scene.ApplyFinishedLightmapsToWorld();;
	}
}

#undef LOCTEXT_NAMESPACE
