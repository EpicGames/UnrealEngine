// Copyright Epic Games, Inc. All Rights Reserved.

#include "IrradianceCaching.h"
#include "GPULightmassModule.h"
#include "RenderGraphUtils.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "RayTracingDefinitions.h"
#include "MeshPassProcessor.h"
#include "GPUSort.h"

IMPLEMENT_UNIFORM_BUFFER_STRUCT(FIrradianceCachingParameters, "IrradianceCachingParameters");
IMPLEMENT_GLOBAL_SHADER(FVisualizeIrradianceCachePS, "/Plugin/GPULightmass/Private/IrradianceCacheVisualization.usf", "VisualizeIrradianceCachePS", SF_Pixel);

FIrradianceCache::FIrradianceCache(FRHICommandList& RHICmdList, int32 Quality, float Spacing, float CornerRejection)
{
	uint32 IrradianceCacheTotalBytes = 0;

	{
		const FRHIBufferCreateDesc RecordsDesc =
			FRHIBufferCreateDesc::CreateStructured<FVector4>(TEXT("FIrradianceCache"), IrradianceCacheMaxSize)
			.AddUsage(BUF_UnorderedAccess | BUF_ShaderResource)
			.DetermineInitialState();
		IrradianceCacheRecords = RHICmdList.CreateBuffer(RecordsDesc);
		IrradianceCacheRecordsUAV = RHICmdList.CreateUnorderedAccessView(IrradianceCacheRecords, FRHIViewDesc::CreateBufferUAV().SetTypeFromBuffer(IrradianceCacheRecords));

		const FRHIBufferCreateDesc RecordBackfaceHitsDesc =
			FRHIBufferCreateDesc::CreateStructured<FVector4>(TEXT("FIrradianceCache"), IrradianceCacheMaxSize)
			.AddUsage(BUF_UnorderedAccess | BUF_ShaderResource)
			.DetermineInitialState();
		IrradianceCacheRecordBackfaceHits = RHICmdList.CreateBuffer(RecordBackfaceHitsDesc);
		IrradianceCacheRecordBackfaceHitsUAV = RHICmdList.CreateUnorderedAccessView(IrradianceCacheRecordBackfaceHits, FRHIViewDesc::CreateBufferUAV().SetTypeFromBuffer(IrradianceCacheRecordBackfaceHits));
		
		IrradianceCacheTotalBytes += RecordsDesc.Size;
		IrradianceCacheTotalBytes += RecordBackfaceHitsDesc.Size;

		RHICmdList.ClearUAVUint(IrradianceCacheRecordsUAV, FUintVector4(0, 0, 0, 0));
		RHICmdList.ClearUAVUint(IrradianceCacheRecordBackfaceHitsUAV, FUintVector4(0, 0, 0, 0));
	}

	int32 HashTableSize = IrradianceCacheMaxSize * 4;

	{
		{
			HashTable.Initialize(RHICmdList, TEXT("ICHashTable"), sizeof(uint32) * 2, HashTableSize, PF_R32G32_UINT, BUF_UnorderedAccess | BUF_ShaderResource);
			HashToIndex.Initialize(RHICmdList, TEXT("ICHashToIndex"), sizeof(uint32), HashTableSize, PF_R32_UINT, BUF_UnorderedAccess | BUF_ShaderResource);
			IndexToHash.Initialize(RHICmdList, TEXT("ICIndexToHash"), sizeof(uint32), HashTableSize, PF_R32_UINT, BUF_UnorderedAccess | BUF_ShaderResource);
			HashTableSemaphore.Initialize(RHICmdList, TEXT("ICHashTableSemaphore"), sizeof(uint32), HashTableSize, PF_R32_UINT, BUF_UnorderedAccess | BUF_ShaderResource);

			IrradianceCacheTotalBytes += HashTable.NumBytes;
			IrradianceCacheTotalBytes += HashToIndex.NumBytes;
			IrradianceCacheTotalBytes += IndexToHash.NumBytes;
			IrradianceCacheTotalBytes += HashTableSemaphore.NumBytes;

			RHICmdList.ClearUAVUint(HashTable.UAV, FUintVector4(0, 0, 0, 0));
			RHICmdList.ClearUAVUint(HashToIndex.UAV, FUintVector4(0, 0, 0, 0));
			RHICmdList.ClearUAVUint(IndexToHash.UAV, FUintVector4(0, 0, 0, 0));
			RHICmdList.ClearUAVUint(HashTableSemaphore.UAV, FUintVector4(0, 0, 0, 0));
		}
		{
			RecordAllocator.Initialize(RHICmdList, TEXT("ICAllocator"), sizeof(uint32), 1, PF_R32_UINT, BUF_UnorderedAccess | BUF_ShaderResource);
			RHICmdList.ClearUAVUint(RecordAllocator.UAV, FUintVector4(0, 0, 0, 0));

			IrradianceCacheTotalBytes += 4;
		}
	}

	UE_LOGF(LogGPULightmass, Log, "Irradiance cache initialized with %.2fMB", IrradianceCacheTotalBytes / 1024.0f / 1024.0f);

	FIrradianceCachingParameters IrradianceCachingParameters;
	IrradianceCachingParameters.IrradianceCacheRecords = IrradianceCacheRecordsUAV;
	IrradianceCachingParameters.IrradianceCacheRecordBackfaceHits = IrradianceCacheRecordBackfaceHitsUAV;
	IrradianceCachingParameters.Quality = Quality;
	IrradianceCachingParameters.Spacing = Spacing;
	IrradianceCachingParameters.CornerRejection = CornerRejection;
	IrradianceCachingParameters.HashTableSize = HashTableSize;
	IrradianceCachingParameters.CacheSize = IrradianceCacheMaxSize;
	IrradianceCachingParameters.RWHashTable = HashTable.UAV;
	IrradianceCachingParameters.RWHashToIndex = HashToIndex.UAV;
	IrradianceCachingParameters.RWIndexToHash = IndexToHash.UAV;
	IrradianceCachingParameters.RecordAllocator = RecordAllocator.UAV;
	IrradianceCachingParameters.HashTableSemaphore = HashTableSemaphore.UAV;
	IrradianceCachingParametersUniformBuffer = TUniformBufferRef<FIrradianceCachingParameters>::CreateUniformBufferImmediate(IrradianceCachingParameters, EUniformBufferUsage::UniformBuffer_MultiFrame);
}
