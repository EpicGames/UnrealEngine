// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "ShaderParameterStruct.h"
#include "SceneView.h"
#include "SceneRenderTargetParameters.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIUtilities.h"

BEGIN_UNIFORM_BUFFER_STRUCT(FIrradianceCachingParameters, )
	SHADER_PARAMETER(uint32, HashTableSize)
	SHADER_PARAMETER(uint32, CacheSize)
	SHADER_PARAMETER(int32, Quality)
	SHADER_PARAMETER(float, Spacing)
	SHADER_PARAMETER(float, CornerRejection)
	SHADER_PARAMETER_UAV(RWStructuredBuffer<uint4>, IrradianceCacheRecords)
	SHADER_PARAMETER_UAV(RWStructuredBuffer<uint4>, IrradianceCacheRecordBackfaceHits)
	SHADER_PARAMETER_UAV(RWStructuredBuffer<uint2>, RWHashTable)
	SHADER_PARAMETER_UAV(RWStructuredBuffer<uint>, RWHashToIndex)
	SHADER_PARAMETER_UAV(RWStructuredBuffer<uint>, RWIndexToHash)
	SHADER_PARAMETER_UAV(RWStructuredBuffer<uint>, RecordAllocator)
	SHADER_PARAMETER_UAV(RWStructuredBuffer<uint>, HashTableSemaphore)
END_UNIFORM_BUFFER_STRUCT()

struct FIrradianceCache
{
	const int32 IrradianceCacheMaxSize = 4194304;

	FBufferRHIRef IrradianceCacheRecords;
	FUnorderedAccessViewRHIRef IrradianceCacheRecordsUAV;
	FBufferRHIRef IrradianceCacheRecordBackfaceHits;
	FUnorderedAccessViewRHIRef IrradianceCacheRecordBackfaceHitsUAV;

	TUniformBufferRef<FIrradianceCachingParameters> IrradianceCachingParametersUniformBuffer;

	FIrradianceCache(FRHICommandList& RHICmdList, int32 Quality, float Spacing, float CornerRejection);

	FRWBuffer HashTable;
	FRWBuffer HashToIndex;
	FRWBuffer IndexToHash;
	FRWBuffer RecordAllocator;
	FRWBuffer HashTableSemaphore;

	int32 CurrentRevision = 0;
};

class FVisualizeIrradianceCachePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeIrradianceCachePS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeIrradianceCachePS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_REF(FIrradianceCachingParameters, IrradianceCachingParameters)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};
