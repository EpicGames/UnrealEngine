// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightmapDenoising.h"
#include "GPULightmassCommon.h"
#include "Async/ParallelFor.h"

FDenoiserFilterSet::FDenoiserFilterSet(FDenoiserContext& Context, FIntPoint NewSize, bool bSHDenoiser)
	: Context(Context)
{
	double Start = FPlatformTime::Seconds();

	Size = NewSize;
	InputBuffer.Empty(Size.X * Size.Y);
	OutputBuffer.Empty(Size.X * Size.Y);

	InputBuffer.AddZeroed(Size.X * Size.Y);
	OutputBuffer.AddZeroed(Size.X * Size.Y);

#if WITH_INTELOIDN
	filter = Context.OIDNDevice.newFilter("RTLightmap");
	filter.setImage("color", InputBuffer.GetData(), oidn::Format::Float3, Size.X, Size.Y);
	filter.setImage("output", OutputBuffer.GetData(), oidn::Format::Float3, Size.X, Size.Y);
	filter.set("directional", bSHDenoiser);
	filter.commit();
#endif

	Context.FilterInitTime += FPlatformTime::Seconds() - Start;
	Context.NumFilterInit++;
}

void FDenoiserFilterSet::Execute()
{
	double Start = FPlatformTime::Seconds();

#if WITH_INTELOIDN
	filter.execute();
#endif

	Context.FilterExecutionTime += FPlatformTime::Seconds() - Start;
	Context.NumFilterExecution++;
}

void FDenoiserFilterSet::Clear()
{
	FMemory::Memset(InputBuffer.GetData(), 0, InputBuffer.GetTypeSize() * InputBuffer.Num());
	FMemory::Memset(OutputBuffer.GetData(), 0, OutputBuffer.GetTypeSize() * OutputBuffer.Num());
}

FLinearColor TonemapLightingForDenoising(FLinearColor Lighting)
{
	Lighting.R = FMath::Sqrt(Lighting.R);
	Lighting.G = FMath::Sqrt(Lighting.G);
	Lighting.B = FMath::Sqrt(Lighting.B);
	return Lighting;
}

FLinearColor InverseTonemapLightingForDenoising(FLinearColor Lighting)
{
	Lighting.R = FMath::Square(Lighting.R);
	Lighting.G = FMath::Square(Lighting.G);
	Lighting.B = FMath::Square(Lighting.B);
	return Lighting;
}

const int32 WindowSize = 2;

int32 EncodeOffset(FIntPoint D)
{
	return (D.Y + WindowSize) * (WindowSize * 2 + 1) + D.X + WindowSize;
}

FIntPoint DecodeOffset(int32 EncodedIndex)
{
	FIntPoint D;
	D.Y = EncodedIndex / (WindowSize * 2 + 1) - WindowSize;
	D.X = EncodedIndex % (WindowSize * 2 + 1) - WindowSize;
	return D;
}

bool IsColorValid(FLinearColor Color)
{
	return FMath::IsFinite(Color.R) && FMath::IsFinite(Color.G) && FMath::IsFinite(Color.B) && FMath::IsFinite(Color.A)
	&& Color.R >= 0 && Color.G >= 0 && Color.B >= 0 && Color.A >= 0;
}

void SimpleFireflyFilter(FLightSampleDataProvider& SampleData)
{
	const FIntPoint Size = SampleData.GetSize();
	
	TArray<int32> Buffer;
	Buffer.AddDefaulted(Size.X * Size.Y);
	
	ParallelFor(Size.Y, [&](int32 Y)
	{
		for (int32 X = 0; X < Size.X; X++)
		{
			if (SampleData.IsMapped({X, Y}))
			{
				int32 ValidCount = 0;
				int32 BiggerCount = 0;
				TArray<int32, TFixedAllocator<(WindowSize * 2 + 1) * (WindowSize * 2 + 1)>> ValidTexelIndices;
				
				for (int32 dx = -WindowSize; dx <= WindowSize; dx++)
				{
					for (int32 dy = -WindowSize; dy <= WindowSize; dy++)
					{
						if (Y + dy >= 0 && Y + dy < Size.Y && X + dx >= 0 && X + dx < Size.X)
						{
							if (SampleData.IsMapped(FIntPoint {X, Y} + FIntPoint {dx, dy}))
							{
								ValidCount++;
								ValidTexelIndices.Add(EncodeOffset({dx, dy}));

								if (SampleData.GetL({X, Y}) >= 1.01 * SampleData.GetL(FIntPoint {X, Y} + FIntPoint {dx, dy}))
								{
									BiggerCount++;
								}
							}
						}
					}
				}

				if (BiggerCount >= 1 && BiggerCount > ValidCount * 0.75)
				{
					ValidTexelIndices.Sort([&SampleData, X, Y](const int32& A, const int32& B)
					{
						return SampleData.GetL(FIntPoint {X, Y} + DecodeOffset(A)) > SampleData.GetL(FIntPoint {X, Y} + DecodeOffset(B));
					});

					Buffer[Y * Size.X + X] = ValidTexelIndices[ValidCount / 2];
				}
				else
				{
					Buffer[Y * Size.X + X] = EncodeOffset({0, 0});
				}
			}
		}
	});
		
	ParallelFor(Size.Y, [&](int32 Y)
	{
		for (int32 X = 0; X < Size.X; X++)
		{
			if (SampleData.IsMapped({X, Y}))
			{
				SampleData.OverwriteTexel({X, Y}, FIntPoint {X, Y} + DecodeOffset(Buffer[Y * Size.X + X]));
			}
		}
	});
}

void NormalizeDirectionalLuminancePair(
	FIntPoint Size,
	TArray<FLinearColor>& IncidentLighting,
	TArray<FLinearColor>& LuminanceSH)
{
	ParallelFor(Size.Y, [&](int32 Y)
	{
		for (int32 X = 0; X < Size.X; X++)
		{
			if (IncidentLighting[Y * Size.X + X].A >= 0.0f)
			{
				float DirLuma[4];
				DirLuma[0] = LuminanceSH[Y * Size.X + X].A / 0.282095f; // Revert diffuse conv done in LightmapEncoding.ush for preview to get actual luma
				DirLuma[1] = LuminanceSH[Y * Size.X + X].R;	// Keep the rest as we need to diffuse conv them anyway
				DirLuma[2] = LuminanceSH[Y * Size.X + X].G;	// Keep the rest as we need to diffuse conv them anyway
				DirLuma[3] = LuminanceSH[Y * Size.X + X].B;	// Keep the rest as we need to diffuse conv them anyway

				float DirScale = 1.0f / DirLuma[0];
				float ColorScale = DirLuma[0];

				if (DirLuma[0] < 0.00001f)
				{
					DirScale = 0.0f;
					ColorScale = 0.0f;
				}

				{
					IncidentLighting[Y * Size.X + X].R = ColorScale * IncidentLighting[Y * Size.X + X].R * IncidentLighting[Y * Size.X + X].R;
					IncidentLighting[Y * Size.X + X].G = ColorScale * IncidentLighting[Y * Size.X + X].G * IncidentLighting[Y * Size.X + X].G;
					IncidentLighting[Y * Size.X + X].B = ColorScale * IncidentLighting[Y * Size.X + X].B * IncidentLighting[Y * Size.X + X].B;

					LuminanceSH[Y * Size.X + X].A = 1.0f;
					LuminanceSH[Y * Size.X + X].R = DirLuma[1] * DirScale;
					LuminanceSH[Y * Size.X + X].G = DirLuma[2] * DirScale;
					LuminanceSH[Y * Size.X + X].B = DirLuma[3] * DirScale;
				}
			}
			else
			{
				IncidentLighting[Y * Size.X + X].R = 0;
				IncidentLighting[Y * Size.X + X].G = 0;
				IncidentLighting[Y * Size.X + X].B = 0;

				LuminanceSH[Y * Size.X + X].A = 0;
				LuminanceSH[Y * Size.X + X].R = 0;
				LuminanceSH[Y * Size.X + X].G = 0;
				LuminanceSH[Y * Size.X + X].B = 0;
			}
		}
	});
}

void DilateDirectionalLuminancePairForDenoising(
	FIntPoint Size,
	TArray<FLinearColor>& IncidentLighting,
	TArray<FLinearColor>& LuminanceSH)
{
	TArray<float> DilationMask;
	DilationMask.AddZeroed(Size.X * Size.Y);

	const int32 TotalIteration = 2;

	for (int32 IterationIndex = 0; IterationIndex < TotalIteration; IterationIndex++)
	{
		for (int32 Y = 0; Y < Size.Y; Y++)
		{
			for (int32 X = 0; X < Size.X; X++)
			{
				if (!(IncidentLighting[Y * Size.X + X].A >= 0.0f))
				{
					for (int32 dx = -1; dx <= 1; dx++)
					{
						for (int32 dy = -1; dy <= 1; dy++)
						{
							if (Y + dy >= 0 && Y + dy < Size.Y && X + dx >= 0 && X + dx < Size.X)
							{
								if (IncidentLighting[(Y + dy) * Size.X + (X + dx)].A >= 0.0f)
								{
									IncidentLighting[Y * Size.X + X] = IncidentLighting[(Y + dy) * Size.X + (X + dx)];
									LuminanceSH[Y * Size.X + X] = LuminanceSH[(Y + dy) * Size.X + (X + dx)];
									DilationMask[Y * Size.X + X] = IncidentLighting[Y * Size.X + X].A;
									IncidentLighting[Y * Size.X + X].A = -1.0f;
								}
							}
						}
					}
				}
			}
		}

		for (int32 Y = 0; Y < Size.Y; Y++)
		{
			for (int32 X = 0; X < Size.X; X++)
			{
				if (DilationMask[Y * Size.X + X] > 0.0f)
				{
					IncidentLighting[Y * Size.X + X].A = DilationMask[Y * Size.X + X];
					DilationMask[Y * Size.X + X] = 0;
				}
			}
		}
	}
}

void ReencodeDirectionalLuminancePair(
	FIntPoint Size,
	TArray<FLinearColor>& IncidentLighting,
	TArray<FLinearColor>& LuminanceSH)
{
	ParallelFor(Size.Y, [&](int32 Y)
	{
		for (int32 X = 0; X < Size.X; X++)
		{
			IncidentLighting[Y * Size.X + X].R = FMath::Sqrt(IncidentLighting[Y * Size.X + X].R);
			IncidentLighting[Y * Size.X + X].G = FMath::Sqrt(IncidentLighting[Y * Size.X + X].G);
			IncidentLighting[Y * Size.X + X].B = FMath::Sqrt(IncidentLighting[Y * Size.X + X].B);

			LuminanceSH[Y * Size.X + X].A *= 0.282095f;
		}
	});
}

void DenoiseDirectionalLuminancePair(
	FIntPoint Size,
	TArray<FLinearColor>& IncidentLighting,
	TArray<FLinearColor>& LuminanceSH,
	FDenoiserContext& DenoiserContext)
{
	// Denoise IncidentLighting
	{
		// Resizing the filter is a super expensive operation
		// Round things into size bins to reduce number of resizes
		FDenoiserFilterSet& FilterSet = DenoiserContext.GetFilterForSize(FIntPoint(FMath::DivideAndRoundUp(Size.X, 64) * 64, FMath::DivideAndRoundUp(Size.Y, 64) * 64));
		
		for (int32 Y = 0; Y < Size.Y; Y++)
		{
			for (int32 X = 0; X < Size.X; X++)
			{
				if (IncidentLighting[Y * Size.X + X].A >= 0)
				{
					FilterSet.InputBuffer[Y * FilterSet.Size.X + X][0] = TonemapLightingForDenoising(IncidentLighting[Y * Size.X + X]).R;
					FilterSet.InputBuffer[Y * FilterSet.Size.X + X][1] = TonemapLightingForDenoising(IncidentLighting[Y * Size.X + X]).G;
					FilterSet.InputBuffer[Y * FilterSet.Size.X + X][2] = TonemapLightingForDenoising(IncidentLighting[Y * Size.X + X]).B;
				}
			}
		}

		FilterSet.Execute();

		for (int32 Y = 0; Y < Size.Y; Y++)
		{
			for (int32 X = 0; X < Size.X; X++)
			{
				IncidentLighting[Y * Size.X + X].R = InverseTonemapLightingForDenoising(FLinearColor(FilterSet.OutputBuffer[Y * FilterSet.Size.X + X])).R;
				IncidentLighting[Y * Size.X + X].G = InverseTonemapLightingForDenoising(FLinearColor(FilterSet.OutputBuffer[Y * FilterSet.Size.X + X])).G;
				IncidentLighting[Y * Size.X + X].B = InverseTonemapLightingForDenoising(FLinearColor(FilterSet.OutputBuffer[Y * FilterSet.Size.X + X])).B;
			}
		}
	}

	// Denoise LuminanceSH
	{
		// Resizing the filter is a super expensive operation
		// Round things into size bins to reduce number of resizes
		FDenoiserFilterSet& FilterSet = DenoiserContext.GetFilterForSize(FIntPoint(FMath::DivideAndRoundUp(Size.X, 64) * 64, FMath::DivideAndRoundUp(Size.Y, 64) * 64), true);

		for (int32 Y = 0; Y < Size.Y; Y++)
		{
			for (int32 X = 0; X < Size.X; X++)
			{
				if (IncidentLighting[Y * Size.X + X].A >= 0)
				{
					FilterSet.InputBuffer[Y * FilterSet.Size.X + X][0] = LuminanceSH[Y * Size.X + X].R;
					FilterSet.InputBuffer[Y * FilterSet.Size.X + X][1] = LuminanceSH[Y * Size.X + X].G;
					FilterSet.InputBuffer[Y * FilterSet.Size.X + X][2] = LuminanceSH[Y * Size.X + X].B;
				}
			}
		}

		FilterSet.Execute();

		for (int32 Y = 0; Y < Size.Y; Y++)
		{
			for (int32 X = 0; X < Size.X; X++)
			{
				LuminanceSH[Y * Size.X + X].R = FilterSet.OutputBuffer[Y * FilterSet.Size.X + X][0];
				LuminanceSH[Y * Size.X + X].G = FilterSet.OutputBuffer[Y * FilterSet.Size.X + X][1];
				LuminanceSH[Y * Size.X + X].B = FilterSet.OutputBuffer[Y * FilterSet.Size.X + X][2];
			}
		}
	}
}

void DenoiseSkyBentNormal(
	FIntPoint Size,
	TArray<FLinearColor>& ValidityMask,
	TArray<FVector3f>& BentNormal,
	FDenoiserContext& DenoiserContext)
{
	TArray<int32> DilationMask;
	DilationMask.AddZeroed(Size.X * Size.Y);

	for (int32 Y = 0; Y < Size.Y; Y++)
	{
		for (int32 X = 0; X < Size.X; X++)
		{
			DilationMask[Y * Size.X + X] = ValidityMask[Y * Size.X + X].A >= 0.0f ? 1 : 0;
		}
	}
	
	const int32 TotalIteration = 2;

	for (int32 IterationIndex = 0; IterationIndex < TotalIteration; IterationIndex++)
	{
		for (int32 Y = 0; Y < Size.Y; Y++)
		{
			for (int32 X = 0; X < Size.X; X++)
			{
				if (DilationMask[Y * Size.X + X] == 0)
				{
					for (int32 dx = -1; dx <= 1; dx++)
					{
						for (int32 dy = -1; dy <= 1; dy++)
						{
							if (Y + dy >= 0 && Y + dy < Size.Y && X + dx >= 0 && X + dx < Size.X)
							{
								if (DilationMask[(Y + dy) * Size.X + (X + dx)] == 1)
								{
									BentNormal[Y * Size.X + X] = BentNormal[(Y + dy) * Size.X + (X + dx)];
									DilationMask[Y * Size.X + X] = 2;
								}
							}
						}
					}
				}
			}
		}

		for (int32 Y = 0; Y < Size.Y; Y++)
		{
			for (int32 X = 0; X < Size.X; X++)
			{
				if (DilationMask[Y * Size.X + X] == 2)
				{
					DilationMask[Y * Size.X + X] = 1;
				}
			}
		}
	}
	// Resizing the filter is a super expensive operation
	// Round things into size bins to reduce number of resizes
	FDenoiserFilterSet& FilterSet = DenoiserContext.GetFilterForSize(FIntPoint(FMath::DivideAndRoundUp(Size.X, 64) * 64, FMath::DivideAndRoundUp(Size.Y, 64) * 64), true);

	for (int32 Y = 0; Y < Size.Y; Y++)
	{
		for (int32 X = 0; X < Size.X; X++)
		{
			const int32 LinearIndex = Y * Size.X + X;
			const FVector3f N = BentNormal[LinearIndex];
			FilterSet.InputBuffer[LinearIndex].X = FMath::Sign(N.X) * FMath::Sqrt(FMath::Abs(N.X));
			FilterSet.InputBuffer[LinearIndex].Y = FMath::Sign(N.Y) * FMath::Sqrt(FMath::Abs(N.Y));
			FilterSet.InputBuffer[LinearIndex].Z = FMath::Sign(N.Z) * FMath::Sqrt(FMath::Abs(N.Z));			
		}
	}
	
	FilterSet.Execute();
	
	for (int32 Y = 0; Y < Size.Y; Y++)
	{
		for (int32 X = 0; X < Size.X; X++)
		{
			const int32 LinearIndex = Y * Size.X + X;
			const FVector3f N = FilterSet.OutputBuffer[LinearIndex];
			BentNormal[LinearIndex].X = FMath::Sign(N.X) * FMath::Square(N.X);
			BentNormal[LinearIndex].Y = FMath::Sign(N.Y) * FMath::Square(N.Y);
			BentNormal[LinearIndex].Z = FMath::Sign(N.Z) * FMath::Square(N.Z);
		}
	}
}

void DenoiseRawData(
	FIntPoint Size,
	TArray<FLinearColor>& IncidentLighting,
	TArray<FLinearColor>& LuminanceSH,
	FDenoiserContext& DenoiserContext, 
	bool bPrepadTexels)
{
	// Required for Intel OIDN as it assumes LuminanceSH is normalized into [-1, 1]
	NormalizeDirectionalLuminancePair(Size, IncidentLighting, LuminanceSH);
	if (bPrepadTexels)
	{
		DilateDirectionalLuminancePairForDenoising(Size, IncidentLighting, LuminanceSH);
	}
	DenoiseDirectionalLuminancePair(Size, IncidentLighting, LuminanceSH, DenoiserContext);
	ReencodeDirectionalLuminancePair(Size, IncidentLighting, LuminanceSH);
}
