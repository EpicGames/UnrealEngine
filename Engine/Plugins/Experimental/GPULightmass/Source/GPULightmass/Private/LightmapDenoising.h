// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightmapEncoding.h"

#ifndef WITH_INTELOIDN
	#define WITH_INTELOIDN 0
#endif

#if WITH_INTELOIDN
#include "OpenImageDenoise/oidn.hpp"
#endif

// TBB suffers from extreme fragmentation problem in editor
#include "Runtime/Core/Private/HAL/Allocators/AnsiAllocator.h"

struct FDenoiserFilterSet
{
	struct FDenoiserContext& Context;

#if WITH_INTELOIDN
	oidn::FilterRef filter;
#endif

	FIntPoint Size;
	TArray<FVector3f> InputBuffer;
	TArray<FVector3f> OutputBuffer;

	FDenoiserFilterSet(FDenoiserContext& Context, FIntPoint NewSize, bool bSHDenoiser = false);

	void Execute();

	void Clear();
};

struct FDenoiserContext
{
	int32 NumFilterInit = 0;
	int32 NumFilterExecution = 0;
	double FilterInitTime = 0.0;
	double FilterExecutionTime = 0.0;

#if WITH_INTELOIDN
	oidn::DeviceRef OIDNDevice;
#endif

	TMap<FIntPoint, FDenoiserFilterSet> Filters;
	TMap<FIntPoint, FDenoiserFilterSet> SHFilters;

	FDenoiserContext()
	{
#if WITH_INTELOIDN
		OIDNDevice = oidn::newDevice();
		OIDNDevice.commit();
#endif
	}

	~FDenoiserContext()
	{
		UE_LOGF(LogTemp, Log, "Denoising: %.2lfs initializing filters (%d), %.2lfs executing filters (%d)", FilterInitTime, NumFilterInit, FilterExecutionTime, NumFilterExecution);
	}

	FDenoiserFilterSet& GetFilterForSize(FIntPoint Size, bool bSHDenoiser = false)
	{
		if (!bSHDenoiser)
		{
			if (!Filters.Contains(Size))
			{
				Filters.Add(Size, FDenoiserFilterSet(*this, Size));
			}

			Filters[Size].Clear();

			return Filters[Size];
		}
		else
		{
			if (!SHFilters.Contains(Size))
			{
				SHFilters.Add(Size, FDenoiserFilterSet(*this, Size, true));
			}

			SHFilters[Size].Clear();

			return SHFilters[Size];
		}
	}
};

void DenoiseSkyBentNormal(
	FIntPoint Size,
	TArray<FLinearColor>& ValidityMask,
	TArray<FVector3f>& BentNormal,
	FDenoiserContext& DenoiserContext);

void DenoiseRawData(
	FIntPoint Size,
	TArray<FLinearColor>& IncidentLighting,
	TArray<FLinearColor>& LuminanceSH,
	FDenoiserContext& DenoiserContext,
	bool bPrepadTexels = true);

struct FLightSampleDataProvider
{
	virtual ~FLightSampleDataProvider() {}
	virtual FIntPoint GetSize() const = 0; 
	virtual bool IsMapped(FIntPoint Location) const = 0;	
	virtual float GetL(FIntPoint Location) const = 0;
	virtual void OverwriteTexel(FIntPoint Dst, FIntPoint Src) = 0;
};

template<typename LightSampleType>
struct TLightSampleDataProvider {};

template<>
struct TLightSampleDataProvider<FLinearColor> : public FLightSampleDataProvider
{
	FIntPoint Size;
	TArray<FLinearColor>& IncidentLighting;
	TArray<FLinearColor>& LuminanceSH;

	TLightSampleDataProvider<FLinearColor>(
		FIntPoint Size,
		TArray<FLinearColor>& IncidentLighting,
		TArray<FLinearColor>& LuminanceSH)
		: Size(Size)
		, IncidentLighting(IncidentLighting)
		, LuminanceSH(LuminanceSH)
	{
	}
	
	virtual FIntPoint GetSize() const override
	{
		return Size;
	}

	virtual bool IsMapped(FIntPoint Location) const override
	{
		return IncidentLighting[Location.Y * Size.X + Location.X].A >= 0;
	}
	
	virtual float GetL(FIntPoint Location) const override
	{
		return IncidentLighting[Location.Y * Size.X + Location.X].GetLuminance();
	}

	virtual void OverwriteTexel(FIntPoint Dst, FIntPoint Src) override
	{
		IncidentLighting[Dst.Y * Size.X + Dst.X] = IncidentLighting[Src.Y * Size.X + Src.X];
		LuminanceSH[Dst.Y * Size.X + Dst.X] = LuminanceSH[Src.Y * Size.X + Src.X];
	}
};

template<>
struct TLightSampleDataProvider<FVector3f> : public FLightSampleDataProvider
{
	FIntPoint Size;
	TArray<FLinearColor>& IncidentLightingAsValidityMask;
	TArray<FVector3f>& SkyBentNormal;

	TLightSampleDataProvider<FVector3f>(
		FIntPoint Size,
		TArray<FLinearColor>& IncidentLightingAsValidityMask,
		TArray<FVector3f>& SkyBentNormal)
		: Size(Size)
		, IncidentLightingAsValidityMask(IncidentLightingAsValidityMask)
		, SkyBentNormal(SkyBentNormal)
	{
	}
	
	virtual FIntPoint GetSize() const override
	{
		return Size;
	}

	virtual bool IsMapped(FIntPoint Location) const override
	{
		return IncidentLightingAsValidityMask[Location.Y * Size.X + Location.X].A >= 0;
	}
	
	virtual float GetL(FIntPoint Location) const override
	{
		return SkyBentNormal[Location.Y * Size.X + Location.X].Length();
	}

	virtual void OverwriteTexel(FIntPoint Dst, FIntPoint Src) override
	{
		SkyBentNormal[Dst.Y * Size.X + Dst.X] = SkyBentNormal[Src.Y * Size.X + Src.X];
	}
};

void SimpleFireflyFilter(FLightSampleDataProvider& SampleData);
