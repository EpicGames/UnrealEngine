// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightmapEncoding.h"
#include "Async/ParallelFor.h"
#include "GPULightmassCommon.h"
#include "GPULightmassModule.h"

static void GetLUVW(const float RGB[3], float& L, float& U, float& V, float& W)
{
	float R = FMath::Max(0.0f, RGB[0]);
	float G = FMath::Max(0.0f, RGB[1]);
	float B = FMath::Max(0.0f, RGB[2]);

	L = 0.3f * R + 0.59f * G + 0.11f * B;
	if (L < 1e-4f)
	{
		U = 1.0f;
		V = 1.0f;
		W = 1.0f;
	}
	else
	{
		U = R / L;
		V = G / L;
		W = B / L;
	}
}

struct MinMaxCoefficients
{
	float MinCoefficient[NUM_STORED_LIGHTMAP_COEF][4];
	float MaxCoefficient[NUM_STORED_LIGHTMAP_COEF][4];
};

void QuantizeLightSamples(
	TArray<FLightSampleData> InLightSamples,
	TArray<FLightMapCoefficients>& OutLightSamples,
	float OutMultiply[NUM_STORED_LIGHTMAP_COEF][4],
	float OutAdd[NUM_STORED_LIGHTMAP_COEF][4])
{
	FMemMark MemMark(FMemStack::Get());
	
	//const float EncodeExponent = .5f;
	const float LogScale = 11.5f;
	const float LogBlackPoint = FMath::Pow(2.0f, -0.5f * LogScale);
	const float SimpleLogScale = 16.0f;
	const float SimpleLogBlackPoint = FMath::Pow(2.0f, -0.5f * SimpleLogScale);

	const int32 NumSamplesPerTask = 4096;
	const int32 NumTasks = FMath::DivideAndRoundUp(InLightSamples.Num(), NumSamplesPerTask);
	TArray<MinMaxCoefficients, TMemStackAllocator<>> TaskStorage;
	TaskStorage.AddUninitialized(NumTasks);
	ParallelFor(NumTasks,[&](int32 TaskIndex)
	{
		float (&MinCoefficient)[NUM_STORED_LIGHTMAP_COEF][4] = TaskStorage[TaskIndex].MinCoefficient;
		float (&MaxCoefficient)[NUM_STORED_LIGHTMAP_COEF][4] = TaskStorage[TaskIndex].MaxCoefficient;

		for (int32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
		{
			for (int32 ColorIndex = 0; ColorIndex < 4; ColorIndex++)
			{
				// Color
				MinCoefficient[CoefficientIndex][ColorIndex] = 10000.0f;
				MaxCoefficient[CoefficientIndex][ColorIndex] = -10000.0f;

				// Direction
				MinCoefficient[CoefficientIndex + 1][ColorIndex] = 10000.0f;
				MaxCoefficient[CoefficientIndex + 1][ColorIndex] = -10000.0f;
			}
		}

		// go over all samples looking for min and max values
		for (int32 SampleIndex = TaskIndex * NumSamplesPerTask; SampleIndex < (TaskIndex + 1) * NumSamplesPerTask && SampleIndex < InLightSamples.Num(); SampleIndex++)
		{
			const FLightSampleData& SourceSample = InLightSamples[SampleIndex];

			if (SourceSample.bIsMapped)
			{
				{
					// Complex
					float L, U, V, W;
					GetLUVW(SourceSample.Coefficients[0], L, U, V, W);

					float LogL = FMath::Log2(L + LogBlackPoint);

					MinCoefficient[0][0] = FMath::Min(MinCoefficient[0][0], U);
					MaxCoefficient[0][0] = FMath::Max(MaxCoefficient[0][0], U);

					MinCoefficient[0][1] = FMath::Min(MinCoefficient[0][1], V);
					MaxCoefficient[0][1] = FMath::Max(MaxCoefficient[0][1], V);

					MinCoefficient[0][2] = FMath::Min(MinCoefficient[0][2], W);
					MaxCoefficient[0][2] = FMath::Max(MaxCoefficient[0][2], W);

					MinCoefficient[0][3] = FMath::Min(MinCoefficient[0][3], LogL);
					MaxCoefficient[0][3] = FMath::Max(MaxCoefficient[0][3], LogL);

					// Dampen dark texel's contribution on the directionality min and max
					float DampenDirectionality = FMath::Clamp(L * 100, 0.0f, 1.0f);

					for (int32 ColorIndex = 0; ColorIndex < 3; ColorIndex++)
					{
						MinCoefficient[1][ColorIndex] = FMath::Min(MinCoefficient[1][ColorIndex], DampenDirectionality * SourceSample.Coefficients[1][ColorIndex]);
						MaxCoefficient[1][ColorIndex] = FMath::Max(MaxCoefficient[1][ColorIndex], DampenDirectionality * SourceSample.Coefficients[1][ColorIndex]);
					}
				}

				{
					// Simple
					float L, U, V, W;
					GetLUVW(SourceSample.Coefficients[2], L, U, V, W);

					float LogL = FMath::Log2(L + SimpleLogBlackPoint) / SimpleLogScale + 0.5f;

					float LogR = LogL * U;
					float LogG = LogL * V;
					float LogB = LogL * W;

					MinCoefficient[2][0] = FMath::Min(MinCoefficient[2][0], LogR);
					MaxCoefficient[2][0] = FMath::Max(MaxCoefficient[2][0], LogR);

					MinCoefficient[2][1] = FMath::Min(MinCoefficient[2][1], LogG);
					MaxCoefficient[2][1] = FMath::Max(MaxCoefficient[2][1], LogG);

					MinCoefficient[2][2] = FMath::Min(MinCoefficient[2][2], LogB);
					MaxCoefficient[2][2] = FMath::Max(MaxCoefficient[2][2], LogB);

					// Dampen dark texel's contribution on the directionality min and max
					float DampenDirectionality = FMath::Clamp(L * 100, 0.0f, 1.0f);

					for (int32 ColorIndex = 0; ColorIndex < 3; ColorIndex++)
					{
						MinCoefficient[3][ColorIndex] = FMath::Min(MinCoefficient[3][ColorIndex], DampenDirectionality * SourceSample.Coefficients[3][ColorIndex]);
						MaxCoefficient[3][ColorIndex] = FMath::Max(MaxCoefficient[3][ColorIndex], DampenDirectionality * SourceSample.Coefficients[3][ColorIndex]);
					}
				}
			}
		}
	});
	
	float MinCoefficient[NUM_STORED_LIGHTMAP_COEF][4];
	float MaxCoefficient[NUM_STORED_LIGHTMAP_COEF][4];
	
	for (int32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
	{
		for (int32 ColorIndex = 0; ColorIndex < 4; ColorIndex++)
		{
			// Color
			MinCoefficient[CoefficientIndex][ColorIndex] = 10000.0f;
			MaxCoefficient[CoefficientIndex][ColorIndex] = -10000.0f;

			// Direction
			MinCoefficient[CoefficientIndex + 1][ColorIndex] = 10000.0f;
			MaxCoefficient[CoefficientIndex + 1][ColorIndex] = -10000.0f;
		}
	}

	for(int32 TaskIndex = 0; TaskIndex < NumTasks; TaskIndex++)
	{
		for (int32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
		{
			for (int32 ColorIndex = 0; ColorIndex < 4; ColorIndex++)
			{
				// Color
				MinCoefficient[CoefficientIndex][ColorIndex] = FMath::Min(MinCoefficient[CoefficientIndex][ColorIndex], TaskStorage[TaskIndex].MinCoefficient[CoefficientIndex][ColorIndex]);
				MaxCoefficient[CoefficientIndex][ColorIndex] = FMath::Max(MaxCoefficient[CoefficientIndex][ColorIndex], TaskStorage[TaskIndex].MaxCoefficient[CoefficientIndex][ColorIndex]);
	
				// Direction
				MinCoefficient[CoefficientIndex + 1][ColorIndex] = FMath::Min(MinCoefficient[CoefficientIndex + 1][ColorIndex],TaskStorage[TaskIndex].MinCoefficient[CoefficientIndex + 1][ColorIndex]);
				MaxCoefficient[CoefficientIndex + 1][ColorIndex] = FMath::Max(MaxCoefficient[CoefficientIndex + 1][ColorIndex],TaskStorage[TaskIndex].MaxCoefficient[CoefficientIndex + 1][ColorIndex]);
			}
		}
	}
	
	// If no sample mapped make range sane.
	// Or if very dark no directionality is added. Make range sane.
	for (int32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex++)
	{
		for (int32 ColorIndex = 0; ColorIndex < 4; ColorIndex++)
		{
			if (MinCoefficient[CoefficientIndex][ColorIndex] > MaxCoefficient[CoefficientIndex][ColorIndex])
			{
				MinCoefficient[CoefficientIndex][ColorIndex] = 0.0f;
				MaxCoefficient[CoefficientIndex][ColorIndex] = 0.0f;
			}
		}
	}

	// Calculate the scale/bias for the light-map coefficients.
	float CoefficientMultiply[NUM_STORED_LIGHTMAP_COEF][4];
	float CoefficientAdd[NUM_STORED_LIGHTMAP_COEF][4];

	for (int32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex++)
	{
		for (int32 ColorIndex = 0; ColorIndex < 4; ColorIndex++)
		{
			// Calculate scale and bias factors to pack into the desired range
			// y = (x - Min) / (Max - Min)
			// Mul = 1 / (Max - Min)
			// Add = -Min / (Max - Min)
			CoefficientMultiply[CoefficientIndex][ColorIndex] = 1.0f / FMath::Max<float>(MaxCoefficient[CoefficientIndex][ColorIndex] - MinCoefficient[CoefficientIndex][ColorIndex], DELTA);
			CoefficientAdd[CoefficientIndex][ColorIndex] = -MinCoefficient[CoefficientIndex][ColorIndex] / FMath::Max<float>(MaxCoefficient[CoefficientIndex][ColorIndex] - MinCoefficient[CoefficientIndex][ColorIndex], DELTA);

			// Output the values used to undo this packing
			OutMultiply[CoefficientIndex][ColorIndex] = 1.0f / CoefficientMultiply[CoefficientIndex][ColorIndex];
			OutAdd[CoefficientIndex][ColorIndex] = -CoefficientAdd[CoefficientIndex][ColorIndex] / CoefficientMultiply[CoefficientIndex][ColorIndex];
		}
	}

	// Bias to avoid divide by zero in shader
	for (int32 ColorIndex = 0; ColorIndex < 3; ColorIndex++)
	{
		OutAdd[2][ColorIndex] = FMath::Max(OutAdd[2][ColorIndex], 1e-2f);
	}

	// Force SH constant term to 0.282095f. Avoids add in shader.
	OutMultiply[1][3] = 0.0f;
	OutAdd[1][3] = 0.282095f;
	OutMultiply[3][3] = 0.0f;
	OutAdd[3][3] = 0.282095f;

	// allocate space in the output
	OutLightSamples.Empty(InLightSamples.Num());
	OutLightSamples.AddUninitialized(InLightSamples.Num());
	
	ParallelFor(NumTasks,[&](int32 TaskIndex)
	{
		// quantize each sample using the above scaling
		for (int32 SampleIndex = TaskIndex * NumSamplesPerTask; SampleIndex < (TaskIndex + 1) * NumSamplesPerTask && SampleIndex < InLightSamples.Num(); SampleIndex++)
		{
			const FLightSampleData& SourceSample = InLightSamples[SampleIndex];
			FLightMapCoefficients& DestCoefficients = OutLightSamples[SampleIndex];

			DestCoefficients.Coverage = SourceSample.bIsMapped ? 255 : 0;

			const FVector BentNormal(SourceSample.SkyOcclusion[0], SourceSample.SkyOcclusion[1], SourceSample.SkyOcclusion[2]);
			const float BentNormalLength = BentNormal.Size();
			const FVector NormalizedBentNormal = BentNormal.GetSafeNormal() * FVector(.5f) + FVector(.5f);

			DestCoefficients.SkyOcclusion[0] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(NormalizedBentNormal[0] * 255.0f), 0, 255);
			DestCoefficients.SkyOcclusion[1] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(NormalizedBentNormal[1] * 255.0f), 0, 255);
			DestCoefficients.SkyOcclusion[2] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(NormalizedBentNormal[2] * 255.0f), 0, 255);
			// Sqrt on length to allocate more precision near 0
			DestCoefficients.SkyOcclusion[3] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(FMath::Sqrt(BentNormalLength) * 255.0f), 0, 255);

			// Sqrt to allocate more precision near 0
			DestCoefficients.AOMaterialMask = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(FMath::Sqrt(SourceSample.AOMaterialMask) * 255.0f), 0, 255);

			{
				float L, U, V, W;
				GetLUVW(SourceSample.Coefficients[0], L, U, V, W);

				// LogLUVW encode color
				float LogL = FMath::Log2(L + LogBlackPoint);

				U = U * CoefficientMultiply[0][0] + CoefficientAdd[0][0];
				V = V * CoefficientMultiply[0][1] + CoefficientAdd[0][1];
				W = W * CoefficientMultiply[0][2] + CoefficientAdd[0][2];
				LogL = LogL * CoefficientMultiply[0][3] + CoefficientAdd[0][3];

				float Residual = LogL * 255.0f - FMath::RoundToFloat(LogL * 255.0f) + 0.5f;

				// U, V, W, LogL
				// UVW stored in gamma space
				DestCoefficients.Coefficients[0][0] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(FMath::Sqrt(U) * 255.0f), 0, 255);
				DestCoefficients.Coefficients[0][1] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(FMath::Sqrt(V) * 255.0f), 0, 255);
				DestCoefficients.Coefficients[0][2] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(FMath::Sqrt(W) * 255.0f), 0, 255);
				DestCoefficients.Coefficients[0][3] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(LogL * 255.0f), 0, 255);

				float Dx = SourceSample.Coefficients[1][0] * CoefficientMultiply[1][0] + CoefficientAdd[1][0];
				float Dy = SourceSample.Coefficients[1][1] * CoefficientMultiply[1][1] + CoefficientAdd[1][1];
				float Dz = SourceSample.Coefficients[1][2] * CoefficientMultiply[1][2] + CoefficientAdd[1][2];

				// Dx, Dy, Dz, Residual
				DestCoefficients.Coefficients[1][0] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(Dx * 255.0f), 0, 255);
				DestCoefficients.Coefficients[1][1] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(Dy * 255.0f), 0, 255);
				DestCoefficients.Coefficients[1][2] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(Dz * 255.0f), 0, 255);
				DestCoefficients.Coefficients[1][3] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(Residual * 255.0f), 0, 255);
			}

			{
				float L, U, V, W;
				GetLUVW(SourceSample.Coefficients[2], L, U, V, W);

				// LogRGB encode color
				float LogL = FMath::Log2(L + SimpleLogBlackPoint) / SimpleLogScale + 0.5f;

				float LogR = LogL * U * CoefficientMultiply[2][0] + CoefficientAdd[2][0];
				float LogG = LogL * V * CoefficientMultiply[2][1] + CoefficientAdd[2][1];
				float LogB = LogL * W * CoefficientMultiply[2][2] + CoefficientAdd[2][2];

				// LogR, LogG, LogB
				DestCoefficients.Coefficients[2][0] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(LogR * 255.0f), 0, 255);
				DestCoefficients.Coefficients[2][1] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(LogG * 255.0f), 0, 255);
				DestCoefficients.Coefficients[2][2] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(LogB * 255.0f), 0, 255);
				DestCoefficients.Coefficients[2][3] = 255;

				float Dx = SourceSample.Coefficients[3][0] * CoefficientMultiply[3][0] + CoefficientAdd[3][0];
				float Dy = SourceSample.Coefficients[3][1] * CoefficientMultiply[3][1] + CoefficientAdd[3][1];
				float Dz = SourceSample.Coefficients[3][2] * CoefficientMultiply[3][2] + CoefficientAdd[3][2];

				// Dx, Dy, Dz
				DestCoefficients.Coefficients[3][0] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(Dx * 255.0f), 0, 255);
				DestCoefficients.Coefficients[3][1] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(Dy * 255.0f), 0, 255);
				DestCoefficients.Coefficients[3][2] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(Dz * 255.0f), 0, 255);
				DestCoefficients.Coefficients[3][3] = 255;
			}
		}
	});
}

FLightSampleData ConvertToLightSample(FLinearColor IncidentLighting, FLinearColor LuminanceSH)
{
	FLightSampleData Sample;

	Sample.bIsMapped = IncidentLighting.A >= 0.0f;

	float DirLuma[4];
	DirLuma[0] = LuminanceSH.A / 0.282095f; // Revert diffuse conv done in LightmapEncoding.ush for preview to get actual luma
	DirLuma[1] = LuminanceSH.R;	// Keep the rest as we need to diffuse conv them anyway
	DirLuma[2] = LuminanceSH.G;	// Keep the rest as we need to diffuse conv them anyway
	DirLuma[3] = LuminanceSH.B;	// Keep the rest as we need to diffuse conv them anyway

	float DirScale = 1.0f / FMath::Max(0.0001f, DirLuma[0]);
	float ColorScale = DirLuma[0];

	{
		Sample.Coefficients[0][0] = ColorScale * IncidentLighting.R * IncidentLighting.R;
		Sample.Coefficients[0][1] = ColorScale * IncidentLighting.G * IncidentLighting.G;
		Sample.Coefficients[0][2] = ColorScale * IncidentLighting.B * IncidentLighting.B;

		Sample.Coefficients[1][0] = DirLuma[1] * DirScale;
		Sample.Coefficients[1][1] = DirLuma[2] * DirScale;
		Sample.Coefficients[1][2] = DirLuma[3] * DirScale;
	}

	{
		// Do the same for LQ coefficients
		Sample.Coefficients[2][0] = ColorScale * IncidentLighting.R * IncidentLighting.R;
		Sample.Coefficients[2][1] = ColorScale * IncidentLighting.G * IncidentLighting.G;
		Sample.Coefficients[2][2] = ColorScale * IncidentLighting.B * IncidentLighting.B;

		Sample.Coefficients[3][0] = DirLuma[1] * DirScale;
		Sample.Coefficients[3][1] = DirLuma[2] * DirScale;
		Sample.Coefficients[3][2] = DirLuma[3] * DirScale;
	}

	return Sample;
}

FQuantizedSignedDistanceFieldShadowSample ConvertToShadowSample(FLinearColor ShadowMask, int32 ChannelIndex)
{
	FQuantizedSignedDistanceFieldShadowSample Sample;
	// Sqrt is already done on GPU
	Sample.Distance = FVector4f(ShadowMask)[ChannelIndex] * 255.0f;
	Sample.Coverage = FVector4f(ShadowMask)[ChannelIndex] >= 0.0f ? 255 : 0;
	Sample.PenumbraSize = 0;
	return Sample;
}
