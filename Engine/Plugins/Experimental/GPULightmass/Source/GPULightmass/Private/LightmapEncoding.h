// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightMap.h"
#include "ShadowMap.h"

/**
 * Incident lighting for a single sample, as produced by a lighting build.
 * FGatheredLightSample is used for gathering lighting instead of this format as FLightSampleData is not additive.
 */
struct FLightSampleData
{
	FLightSampleData()
	{
		bIsMapped = false;
		FMemory::Memzero(Coefficients, sizeof(Coefficients));
		SkyOcclusion[0] = 0;
		SkyOcclusion[1] = 0;
		SkyOcclusion[2] = 0;
		AOMaterialMask = 0;
	}

	/**
	 * Coefficients[0] stores the normalized average color,
	 * Coefficients[1] stores the maximum color component in each lightmap basis direction,
	 * and Coefficients[2] stores the simple lightmap which is colored incident lighting along the vertex normal.
	 */
	float Coefficients[NUM_STORED_LIGHTMAP_COEF][3];

	float SkyOcclusion[3];

	float AOMaterialMask;

	/** True if this sample maps to a valid point on a triangle.  This is only meaningful for texture lightmaps. */
	bool bIsMapped;

	/**
	 * Export helper
	 * @param Component Which directional lightmap component to retrieve
	 * @return An FColor for this component, clamped to White
	 */
	FColor GetColor(int32 Component) const
	{
		return FColor(
			(uint8)FMath::Clamp<int32>(Coefficients[Component][0] * 255, 0, 255),
			(uint8)FMath::Clamp<int32>(Coefficients[Component][1] * 255, 0, 255),
			(uint8)FMath::Clamp<int32>(Coefficients[Component][2] * 255, 0, 255),
			0);
	}
};

FLightSampleData ConvertToLightSample(FLinearColor IncidentLighting, FLinearColor LuminanceSH);

void QuantizeLightSamples(
	TArray<FLightSampleData> InLightSamples,
	TArray<FLightMapCoefficients>& OutLightSamples,
	float OutMultiply[NUM_STORED_LIGHTMAP_COEF][4],
	float OutAdd[NUM_STORED_LIGHTMAP_COEF][4]);

FQuantizedSignedDistanceFieldShadowSample ConvertToShadowSample(FLinearColor ShadowMask, int32 ChannelIndex);
