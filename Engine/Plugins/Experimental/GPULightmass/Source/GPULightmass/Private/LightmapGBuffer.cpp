// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightmapGBuffer.h"

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FLightmapGBufferParams, "LightmapGBufferParams", SceneTextures);

IMPLEMENT_MATERIAL_SHADER_TYPE(, FLightmapGBufferVS, TEXT("/Plugin/GPULightmass/Private/LightmapGBuffer.usf"), TEXT("LightmapGBufferVS"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(, FLightmapGBufferPS, TEXT("/Plugin/GPULightmass/Private/LightmapGBuffer.usf"), TEXT("LightmapGBufferPS"), SF_Pixel);
