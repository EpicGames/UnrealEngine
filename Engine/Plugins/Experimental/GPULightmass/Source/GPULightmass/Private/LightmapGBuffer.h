// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataDrivenShaderPlatformInfo.h"
#include "GPULightmassCommon.h"
#include "PrimitiveSceneProxy.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "MeshMaterialShader.h"
#include "LightMapRendering.h"
#include "Materials/Material.h"
#include "SceneManagement.h"
#include "MaterialDomain.h"

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FLightmapGBufferParams, )
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ScratchTilePoolLayer0)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ScratchTilePoolLayer1)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ScratchTilePoolLayer2)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

struct FLightmapElementData : public FMeshMaterialShaderElementData
{
	const FLightCacheInterface* LCI;

	FVector4f VirtualTexturePhysicalTileCoordinateScaleAndBias;
	int32 RenderPassIndex;
	FIntPoint ScratchTilePoolOffset;
	int32 bMaterialTwoSided = 0;

	FLightmapElementData(const FLightCacheInterface* LCI) : LCI(LCI) {}
};

class FLightmapGBufferVS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FLightmapGBufferVS, MeshMaterial);

	LAYOUT_FIELD(FShaderParameter, VirtualTexturePhysicalTileCoordinateScaleAndBias)
	LAYOUT_FIELD(FShaderParameter, RenderPassIndex)
	LAYOUT_FIELD(FShaderUniformBufferParameter, PrecomputedLightingBufferParameter);

protected:

	FLightmapGBufferVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		PrecomputedLightingBufferParameter.Bind(Initializer.ParameterMap, TEXT("PrecomputedLightingBuffer"));
		VirtualTexturePhysicalTileCoordinateScaleAndBias.Bind(Initializer.ParameterMap, TEXT("VirtualTexturePhysicalTileCoordinateScaleAndBias"));
		RenderPassIndex.Bind(Initializer.ParameterMap, TEXT("RenderPassIndex"));
	}

	FLightmapGBufferVS() = default;

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("RANDOM_SAMPLER"), (int)2);
		OutEnvironment.SetDefine(TEXT("NEEDS_LIGHTMAP_COORDINATE"), 1);
		OutEnvironment.SetDefine(TEXT("SCENE_TEXTURES_DISABLED"), 1);
		OutEnvironment.SetDefine(TEXT("GPreviewLightmapPhysicalTileSize"), GPreviewLightmapPhysicalTileSize);
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		const bool bAllowStaticLighting = IsStaticLightingAllowed();

		if (EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData)
			&& IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5)
			&& bAllowStaticLighting
			&& Parameters.VertexFactoryType->SupportsStaticLighting()
			&& Parameters.MaterialParameters.bIsSpecialEngineMaterial)
		{
			return true;
		}
		else
		{
			return false;
		}
	}

public:
	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FLightmapElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		if (PrecomputedLightingBufferParameter.IsBound())
		{
			if (!ShaderElementData.LCI || !ShaderElementData.LCI->GetPrecomputedLightingBuffer())
			{
				ShaderBindings.Add(PrecomputedLightingBufferParameter, GEmptyPrecomputedLightingUniformBuffer.GetUniformBufferRHI());
			}
			else
			{
				ShaderBindings.Add(PrecomputedLightingBufferParameter, ShaderElementData.LCI->GetPrecomputedLightingBuffer());
			}
		}

		ShaderBindings.Add(VirtualTexturePhysicalTileCoordinateScaleAndBias, ShaderElementData.VirtualTexturePhysicalTileCoordinateScaleAndBias);
		ShaderBindings.Add(RenderPassIndex, ShaderElementData.RenderPassIndex);
	}
};

class FLightmapGBufferPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FLightmapGBufferPS, MeshMaterial);

	LAYOUT_FIELD(FShaderParameter, ScratchTilePoolOffset)
	LAYOUT_FIELD(FShaderParameter, bMaterialTwoSided)
public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		const bool bAllowStaticLighting = IsStaticLightingAllowed();

		if (EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData)
			&& IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5)
			&& bAllowStaticLighting
			&& Parameters.VertexFactoryType->SupportsStaticLighting()
			&& Parameters.MaterialParameters.bIsSpecialEngineMaterial)
		{
			return true;
		}
		else
		{
			return false;
		}
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetRenderTargetOutputFormat(0, PF_A32B32G32R32F);
		OutEnvironment.SetDefine(TEXT("NEEDS_LIGHTMAP_COORDINATE"), 1);
		OutEnvironment.SetDefine(TEXT("SCENE_TEXTURES_DISABLED"), 1);
		OutEnvironment.SetDefine(TEXT("GPreviewLightmapPhysicalTileSize"), GPreviewLightmapPhysicalTileSize);
	}

	FLightmapGBufferPS() = default;

	FLightmapGBufferPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		ScratchTilePoolOffset.Bind(Initializer.ParameterMap, TEXT("ScratchTilePoolOffset"));
		bMaterialTwoSided.Bind(Initializer.ParameterMap, TEXT("bMaterialTwoSided"));
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FLightmapElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);
		ShaderBindings.Add(ScratchTilePoolOffset, ShaderElementData.ScratchTilePoolOffset);
		ShaderBindings.Add(bMaterialTwoSided, ShaderElementData.bMaterialTwoSided);
	}
};

class FLightmapGBufferMeshProcessor : public FMeshPassProcessor
{
public:
	FLightmapGBufferMeshProcessor(
		const FScene* InScene, 
		const FSceneView* InView,
		FMeshPassDrawListContext* InDrawListContext,
		FVector4f VirtualTexturePhysicalTileCoordinateScaleAndBias,
		int32 RenderPassIndex,
		FIntPoint ScratchTilePoolOffset
	)
		: FMeshPassProcessor(TEXT("LightmapGBuffer"), InScene, InView->GetFeatureLevel(), InView, InDrawListContext)
		, VirtualTexturePhysicalTileCoordinateScaleAndBias(VirtualTexturePhysicalTileCoordinateScaleAndBias)
		, RenderPassIndex(RenderPassIndex)
		, ScratchTilePoolOffset(ScratchTilePoolOffset)
	{
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
		DrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
	}

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final
	{
		if (MeshBatch.bUseForMaterial)
		{
			bool bMaterialTwoSided = false;
			if (MeshBatch.MaterialRenderProxy && MeshBatch.MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel))
			{
				bMaterialTwoSided = MeshBatch.MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel)->IsTwoSided();
			}
			
			const FMaterialRenderProxy& DefaultProxy = *UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
			const FMaterial& DefaultMaterial = *DefaultProxy.GetMaterialNoFallback(FeatureLevel);

			Process(MeshBatch, BatchElementMask, StaticMeshId, bMaterialTwoSided, PrimitiveSceneProxy, DefaultProxy, DefaultMaterial);
		}
	}

private:
	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		bool bMaterialTwoSided,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource)
	{
		const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<FLightmapGBufferVS>();
		ShaderTypes.AddShaderType<FLightmapGBufferPS>();

		FMaterialShaders MaterialShaders;
		verify(MaterialResource.TryGetShaders(ShaderTypes, MeshBatch.VertexFactory->GetType(), MaterialShaders));
		
		TMeshProcessorShaders<
			FLightmapGBufferVS,
			FLightmapGBufferPS> Shaders;

		MaterialShaders.TryGetVertexShader(Shaders.VertexShader);
		MaterialShaders.TryGetPixelShader(Shaders.PixelShader);

		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(MaterialResource, OverrideSettings);
		ERasterizerCullMode MeshCullMode = CM_None;

		FLightmapElementData ShaderElementData(MeshBatch.LCI);
		ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);
		ShaderElementData.VirtualTexturePhysicalTileCoordinateScaleAndBias = VirtualTexturePhysicalTileCoordinateScaleAndBias;
		ShaderElementData.RenderPassIndex = RenderPassIndex;
		ShaderElementData.ScratchTilePoolOffset = ScratchTilePoolOffset;
		ShaderElementData.bMaterialTwoSided = bMaterialTwoSided;

		FMeshDrawCommandSortKey SortKey {};

		BuildMeshDrawCommands(
			MeshBatch,
			BatchElementMask,
			PrimitiveSceneProxy,
			MaterialRenderProxy,
			MaterialResource,
			DrawRenderState,
			Shaders,
			MeshFillMode,
			MeshCullMode,
			SortKey,
			EMeshPassFeatures::Default,
			ShaderElementData);

		return true;
	}

private:
	FMeshPassProcessorRenderState DrawRenderState;

	FVector4f VirtualTexturePhysicalTileCoordinateScaleAndBias;
	int32 RenderPassIndex;
	FIntPoint ScratchTilePoolOffset;
};
