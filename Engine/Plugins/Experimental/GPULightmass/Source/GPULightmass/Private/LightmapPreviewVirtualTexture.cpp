// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightmapPreviewVirtualTexture.h"
#include "GPULightmassCommon.h"
#include "EngineModule.h"
#include "Scene/Lights.h"
#include "LightmapRenderer.h"
#include "RHI.h"
#include "RHICommandList.h"

namespace GPULightmass
{

FLightmapPreviewVirtualTexture::FLightmapPreviewVirtualTexture(FLightmapRenderStateRef LightmapRenderState, FLightmapRenderer* Renderer)
	: LightmapRenderState(LightmapRenderState)
	, LightmapRenderer(Renderer)
{
	FIntPoint SizeInTiles = LightmapRenderState->GetPaddedSizeInTiles();

	FVTProducerDescription ProducerDesc;

	{
		ProducerDesc.bPersistentHighestMip = false;
		ProducerDesc.bContinuousUpdate = true;
		ProducerDesc.Dimensions = 2;
		ProducerDesc.TileSize = GPreviewLightmapVirtualTileSize;
		ProducerDesc.TileBorderSize = GPreviewLightmapTileBorderSize;
		ProducerDesc.BlockWidthInTiles = SizeInTiles.X;
		ProducerDesc.BlockHeightInTiles = SizeInTiles.Y;
		ProducerDesc.DepthInTiles = 1;
		ProducerDesc.NumTextureLayers = 4; // LightMapVirtualTexture->TypeToLayer.Num();
		ProducerDesc.NumPhysicalGroups = 4;
		ProducerDesc.LayerFormat[0] = EPixelFormat::PF_A32B32G32R32F;
		ProducerDesc.LayerFormat[1] = EPixelFormat::PF_A32B32G32R32F;
		ProducerDesc.LayerFormat[2] = EPixelFormat::PF_A32B32G32R32F;
		ProducerDesc.LayerFormat[3] = EPixelFormat::PF_A32B32G32R32F;
		ProducerDesc.PhysicalGroupIndex[0] = 0;
		ProducerDesc.PhysicalGroupIndex[1] = 1;
		ProducerDesc.PhysicalGroupIndex[2] = 2;
		ProducerDesc.PhysicalGroupIndex[3] = 3;
		ProducerDesc.MaxLevel = FMath::Min((int32)FMath::CeilLogTwo((uint32)FMath::Min(SizeInTiles.X, SizeInTiles.Y)), GPreviewLightmapMipmapMaxLevel);

		ProducerHandle = GetRendererModule().RegisterVirtualTextureProducer(ProducerDesc, this);
	}

	{
		FAllocatedVTDescription Desc;

		Desc.Dimensions = ProducerDesc.Dimensions;
		Desc.TileSize = ProducerDesc.TileSize;
		Desc.TileBorderSize = ProducerDesc.TileBorderSize;
		Desc.NumTextureLayers = ProducerDesc.NumTextureLayers;

		for (uint32 LayerIndex = 0u; LayerIndex < Desc.NumTextureLayers; ++LayerIndex)
		{
			Desc.ProducerHandle[LayerIndex] = ProducerHandle;
			Desc.ProducerLayerIndex[LayerIndex] = LayerIndex;
		}

		AllocatedVT = GetRendererModule().AllocateVirtualTexture(Desc);
		ensure(AllocatedVT->GetVirtualAddress() != ~0u);
	}
}

FVTRequestPageResult FLightmapPreviewVirtualTexture::RequestPageData(
	FRHICommandListBase& RHICmdList,
	const FVirtualTextureProducerHandle& InProducerHandle,
	uint8 LayerMask,
	uint8 vLevel,
	uint64 vAddress,
	EVTRequestPagePriority Priority)
{
	return FVTRequestPageResult(EVTRequestPageStatus::Available, 0);
}

IVirtualTextureFinalizer* FLightmapPreviewVirtualTexture::ProducePageData(
	FRHICommandListBase& RHICmdList,
	ERHIFeatureLevel::Type FeatureLevel,
	EVTProducePageFlags Flags,
	const FVirtualTextureProducerHandle& InProducerHandle,
	uint8 LayerMask,
	uint8 vLevel,
	uint64 vAddress,
	uint64 RequestHandle,
	const FVTProduceTargetLayer* TargetLayers)
{
	check(vAddress <= MAX_uint32); // Not supporting 64 bit vAddress here. Only currrently supported for adaptive runtime virtual texture.
	
	FLightmapTileRequest TileRequest(LightmapRenderState, FTileVirtualCoordinates((uint32)vAddress, vLevel));

	if (!LightmapRenderState->IsTileCoordinatesValid(TileRequest.VirtualCoordinates))
	{
		return nullptr;
	}

	for (int32 LayerIndex = 0; LayerIndex < 8; LayerIndex++)
	{
		if (LayerMask & (1u << LayerIndex))
		{
			TileRequest.OutputPhysicalCoordinates[LayerIndex] = FIntPoint(TargetLayers[LayerIndex].pPageLocation.X, TargetLayers[LayerIndex].pPageLocation.Y);
			TileRequest.OutputRenderTargets[LayerIndex] = TargetLayers[LayerIndex].PooledRenderTarget;
		}
	}

	LightmapRenderer->AddRequest(TileRequest);

	return LightmapRenderer;
}

}
