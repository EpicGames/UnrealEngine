// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightmapStorage.h"
#include "RHIFeatureLevel.h"
#include "VirtualTexturing.h"

class FRHICommandListBase;

namespace GPULightmass
{

class FLightmapRenderer;

class FLightmapPreviewVirtualTexture : public IVirtualTexture
{
public:
	FLightmapPreviewVirtualTexture(FLightmapRenderStateRef LightmapRenderState, FLightmapRenderer* Renderer);

	virtual bool IsPageStreamed(uint8 vLevel, uint32 vAddress) const override
	{
		return false;
	}

	virtual FVTRequestPageResult RequestPageData(
		FRHICommandListBase& RHICmdList,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		EVTRequestPagePriority Priority
	) override;

	virtual IVirtualTextureFinalizer* ProducePageData(
		FRHICommandListBase& RHICmdList,
		ERHIFeatureLevel::Type FeatureLevel,
		EVTProducePageFlags Flags,
		const FVirtualTextureProducerHandle& InProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		uint64 RequestHandle,
		const FVTProduceTargetLayer* TargetLayers
	) override;

	IAllocatedVirtualTexture* AllocatedVT;
	FVirtualTextureProducerHandle ProducerHandle;

private:
	FLightmapRenderStateRef LightmapRenderState;
	FLightmapRenderer* LightmapRenderer;
};

}
