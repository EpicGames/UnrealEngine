// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightmapRayTracing.h"
#include "ScenePrivate.h"

bool IsSupportedVertexFactoryType(const FVertexFactoryType* VertexFactoryType)
{
	return VertexFactoryType->SupportsLightmapBaking();
}

#if RHI_RAYTRACING

FLightmapRayTracingMeshProcessor::FLightmapRayTracingMeshProcessor(FRayTracingMeshCommandContext* InCommandContext)
	: FRayTracingMeshProcessor(InCommandContext, nullptr, nullptr, ERayTracingType::LightMapTracing)
{
}

IMPLEMENT_GLOBAL_SHADER(FLightmapPathTracingRGS, "/Plugin/GPULightmass/Private/LightmapPathTracing.usf", "LightmapPathTracingMainRG", SF_RayGen);
IMPLEMENT_GLOBAL_SHADER(FVolumetricLightmapPathTracingRGS, "/Plugin/GPULightmass/Private/LightmapPathTracing.usf", "VolumetricLightmapPathTracingMainRG", SF_RayGen);
IMPLEMENT_GLOBAL_SHADER(FStationaryLightShadowTracingRGS, "/Plugin/GPULightmass/Private/LightmapPathTracing.usf", "StationaryLightShadowTracingMainRG", SF_RayGen);
IMPLEMENT_GLOBAL_SHADER(FStaticShadowDepthMapTracingRGS, "/Plugin/GPULightmass/Private/StaticShadowDepthMap.usf", "StaticShadowDepthMapTracingRG", SF_RayGen);

IMPLEMENT_GLOBAL_SHADER(FFirstBounceRayGuidingCDFBuildCS, "/Plugin/GPULightmass/Private/FirstBounceRayGuidingCDFBuild.usf", "FirstBounceRayGuidingCDFBuildCS", SF_Compute);

#endif
