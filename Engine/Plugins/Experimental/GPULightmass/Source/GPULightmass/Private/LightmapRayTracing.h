// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"

#if RHI_RAYTRACING

#include "GPULightmassCommon.h"
#include "ShaderParameterStruct.h"
#include "GlobalShader.h"
#include "RayTracing/RayTracingSkyLight.h"
#include "SceneView.h"
#include "RayTracing/RayTracingMaterialHitShaders.h"
#include "IrradianceCaching.h"
#include "RayTracingTypes.h"
#include "RayTracingPayloadType.h"

BEGIN_SHADER_PARAMETER_STRUCT(FPathTracingLightGrid, RENDERER_API)
	SHADER_PARAMETER(uint32, SceneInfiniteLightCount)
	SHADER_PARAMETER(FVector3f, SceneLightsTranslatedBoundMin)
	SHADER_PARAMETER(FVector3f, SceneLightsTranslatedBoundMax)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>, LightGrid)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, LightGridData)
	SHADER_PARAMETER(unsigned, LightGridResolution)
	SHADER_PARAMETER(unsigned, LightGridMaxCount)
	SHADER_PARAMETER(unsigned, LightGridAxis)
END_SHADER_PARAMETER_STRUCT()

class FLightmapRayTracingMeshProcessor : public FRayTracingMeshProcessor
{
public:
	FLightmapRayTracingMeshProcessor(FRayTracingMeshCommandContext* InCommandContext);
};

class FLightmapPathTracingRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLightmapPathTracingRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FLightmapPathTracingRGS, FGlobalShader)

	class FUseIrradianceCaching : SHADER_PERMUTATION_BOOL("USE_IRRADIANCE_CACHING");
	class FUseFirstBounceRayGuiding : SHADER_PERMUTATION_BOOL("USE_FIRST_BOUNCE_RAY_GUIDING");
	class FUseICBackfaceDetection : SHADER_PERMUTATION_BOOL("IC_BACKFACE_DETECTION");

	using FPermutationDomain = TShaderPermutationDomain<FUseIrradianceCaching, FUseFirstBounceRayGuiding, FUseICBackfaceDetection>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		// Disallow invalid permutations
		if (!PermutationVector.Get<FUseIrradianceCaching>())
		{
			if (PermutationVector.Get<FUseICBackfaceDetection>()) return false;
			if (PermutationVector.Get<FUseFirstBounceRayGuiding>()) return false;
		}
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GPreviewLightmapPhysicalTileSize"), GPreviewLightmapPhysicalTileSize);
		OutEnvironment.SetDefine(TEXT("LIGHTMAP_PATH_TRACING_MAIN_RG"), 1);
		OutEnvironment.SetDefine(TEXT("USE_RECT_LIGHT_TEXTURES"), 1);
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::GPULightmass;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int, LastInvalidationFrame)
		SHADER_PARAMETER(int, NumTotalSamples)
		SHADER_PARAMETER(int, NumRayGuidingTrialSamples)
		SHADER_PARAMETER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, GBufferWorldPosition)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, GBufferWorldNormal)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, GBufferShadingNormal)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, IrradianceAndSampleCount)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SHDirectionality)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SHCorrectionAndStationarySkyLightBentNormal)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RayGuidingLuminance)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, RayGuidingCDFX)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, RayGuidingCDFY)
		SHADER_PARAMETER_SRV(StructuredBuffer<FGPUTileDescription>, BatchedTiles)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPathTracingLight>, SceneLights)
		SHADER_PARAMETER(uint32, SceneLightCount)
		SHADER_PARAMETER(uint32, SceneVisibleLightCount)
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingLightGrid, LightGridParameters)
		// Skylight
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SkylightTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SkylightPdf)
		SHADER_PARAMETER_SAMPLER(SamplerState, SkylightTextureSampler)
		SHADER_PARAMETER(float, SkylightInvResolution)
		SHADER_PARAMETER(int32, SkylightMipCount)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_STRUCT_REF(FIrradianceCachingParameters, IrradianceCachingParameters)
	END_SHADER_PARAMETER_STRUCT()
};

class FVolumetricLightmapPathTracingRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVolumetricLightmapPathTracingRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FVolumetricLightmapPathTracingRGS, FGlobalShader)

	class FUseIrradianceCaching : SHADER_PERMUTATION_BOOL("USE_IRRADIANCE_CACHING");

	using FPermutationDomain = TShaderPermutationDomain<FUseIrradianceCaching>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GPreviewLightmapPhysicalTileSize"), GPreviewLightmapPhysicalTileSize);
		OutEnvironment.SetDefine(TEXT("VOLUMETRIC_LIGHTMAP_PATH_TRACING_MAIN_RG"), 1);
		OutEnvironment.SetDefine(TEXT("USE_RECT_LIGHT_TEXTURES"), 1);
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::GPULightmass;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, FrameNumber)
		SHADER_PARAMETER(FVector4f, VolumeMin)
		SHADER_PARAMETER(FVector4f, VolumeSize)
		SHADER_PARAMETER(FIntVector, IndirectionTextureDim)
		SHADER_PARAMETER(int32, NumTotalBricks)
		SHADER_PARAMETER(int32, BrickBatchOffset)
		SHADER_PARAMETER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_SRV(Buffer<uint4>, BrickRequests)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, AmbientVector)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, SHCoefficients0R)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, SHCoefficients1R)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, SHCoefficients0G)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, SHCoefficients1G)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, SHCoefficients0B)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, SHCoefficients1B)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, SkyBentNormal)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>, DirectionalLightShadowing)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPathTracingLight>, SceneLights)
		SHADER_PARAMETER(uint32, SceneLightCount)
		SHADER_PARAMETER(uint32, SceneVisibleLightCount)
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingLightGrid, LightGridParameters)
		// Skylight
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SkylightTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SkylightPdf)
		SHADER_PARAMETER_SAMPLER(SamplerState, SkylightTextureSampler)
		SHADER_PARAMETER(float, SkylightInvResolution)
		SHADER_PARAMETER(int32, SkylightMipCount)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_REF(FIrradianceCachingParameters, IrradianceCachingParameters)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FLightShaderConstants>, LightShaderParametersArray)
	END_SHADER_PARAMETER_STRUCT()
};

class FStationaryLightShadowTracingRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FStationaryLightShadowTracingRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FStationaryLightShadowTracingRGS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GPreviewLightmapPhysicalTileSize"), GPreviewLightmapPhysicalTileSize);
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::GPULightmass;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_SRV(Buffer<int>, LightTypeArray)
		SHADER_PARAMETER_SRV(Buffer<int>, ChannelIndexArray)
		SHADER_PARAMETER_SRV(Buffer<int>, LightSampleIndexArray)
		SHADER_PARAMETER_SRV(StructuredBuffer<FLightShaderConstants>, LightShaderParametersArray)
		SHADER_PARAMETER_SRV(StructuredBuffer<FGPUTileDescription>, BatchedTiles)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, GBufferWorldPosition)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, GBufferWorldNormal)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, GBufferShadingNormal)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ShadowMask)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ShadowMaskSampleCount)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPathTracingLight>, SceneLights)
		SHADER_PARAMETER(uint32, SceneLightCount)
		SHADER_PARAMETER(uint32, SceneVisibleLightCount)
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingLightGrid, LightGridParameters)
	END_SHADER_PARAMETER_STRUCT()
};

class FFirstBounceRayGuidingCDFBuildCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FFirstBounceRayGuidingCDFBuildCS);
	SHADER_USE_PARAMETER_STRUCT(FFirstBounceRayGuidingCDFBuildCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GPreviewLightmapPhysicalTileSize"), GPreviewLightmapPhysicalTileSize);
		OutEnvironment.CompilerFlags.Add(CFLAG_WaveOperations);
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int, RayGuidingEndPassIndex)
		SHADER_PARAMETER_SRV(StructuredBuffer<FGPUTileDescription>, BatchedTiles)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RayGuidingLuminance)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RayGuidingCDFX)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RayGuidingCDFY)
	END_SHADER_PARAMETER_STRUCT()
};

class FStaticShadowDepthMapTracingRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FStaticShadowDepthMapTracingRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FStaticShadowDepthMapTracingRGS, FGlobalShader)

	class FLightType : SHADER_PERMUTATION_INT("LIGHT_TYPE", 3);

	using FPermutationDomain = TShaderPermutationDomain<FLightType>;
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GPreviewLightmapPhysicalTileSize"), GPreviewLightmapPhysicalTileSize);
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::GPULightmass;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(int32, StaticShadowDepthMapSuperSampleFactor)
		SHADER_PARAMETER(FIntPoint, ShadowMapSize)
		SHADER_PARAMETER(FMatrix44f, LightToWorld)
		SHADER_PARAMETER(FMatrix44f, WorldToLight)
		SHADER_PARAMETER(FVector3f, LightSpaceImportanceBoundsMin)
		SHADER_PARAMETER(FVector3f, LightSpaceImportanceBoundsMax)
		SHADER_PARAMETER(float, MaxPossibleDistance)
		SHADER_PARAMETER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<half>, DepthMapTexture)
	END_SHADER_PARAMETER_STRUCT()
};

#endif