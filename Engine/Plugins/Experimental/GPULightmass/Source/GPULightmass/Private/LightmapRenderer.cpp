// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightmapRenderer.h"
#include "GPULightmassModule.h"
#include "GPULightmassCommon.h"
#include "SceneRendering.h"
#include "Scene/Scene.h"
#include "Scene/StaticMesh.h"
#include "LightmapGBuffer.h"
#include "LightmapRayTracing.h"
#include "PathTracingLightParameters.inl"
#include "ClearQuad.h"
#include "RayTracing/RayTracingMaterialHitShaders.h"
#include "RayTracing/RayTracing.h"
#include "Async/ParallelFor.h"
#include "Async/Async.h"
#include "LightmapPreviewVirtualTexture.h"
#include "RHIGPUReadback.h"
#include "LightmapStorage.h"
#include "LevelEditorViewport.h"
#include "Editor.h"
#include "CanvasTypes.h"
#include "LightmapDenoising.h"
#include "EngineModule.h"
#include "PostProcess/PostProcessing.h"
#include "RayTracingGeometryManagerInterface.h"
#include "RayTracingInstanceBufferUtil.h"
#include "ScreenPass.h"
#include "RayTracingDynamicGeometryUpdateManager.h"
#include "ShaderCompiler.h"
#include "RectLightTextureManager.h"
#include "Engine/SubsurfaceProfile.h"
#include "IESTextureManager.h"
#include "PathTracing.h"
#include "PostProcess/DrawRectangle.h"
#include "SceneUniformBuffer.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "PrimitiveSceneShaderData.h"

RENDERER_API uint8 BlendModeToRayTracingInstanceMask(const EBlendMode BlendMode, bool bIsDitherMasked, bool bCastShadow, ERayTracingType RayTracingType);

class FCopyConvergedLightmapTilesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyConvergedLightmapTilesCS)
	SHADER_USE_PARAMETER_STRUCT(FCopyConvergedLightmapTilesCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GPreviewLightmapPhysicalTileSize"), GPreviewLightmapPhysicalTileSize);
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, NumBatchedTiles)
		SHADER_PARAMETER(uint32, StagingPoolSizeX)
		SHADER_PARAMETER_SRV(StructuredBuffer<FGPUTileDescription>, BatchedTiles)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, IrradianceAndSampleCount)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SHDirectionality)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SHCorrectionAndStationarySkyLightBentNormal)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ShadowMask)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ShadowMaskSampleCount)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, StagingHQLayer0)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, StagingHQLayer1)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, StagingShadowMask)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, StagingSkyOcclusion)
	END_SHADER_PARAMETER_STRUCT()
};

class FUploadConvergedLightmapTilesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FUploadConvergedLightmapTilesCS)
	SHADER_USE_PARAMETER_STRUCT(FUploadConvergedLightmapTilesCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GPreviewLightmapPhysicalTileSize"), GPreviewLightmapPhysicalTileSize);
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, NumBatchedTiles)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SrcTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DstTexture)
		SHADER_PARAMETER_SRV(StructuredBuffer<int2>, SrcTilePositions)
		SHADER_PARAMETER_SRV(StructuredBuffer<int2>, DstTilePositions)
	END_SHADER_PARAMETER_STRUCT()
};

class FSelectiveLightmapOutputCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSelectiveLightmapOutputCS)
	SHADER_USE_PARAMETER_STRUCT(FSelectiveLightmapOutputCS, FGlobalShader)

	class FOutputLayerDim : SHADER_PERMUTATION_INT("DIM_OUTPUT_LAYER", 4);
	class FDrawProgressBars : SHADER_PERMUTATION_BOOL("DRAW_PROGRESS_BARS");
	using FPermutationDomain = TShaderPermutationDomain<FOutputLayerDim, FDrawProgressBars>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GPreviewLightmapPhysicalTileSize"), GPreviewLightmapPhysicalTileSize);
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, NumBatchedTiles)
		SHADER_PARAMETER(int32, NumTotalSamples)
		SHADER_PARAMETER(int32, NumIrradianceCachePasses)
		SHADER_PARAMETER(int32, NumRayGuidingTrialSamples)
		SHADER_PARAMETER_SRV(StructuredBuffer<FGPUTileDescription>, BatchedTiles)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutputTileAtlas)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, IrradianceAndSampleCount)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SHDirectionality)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ShadowMask)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ShadowMaskSampleCount)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SHCorrectionAndStationarySkyLightBentNormal)
	END_SHADER_PARAMETER_STRUCT()
};

class FMultiTileClearCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMultiTileClearCS)
	SHADER_USE_PARAMETER_STRUCT(FMultiTileClearCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GPreviewLightmapPhysicalTileSize"), GPreviewLightmapPhysicalTileSize);
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, NumTiles)
		SHADER_PARAMETER(int32, TileSize)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int2>, TilePositions)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, TilePool)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCopyConvergedLightmapTilesCS, "/Plugin/GPULightmass/Private/LightmapBufferClear.usf", "CopyConvergedLightmapTilesCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FUploadConvergedLightmapTilesCS, "/Plugin/GPULightmass/Private/LightmapBufferClear.usf", "UploadConvergedLightmapTilesCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FSelectiveLightmapOutputCS, "/Plugin/GPULightmass/Private/LightmapOutput.usf", "SelectiveLightmapOutputCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FMultiTileClearCS, "/Plugin/GPULightmass/Private/LightmapBufferClear.usf", "MultiTileClearCS", SF_Compute);

struct FGPUTileDescription
{
	FIntPoint LightmapSize;
	FIntPoint VirtualTilePosition;
	FIntPoint WorkingSetPosition;
	FIntPoint ScratchPosition;
	FIntPoint OutputLayer0Position;
	FIntPoint OutputLayer1Position;
	FIntPoint OutputLayer2Position;
	FIntPoint OutputLayer3Position;
	int32 FrameIndex;
	int32 RenderPassIndex;
};

namespace GPULightmass
{

struct FGPUBatchedTileRequests
{
	FBufferRHIRef BatchedTilesBuffer;
	FShaderResourceViewRHIRef BatchedTilesSRV;
	TResourceArray<FGPUTileDescription> BatchedTilesDesc;

	void BuildFromTileDescs(TArray<FLightmapTileRequest>& TileRequests, FLightmapTilePoolGPU& LightmapTilePoolGPU, FLightmapTilePoolGPU& ScratchTilePoolGPU)
	{
		for (const FLightmapTileRequest& Tile : TileRequests)
		{
			FGPUTileDescription TileDesc = {};
			TileDesc.LightmapSize = Tile.RenderState->GetSize();
			TileDesc.VirtualTilePosition = Tile.VirtualCoordinates.Position * GPreviewLightmapVirtualTileSize;
			TileDesc.WorkingSetPosition = LightmapTilePoolGPU.GetPositionFromLinearAddress(Tile.TileAddressInWorkingSet) * GPreviewLightmapPhysicalTileSize;
			TileDesc.ScratchPosition = ScratchTilePoolGPU.GetPositionFromLinearAddress(Tile.TileAddressInScratch) * GPreviewLightmapPhysicalTileSize;
			TileDesc.OutputLayer0Position = Tile.OutputPhysicalCoordinates[0] * GPreviewLightmapPhysicalTileSize;
			TileDesc.OutputLayer1Position = Tile.OutputPhysicalCoordinates[1] * GPreviewLightmapPhysicalTileSize;
			TileDesc.OutputLayer2Position = Tile.OutputPhysicalCoordinates[2] * GPreviewLightmapPhysicalTileSize;
			TileDesc.OutputLayer3Position = Tile.OutputPhysicalCoordinates[3] * GPreviewLightmapPhysicalTileSize;
			TileDesc.FrameIndex = Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).Revision;
			TileDesc.RenderPassIndex = Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).RenderPassIndex;
			BatchedTilesDesc.Add(TileDesc);
		}
	}

	void Commit(uint32 GPUIndex)
	{
		if (BatchedTilesDesc.Num() > 0)
		{
			FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateStructured<FGPUTileDescription>(TEXT("BatchedTilesBuffer"), BatchedTilesDesc.Num())
				.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
				.SetGPUMask(FRHIGPUMask::FromIndex(GPUIndex))
				.SetInitActionResourceArray(&BatchedTilesDesc)
				.DetermineInitialState();

			BatchedTilesBuffer = RHICmdList.CreateBuffer(CreateDesc);
			BatchedTilesSRV = RHICmdList.CreateShaderResourceView(BatchedTilesBuffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(BatchedTilesBuffer));
		}
	}
};

FLightmapRenderer::FLightmapRenderer(FRHICommandList& RHICmdList, FSceneRenderState* InScene)
	: Scene(InScene)
	, LightmapTilePoolGPU(FIntPoint(Scene->Settings->LightmapTilePoolSize))
{
	NumTotalPassesToRender = Scene->Settings->GISamples;
	
	if (Scene->Settings->bUseIrradianceCaching)
	{
		NumTotalPassesToRender += Scene->Settings->IrradianceCacheQuality;	
	}
	
	if (Scene->Settings->bUseFirstBounceRayGuiding)
	{
		NumTotalPassesToRender += Scene->Settings->FirstBounceRayGuidingTrialSamples;
	}

	if (!Scene->Settings->bUseFirstBounceRayGuiding)
	{
		LightmapTilePoolGPU.Initialize(
			RHICmdList,
			{
				{ PF_A32B32G32R32F, FIntPoint(GPreviewLightmapPhysicalTileSize) }, // IrradianceAndSampleCount
				{ PF_A32B32G32R32F, FIntPoint(GPreviewLightmapPhysicalTileSize) }, // SHDirectionality
				{ PF_A32B32G32R32F, FIntPoint(GPreviewLightmapPhysicalTileSize) }, // ShadowMask
				{ PF_A32B32G32R32F, FIntPoint(GPreviewLightmapPhysicalTileSize) }, // ShadowMaskSampleCount
				{ PF_A32B32G32R32F, FIntPoint(GPreviewLightmapPhysicalTileSize) }, // SHCorrectionAndStationarySkyLightBentNormal
			});
	}
	else
	{
		LightmapTilePoolGPU.Initialize(
			RHICmdList,
			{
				{ PF_A32B32G32R32F, FIntPoint(GPreviewLightmapPhysicalTileSize) }, // IrradianceAndSampleCount
				{ PF_A32B32G32R32F, FIntPoint(GPreviewLightmapPhysicalTileSize) }, // SHDirectionality
				{ PF_A32B32G32R32F, FIntPoint(GPreviewLightmapPhysicalTileSize) }, // ShadowMask
				{ PF_A32B32G32R32F, FIntPoint(GPreviewLightmapPhysicalTileSize) }, // ShadowMaskSampleCount
				{ PF_A32B32G32R32F, FIntPoint(GPreviewLightmapPhysicalTileSize) }, // SHCorrectionAndStationarySkyLightBentNormal
				{ PF_R32_UINT, FIntPoint(128) }, // RayGuidingLuminance
				{ PF_R32_FLOAT, FIntPoint(128) }, // RayGuidingCDFX
				{ PF_R32_FLOAT, FIntPoint(32) }, // RayGuidingCDFY
			});
	}

	bDenoiseDuringInteractiveBake = Scene->Settings->DenoisingOptions == EGPULightmassDenoisingOptions::DuringInteractivePreview;
	bOnlyBakeWhatYouSee = Scene->Settings->Mode == EGPULightmassMode::BakeWhatYouSee;
	DenoisingThreadPool = FQueuedThreadPool::Allocate();
	DenoisingThreadPool->Create(1, 64 * 1024 * 1024);

	if (bOnlyBakeWhatYouSee)
	{
		TilesVisibleLastFewFrames.AddDefaulted(60);
	}

	IrradianceCacheVisualizationDelegateHandle = GetRendererModule().RegisterPostOpaqueRenderDelegate(FPostOpaqueRenderDelegate::CreateRaw(this, &FLightmapRenderer::RenderIrradianceCacheVisualization));
}

FLightmapRenderer::~FLightmapRenderer()
{
	delete DenoisingThreadPool;

	GetRendererModule().RemovePostOpaqueRenderDelegate(IrradianceCacheVisualizationDelegateHandle);
}

void FLightmapRenderer::AddRequest(FLightmapTileRequest TileRequest)
{
	PendingTileRequests.AddUnique(TileRequest);
}

int32 FSceneRenderState::GetPrimitiveIdForGPUScene(const FGeometryInstanceRenderStateRef& GeometryInstanceRef) const
{
	int32 PrimitiveId = GeometryInstanceRef.GetElementIdChecked();
	if (StaticMeshInstanceRenderStates.Contains(GeometryInstanceRef)) return PrimitiveId;
	PrimitiveId += StaticMeshInstanceRenderStates.Elements.Num();
	if (InstanceGroupRenderStates.Contains(GeometryInstanceRef)) return PrimitiveId;
	PrimitiveId += InstanceGroupRenderStates.Elements.Num();
	if (LandscapeRenderStates.Contains(GeometryInstanceRef)) return PrimitiveId;

	checkf(false, TEXT("The referenced geometry isn't in any of the geometry arrays"));
	return INDEX_NONE;
}

void FCachedRayTracingSceneData::SetupViewAndSceneUniformBufferFromSceneRenderState(FRDGBuilder& GraphBuilder, FSceneRenderState& Scene, FSceneUniformBuffer& SceneUniforms)
{
	TArray<FPrimitiveSceneShaderData>	PrimitiveSceneData;
	TArray<FLightmapSceneShaderData>	LightmapSceneData;
	TArray<FInstanceSceneShaderData>	InstanceSceneData;
	TArray<FVector4f>					InstancePayloadData;

	PrimitiveSceneData.AddZeroed(Scene.StaticMeshInstanceRenderStates.Elements.Num());
	InstanceSceneData.AddZeroed(Scene.StaticMeshInstanceRenderStates.Elements.Num());
	InstanceDataOriginalOffsets.AddZeroed(Scene.StaticMeshInstanceRenderStates.Elements.Num());

	TArray<int32> LightmapSceneDataStartOffsets;
	LightmapSceneDataStartOffsets.AddZeroed(Scene.StaticMeshInstanceRenderStates.Elements.Num());
	LightmapSceneDataStartOffsets.AddZeroed(Scene.InstanceGroupRenderStates.Elements.Num());
	LightmapSceneDataStartOffsets.AddZeroed(Scene.LandscapeRenderStates.Elements.Num());

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ComputePrefixSum);

		int32 ConservativeLightmapEntriesNum = 0;
		int32 PrimitiveId = 0;

		for (int32 InstanceIndex = 0; InstanceIndex < Scene.StaticMeshInstanceRenderStates.Elements.Num(); InstanceIndex++)
		{
			FStaticMeshInstanceRenderState& Instance = Scene.StaticMeshInstanceRenderStates.Elements[InstanceIndex];
			LightmapSceneDataStartOffsets[PrimitiveId++] = ConservativeLightmapEntriesNum;
			ConservativeLightmapEntriesNum += Instance.LODLightmapRenderStates.Num();
		}

		for (int32 InstanceGroupIndex = 0; InstanceGroupIndex < Scene.InstanceGroupRenderStates.Elements.Num(); InstanceGroupIndex++)
		{
			FInstanceGroupRenderState& InstanceGroup = Scene.InstanceGroupRenderStates.Elements[InstanceGroupIndex];
			LightmapSceneDataStartOffsets[PrimitiveId++] = ConservativeLightmapEntriesNum;
			ConservativeLightmapEntriesNum += InstanceGroup.LODLightmapRenderStates.Num();
		}

		for (int32 LandscapeIndex = 0; LandscapeIndex < Scene.LandscapeRenderStates.Elements.Num(); LandscapeIndex++)
		{
			FLandscapeRenderState& Landscape = Scene.LandscapeRenderStates.Elements[LandscapeIndex];
			LightmapSceneDataStartOffsets[PrimitiveId++] = ConservativeLightmapEntriesNum;
			ConservativeLightmapEntriesNum += Landscape.LODLightmapRenderStates.Num();
		}

		LightmapSceneData.AddZeroed(ConservativeLightmapEntriesNum);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SetupGPUScene);

		ParallelFor(Scene.StaticMeshInstanceRenderStates.Elements.Num(),
			[
				this,
				&Scene,
				&PrimitiveSceneData,
				&LightmapSceneData,
				&LightmapSceneDataStartOffsets,
				&InstanceSceneData,
				&InstanceDataOffsets = InstanceDataOriginalOffsets
			](int32 InstanceIndex = 0)
			{
				FStaticMeshInstanceRenderState& Instance = Scene.StaticMeshInstanceRenderStates.Elements[InstanceIndex];

				FPrimitiveUniformShaderParameters PrimitiveUniformShaderParameters =
					FPrimitiveUniformShaderParametersBuilder{}
					.Defaults()
					.LocalToWorld(Instance.LocalToWorld)
					.ActorWorldPosition(Instance.ActorPosition)
					.WorldBounds(Instance.WorldBounds)
					.LocalBounds(Instance.LocalBounds)
					.LightingChannelMask(0b111)
					.LightmapDataIndex(LightmapSceneDataStartOffsets[InstanceIndex])
					.InstanceSceneDataOffset(InstanceIndex)
					.NumInstanceSceneDataEntries(1)
					.InstancePayloadDataOffset(INDEX_NONE)
					.InstancePayloadDataStride(0)
				.Build();
				
				PrimitiveSceneData[InstanceIndex] = FPrimitiveSceneShaderData(PrimitiveUniformShaderParameters);

				InstanceDataOffsets[InstanceIndex] = InstanceIndex;

				FInstanceSceneShaderData& SceneData = InstanceSceneData[InstanceIndex];
				SceneData.Build
				(
					InstanceIndex, /* Primitive Id */
					0, /* Relative Instance Id */
					0, /* Payload Data Flags */
					INVALID_LAST_UPDATE_FRAME,
					0, /* Custom Data Count */
					0.0f, /* Random ID */
					FRenderTransform::Identity,
					PrimitiveUniformShaderParameters.LocalToRelativeWorld
				);

				for (int32 LODIndex = 0; LODIndex < Instance.LODLightmapRenderStates.Num(); LODIndex++)
				{
					FPrecomputedLightingUniformParameters LightmapParams;
					GetDefaultPrecomputedLightingParameters(LightmapParams);

					if (Instance.LODLightmapRenderStates[LODIndex].IsValid())
					{
						LightmapParams.LightmapVTPackedPageTableUniform[0] = Instance.LODLightmapRenderStates[LODIndex]->LightmapVTPackedPageTableUniform[0];
						for (uint32 LayerIndex = 0u; LayerIndex < 5u; ++LayerIndex)
						{
							LightmapParams.LightmapVTPackedUniform[LayerIndex] = Instance.LODLightmapRenderStates[LODIndex]->LightmapVTPackedUniform[LayerIndex];
						}

						LightmapParams.LightMapCoordinateScaleBias = Instance.LODLightmapRenderStates[LODIndex]->LightmapCoordinateScaleBias;
					}

					LightmapSceneData[LightmapSceneDataStartOffsets[InstanceIndex] + LODIndex] = FLightmapSceneShaderData(LightmapParams);
				}
			});

		int32 PrimitiveId = Scene.StaticMeshInstanceRenderStates.Elements.Num();

		for (int32 InstanceGroupIndex = 0; InstanceGroupIndex < Scene.InstanceGroupRenderStates.Elements.Num(); InstanceGroupIndex++)
		{
			FInstanceGroupRenderState& InstanceGroup = Scene.InstanceGroupRenderStates.Elements[InstanceGroupIndex];

			int32 NumInstancesThisGroup = (int32)InstanceGroup.NumInstances;

			FPrimitiveUniformShaderParameters PrimitiveUniformShaderParameters =
				FPrimitiveUniformShaderParametersBuilder{}
				.Defaults()
					.LocalToWorld(InstanceGroup.LocalToWorld)
					.ActorWorldPosition(InstanceGroup.ActorPosition)
					.WorldBounds(InstanceGroup.WorldBounds)
					.LocalBounds(InstanceGroup.LocalBounds)
					.LightingChannelMask(0b111)
					.LightmapDataIndex(LightmapSceneDataStartOffsets[PrimitiveId])
					.InstanceSceneDataOffset(InstanceSceneData.Num())
					.NumInstanceSceneDataEntries(NumInstancesThisGroup)
					.InstancePayloadDataOffset(InstancePayloadData.Num())
					.InstancePayloadDataStride(1)
				.Build();

			InstanceDataOriginalOffsets.Add(InstanceSceneData.Num());

			for (int32 LODIndex = 0; LODIndex < InstanceGroup.LODLightmapRenderStates.Num(); LODIndex++)
			{
				FPrecomputedLightingUniformParameters LightmapParams;
				GetDefaultPrecomputedLightingParameters(LightmapParams);

				if (InstanceGroup.LODLightmapRenderStates[LODIndex].IsValid())
				{
					LightmapParams.LightmapVTPackedPageTableUniform[0] = InstanceGroup.LODLightmapRenderStates[LODIndex]->LightmapVTPackedPageTableUniform[0];
					for (uint32 LayerIndex = 0u; LayerIndex < 5u; ++LayerIndex)
					{
						LightmapParams.LightmapVTPackedUniform[LayerIndex] = InstanceGroup.LODLightmapRenderStates[LODIndex]->LightmapVTPackedUniform[LayerIndex];
					}

					LightmapParams.LightMapCoordinateScaleBias = InstanceGroup.LODLightmapRenderStates[LODIndex]->LightmapCoordinateScaleBias;
				}

				LightmapSceneData[LightmapSceneDataStartOffsets[PrimitiveId] + LODIndex] = FLightmapSceneShaderData(LightmapParams);
			}

			FInstanceSceneDataBuffers::FReadView InstanceData = InstanceGroup.InstanceSceneDataBuffers->GetReadView();
			for (int32 InstanceIdx = 0; InstanceIdx < NumInstancesThisGroup; InstanceIdx++)
			{
				FInstanceSceneShaderData& SceneData = InstanceSceneData.Emplace_GetRef();
				SceneData.BuildInternal(
					PrimitiveId,
					InstanceIdx, /* Relative Instance Id */
					INSTANCE_SCENE_DATA_FLAG_HAS_LIGHTSHADOW_UV_BIAS, /* Payload Data Flags */
					INVALID_LAST_UPDATE_FRAME,
					0, /* Custom Data Count */
					0.0f, /* Random ID */
					InstanceGroup.InstanceSceneDataBuffers->GetInstanceToPrimitiveRelative(InstanceIdx),
					true,
					FInstanceSceneShaderData::SupportsCompressedTransforms()
				);

				InstancePayloadData.Emplace(InstanceData.InstanceLightShadowUVBias[InstanceIdx]);
			}

			PrimitiveSceneData.Add(FPrimitiveSceneShaderData(PrimitiveUniformShaderParameters));

			PrimitiveId++;
		}

		for (int32 LandscapeIndex = 0; LandscapeIndex < Scene.LandscapeRenderStates.Elements.Num(); LandscapeIndex++)
		{
			FLandscapeRenderState& Landscape = Scene.LandscapeRenderStates.Elements[LandscapeIndex];

			FPrimitiveUniformShaderParameters PrimitiveUniformShaderParameters =
				FPrimitiveUniformShaderParametersBuilder{}
				.Defaults()
					.LocalToWorld(Landscape.LocalToWorld)
					.ActorWorldPosition(Landscape.ActorPosition)
					.WorldBounds(Landscape.WorldBounds)
					.LocalBounds(Landscape.LocalBounds)
					.LightingChannelMask(0b111)
					.LightmapDataIndex(LightmapSceneDataStartOffsets[PrimitiveId])
					.InstanceSceneDataOffset(InstanceSceneData.Num())
					.NumInstanceSceneDataEntries(1)
				.Build();

			InstanceDataOriginalOffsets.Add(InstanceSceneData.Num());

			for (int32 LODIndex = 0; LODIndex < Landscape.LODLightmapRenderStates.Num(); LODIndex++)
			{
				FPrecomputedLightingUniformParameters LightmapParams;
				GetDefaultPrecomputedLightingParameters(LightmapParams);

				if (Landscape.LODLightmapRenderStates[LODIndex].IsValid())
				{
					LightmapParams.LightmapVTPackedPageTableUniform[0] = Landscape.LODLightmapRenderStates[LODIndex]->LightmapVTPackedPageTableUniform[0];
					for (uint32 LayerIndex = 0u; LayerIndex < 5u; ++LayerIndex)
					{
						LightmapParams.LightmapVTPackedUniform[LayerIndex] = Landscape.LODLightmapRenderStates[LODIndex]->LightmapVTPackedUniform[LayerIndex];
					}

					LightmapParams.LightMapCoordinateScaleBias = Landscape.LODLightmapRenderStates[LODIndex]->LightmapCoordinateScaleBias;
				}

				LightmapSceneData[LightmapSceneDataStartOffsets[PrimitiveId] + LODIndex] = FLightmapSceneShaderData(LightmapParams);
			}

			FInstanceSceneShaderData& Instance = InstanceSceneData.Emplace_GetRef();
			Instance.BuildInternal
			(
				PrimitiveId,
				0, /* Relative Instance Id */
				0, /* Payload Data Flags */
				INVALID_LAST_UPDATE_FRAME,
				0, /* Custom Data Count */
				0.0f, /* Random ID */
				PrimitiveUniformShaderParameters.LocalToRelativeWorld,
				true,
				FInstanceSceneShaderData::SupportsCompressedTransforms()
			);

			PrimitiveSceneData.Add(FPrimitiveSceneShaderData(PrimitiveUniformShaderParameters));

			PrimitiveId++;
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SetupSceneBuffers);

		FGPUSceneResourceParameters GPUScene{};
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PrimitiveSceneData);

			if (PrimitiveSceneData.Num() == 0)
			{
				PrimitiveSceneData.Add(FPrimitiveSceneShaderData(GetIdentityPrimitiveParameters()));
			}
			
			FRDGBufferRef RDGPrimitiveSceneDataBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("PrimitiveSceneDataBuffer"), PrimitiveSceneData);
			GPUScenePrimitiveDataBuffer = GraphBuilder.ConvertToExternalBuffer(RDGPrimitiveSceneDataBuffer);
			GPUScene.GPUScenePrimitiveSceneData = GraphBuilder.CreateSRV(RDGPrimitiveSceneDataBuffer);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LightmapSceneData);

			if (LightmapSceneData.Num() == 0)
			{
				LightmapSceneData.Add(FLightmapSceneShaderData());
			}

			FRDGBufferRef RDGLightmapSceneDataBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("LightmapSceneDataBuffer"), LightmapSceneData);
			GPUSceneLightmapDataBuffer = GraphBuilder.ConvertToExternalBuffer(RDGLightmapSceneDataBuffer);
			GPUScene.GPUSceneLightmapData = GraphBuilder.CreateSRV(RDGLightmapSceneDataBuffer);
		}
		
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(InstanceSceneData);

			GPUSceneNumInstances = InstanceSceneData.Num();
			GPUSceneInstanceDataSOAStride = FMath::Max(1u, FMath::RoundUpToPowerOfTwo(InstanceSceneData.Num()));

			FRDGBufferRef RDGInstanceSceneDataBuffer = nullptr;
			{
				TArray<FVector4f> InstanceSceneDataSOA;
				InstanceSceneDataSOA.AddZeroed(FInstanceSceneShaderData::GetDataStrideInFloat4s() * GPUSceneInstanceDataSOAStride);
				for (uint32 ArrayIndex = 0; ArrayIndex < FInstanceSceneShaderData::GetDataStrideInFloat4s(); ArrayIndex++)
				{
					for (int32 DataIndex = 0; DataIndex < InstanceSceneData.Num(); DataIndex++)
					{
						InstanceSceneDataSOA[ArrayIndex * GPUSceneInstanceDataSOAStride + DataIndex] = InstanceSceneData[DataIndex].Data[ArrayIndex];
					}
				}

				if (InstanceSceneDataSOA.Num() == 0)
				{
					InstanceSceneDataSOA.AddZeroed(FInstanceSceneShaderData::GetDataStrideInFloat4s());
				}

				RDGInstanceSceneDataBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("InstanceSceneDataBuffer"), MoveTemp(InstanceSceneDataSOA));
			}
			GPUSceneInstanceDataBuffer = GraphBuilder.ConvertToExternalBuffer(RDGInstanceSceneDataBuffer);
			GPUScene.GPUSceneInstanceSceneData = GraphBuilder.CreateSRV(RDGInstanceSceneDataBuffer);

			check(FMath::IsPowerOfTwo(GPUSceneInstanceDataSOAStride));
			GPUScene.CommonParameters.GPUSceneInstanceDataTileSizeLog2 = FMath::FloorLog2(GPUSceneInstanceDataSOAStride);
			GPUScene.CommonParameters.GPUSceneInstanceDataTileSizeMask = (1u << GPUScene.CommonParameters.GPUSceneInstanceDataTileSizeLog2) - 1u;
			GPUScene.CommonParameters.GPUSceneInstanceDataTileStride = FInstanceSceneShaderData::GetDataStrideInFloat4s() << GPUScene.CommonParameters.GPUSceneInstanceDataTileSizeLog2;
			GPUScene.CommonParameters.GPUSceneFrameNumber = 0u;
			GPUScene.CommonParameters.GPUSceneMaxAllocatedInstanceId = GPUSceneNumInstances;
			GPUScene.CommonParameters.GPUSceneMaxPersistentPrimitiveIndex = 0;
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(InstancePayloadData);

			if (InstancePayloadData.Num() == 0)
			{
				InstancePayloadData.Add(FVector4f());
			}

			FRDGBufferRef RDGInstancePayloadDataBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("InstancePayloadDataBuffer"), InstancePayloadData);
			GPUSceneInstancePayloadDataBuffer = GraphBuilder.ConvertToExternalBuffer(RDGInstancePayloadDataBuffer);
			GPUScene.GPUSceneInstancePayloadData = GraphBuilder.CreateSRV(RDGInstancePayloadDataBuffer);
		}

		{
			FRDGBufferRef RDGLightDataBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FLightSceneData));
			GPUSceneLightDataBuffer = GraphBuilder.ConvertToExternalBuffer(RDGLightDataBuffer);
			GPUScene.GPUSceneLightData = GraphBuilder.CreateSRV(RDGLightDataBuffer);
		}

		SceneUniforms.Set(SceneUB::GPUScene, GPUScene);
	}
}

//-------------------------------------------------------
// LightmapRenderer related mask update
//-------------------------------------------------------
struct FRayTracingMaskAndStatus
{
	uint8 InstanceMask = 0;
	bool bAllSegmentsUnlit = true;
	bool bAllSegmentsOpaque = true;
	bool bAnySegmentsCastShadow = false;

	void UpdateInstanceMaskAndStatus(ERHIFeatureLevel::Type FeatureLevel, TArray<FMeshBatch>& MeshBatches)
	{
		for (int32 SegmentIndex = 0; SegmentIndex < MeshBatches.Num(); SegmentIndex++)
		{
			const FMeshBatch& MeshBatch = MeshBatches[SegmentIndex];

			const FMaterial& Material = MeshBatch.MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
			const EBlendMode BlendMode = Material.GetBlendMode();

			const bool bSegmentCastsShadow = MeshBatch.CastRayTracedShadow && Material.CastsRayTracedShadows(); // TODO: && BlendMode != BLEND_Additive;

			bAllSegmentsUnlit &= Material.GetShadingModels().HasOnlyShadingModel(MSM_Unlit) || !MeshBatch.CastShadow;
			bAllSegmentsOpaque &= Material.GetBlendMode() == EBlendMode::BLEND_Opaque;
			bAnySegmentsCastShadow |= bSegmentCastsShadow;
			InstanceMask |= BlendModeToRayTracingInstanceMask(Material.GetBlendMode(), Material.IsDitherMasked(), bSegmentCastsShadow, ERayTracingType::LightMapTracing);
		}
	}
};

void FCachedRayTracingSceneData::SetupFromSceneRenderState(FSceneRenderState& Scene)
{
#if RHI_RAYTRACING
	RayTracingGeometryInstancesPerLOD.AddDefaulted(MAX_STATIC_MESH_LODS);
	ShaderBindingsPerLOD.AddDefaulted(MAX_STATIC_MESH_LODS);
	RayTracingNumSegmentsPerLOD.AddDefaulted(MAX_STATIC_MESH_LODS);

	FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions();

	for (int32 LODIndex = 0; LODIndex < MAX_STATIC_MESH_LODS; LODIndex++)
	{
		bool bShouldIncludeThisLODLevel = false;

		for (int32 StaticMeshIndex = 0; StaticMeshIndex < Scene.StaticMeshInstanceRenderStates.Elements.Num(); StaticMeshIndex++)
		{
			FStaticMeshInstanceRenderState& Instance = Scene.StaticMeshInstanceRenderStates.Elements[StaticMeshIndex];
			int32 LODIndexToUse = FMath::Clamp(LODIndex, Instance.ClampedMinLOD, Instance.RenderData->LODResources.Num() - 1);
			if (LODIndexToUse == LODIndex)
			{
				bShouldIncludeThisLODLevel = true;
				break;
			}
		}

		for (int32 InstanceGroupIndex = 0; InstanceGroupIndex < Scene.InstanceGroupRenderStates.Elements.Num(); InstanceGroupIndex++)
		{
			FInstanceGroupRenderState& InstanceGroup = Scene.InstanceGroupRenderStates.Elements[InstanceGroupIndex];
			int32 LODIndexToUse = FMath::Clamp(LODIndex, 0, InstanceGroup.ComponentUObject->GetStaticMesh()->GetRenderData()->LODResources.Num() - 1);
			if (LODIndexToUse == LODIndex)
			{
				bShouldIncludeThisLODLevel = true;
				break;
			}
		}

		if (!bShouldIncludeThisLODLevel)
		{
			continue;
		}

		RayTracingGeometryInstancesPerLOD[LODIndex].Reserve(Scene.StaticMeshInstanceRenderStates.Elements.Num());

		for (int32 StaticMeshIndex = 0; StaticMeshIndex < Scene.StaticMeshInstanceRenderStates.Elements.Num(); StaticMeshIndex++)
		{
			FStaticMeshInstanceRenderState& Instance = Scene.StaticMeshInstanceRenderStates.Elements[StaticMeshIndex];

			int32 LODIndexToUse = FMath::Clamp(LODIndex, Instance.ClampedMinLOD, Instance.RenderData->LODResources.Num() - 1);

			TArray<FMeshBatch> MeshBatches = Instance.GetMeshBatchesForGBufferRendering(LODIndexToUse);

			FRayTracingMaskAndStatus RayTracingMaskAndStatus;
			RayTracingMaskAndStatus.UpdateInstanceMaskAndStatus(Scene.FeatureLevel, MeshBatches);

			if (!RayTracingMaskAndStatus.bAllSegmentsUnlit)
			{
				FRayTracingSBTAllocation* SBTAllocation = RaytracingSBT.AllocateStaticRange(ERayTracingShaderBindingLayerMask::Base, MeshBatches.Num());
				StaticSBTAllocations.Add(SBTAllocation);

				RayTracingNumSegmentsPerLOD[LODIndex] += MeshBatches.Num();

				const FRayTracingGeometry* RayTracingGeometry = Instance.RenderData->RayTracingProxy->LODs[LODIndexToUse].RayTracingGeometry;

				FRayTracingGeometryInstance& RayTracingInstance = RayTracingGeometryInstancesPerLOD[LODIndex].AddDefaulted_GetRef();
				RayTracingInstance.GeometryRHI = RayTracingGeometry->GetRHI();
				RayTracingInstance.Transforms = MakeArrayView(&Instance.LocalToWorld, 1);
				RayTracingInstance.NumTransforms = 1;
				RayTracingInstance.InstanceContributionToHitGroupIndex = SBTAllocation->GetInstanceContributionToHitGroupIndex(ERayTracingShaderBindingLayer::Base);
				RayTracingInstance.DefaultUserData = (uint32)StaticMeshIndex;
				RayTracingInstance.Mask = RayTracingMaskAndStatus.InstanceMask;
				if (RayTracingMaskAndStatus.bAllSegmentsOpaque)
				{
					RayTracingInstance.Flags |= ERayTracingInstanceFlags::ForceOpaque;
				}

				ensure(RayTracingInstance.GeometryRHI->GetNumSegments() == MeshBatches.Num());

				for (int32 SegmentIndex = 0; SegmentIndex < MeshBatches.Num(); SegmentIndex++)
				{
					FFullyCachedRayTracingMeshCommandContext CommandContext(MeshCommandStorage, ShaderBindingsPerLOD[LODIndex], RayTracingInstance.GeometryRHI, SegmentIndex, RayTracingGeometry->GetInitializer().bAllowUpdate, SBTAllocation);
					FLightmapRayTracingMeshProcessor RayTracingMeshProcessor(&CommandContext);

					RayTracingMeshProcessor.AddMeshBatch(MeshBatches[SegmentIndex], 1, nullptr);
				}
			}
		}

		RayTracingGeometryInstancesPerLOD[LODIndex].Reserve(Scene.InstanceGroupRenderStates.Elements.Num());

		for (int32 InstanceGroupIndex = 0; InstanceGroupIndex < Scene.InstanceGroupRenderStates.Elements.Num(); InstanceGroupIndex++)
		{
			FInstanceGroupRenderState& InstanceGroup = Scene.InstanceGroupRenderStates.Elements[InstanceGroupIndex];

			int32 LODIndexToUse = FMath::Clamp(LODIndex, 0, InstanceGroup.ComponentUObject->GetStaticMesh()->GetRenderData()->LODResources.Num() - 1);

			TArray<FMeshBatch> MeshBatches = InstanceGroup.GetMeshBatchesForGBufferRendering(LODIndexToUse, FTileVirtualCoordinates{});

			FRayTracingMaskAndStatus RayTracingMaskAndStatus;
			RayTracingMaskAndStatus.UpdateInstanceMaskAndStatus(Scene.FeatureLevel, MeshBatches);

			if (!RayTracingMaskAndStatus.bAllSegmentsUnlit)
			{
				FRayTracingSBTAllocation* SBTAllocation = RaytracingSBT.AllocateStaticRange(ERayTracingShaderBindingLayerMask::Base, MeshBatches.Num());
				StaticSBTAllocations.Add(SBTAllocation);

				RayTracingNumSegmentsPerLOD[LODIndex] += MeshBatches.Num();

				const FRayTracingGeometry* RayTracingGeometry = InstanceGroup.ComponentUObject->GetStaticMesh()->GetRenderData()->RayTracingProxy->LODs[LODIndexToUse].RayTracingGeometry;

				FRayTracingGeometryInstance& RayTracingInstance = RayTracingGeometryInstancesPerLOD[LODIndex].AddDefaulted_GetRef();
				RayTracingInstance.GeometryRHI = RayTracingGeometry->GetRHI();

				const int32 NumInstances = (int32)InstanceGroup.NumInstances;
				TArrayView<FMatrix> NewTransforms = MakeArrayView(
					OwnedRayTracingInstanceTransforms.Add_GetRef(TUniquePtr<FMatrix>(new FMatrix[NumInstances])).Get(),
					NumInstances);

				for (int32 InstanceIdx = 0; InstanceIdx < NumInstances; InstanceIdx++)
				{
					NewTransforms[InstanceIdx] = InstanceGroup.InstanceSceneDataBuffers->GetInstanceToWorld(InstanceIdx);
				}

				RayTracingInstance.Transforms = NewTransforms;
				RayTracingInstance.NumTransforms = NumInstances;

				RayTracingInstance.InstanceContributionToHitGroupIndex = SBTAllocation->GetInstanceContributionToHitGroupIndex(ERayTracingShaderBindingLayer::Base);

				RayTracingInstance.DefaultUserData = (uint32)(Scene.StaticMeshInstanceRenderStates.Elements.Num() + InstanceGroupIndex);
				RayTracingInstance.Mask = RayTracingMaskAndStatus.InstanceMask;

				if (RayTracingMaskAndStatus.bAllSegmentsOpaque)
				{
					RayTracingInstance.Flags |= ERayTracingInstanceFlags::ForceOpaque;
				}

				ensure(RayTracingInstance.GeometryRHI->GetNumSegments() == MeshBatches.Num());

				for (int32 SegmentIndex = 0; SegmentIndex < MeshBatches.Num(); SegmentIndex++)
				{
					FFullyCachedRayTracingMeshCommandContext CommandContext(MeshCommandStorage, ShaderBindingsPerLOD[LODIndex], RayTracingInstance.GeometryRHI, SegmentIndex, RayTracingGeometry->GetInitializer().bAllowUpdate, SBTAllocation);
					FLightmapRayTracingMeshProcessor RayTracingMeshProcessor(&CommandContext);

					RayTracingMeshProcessor.AddMeshBatch(MeshBatches[SegmentIndex], 1, nullptr);
				}
			}
		}
	}
#else // RHI_RAYTRACING
	checkNoEntry();
#endif // RHI_RAYTRACING
}

void FCachedRayTracingSceneData::RestoreCachedBuffers(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniforms)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RestoreCachedBuffers);

	FGPUSceneResourceParameters GPUScene = {};
	
	check(GPUScenePrimitiveDataBuffer.IsValid())
	GPUScene.GPUScenePrimitiveSceneData = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(GPUScenePrimitiveDataBuffer));

	check(GPUSceneLightmapDataBuffer.IsValid())
	GPUScene.GPUSceneLightmapData = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(GPUSceneLightmapDataBuffer));

	check(GPUSceneInstanceDataBuffer.IsValid())
	GPUScene.GPUSceneInstanceSceneData = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(GPUSceneInstanceDataBuffer));

	check(FMath::IsPowerOfTwo(GPUSceneInstanceDataSOAStride));
	GPUScene.CommonParameters.GPUSceneInstanceDataTileSizeLog2 = FMath::FloorLog2(GPUSceneInstanceDataSOAStride);
	GPUScene.CommonParameters.GPUSceneInstanceDataTileSizeMask = (1u << GPUScene.CommonParameters.GPUSceneInstanceDataTileSizeLog2) - 1u;
	GPUScene.CommonParameters.GPUSceneInstanceDataTileStride = FInstanceSceneShaderData::GetDataStrideInFloat4s() << GPUScene.CommonParameters.GPUSceneInstanceDataTileSizeLog2;
	GPUScene.CommonParameters.GPUSceneFrameNumber = 0u;
	GPUScene.CommonParameters.GPUSceneMaxAllocatedInstanceId = GPUSceneNumInstances;
	GPUScene.CommonParameters.GPUSceneMaxPersistentPrimitiveIndex = 0;

	check(GPUSceneInstancePayloadDataBuffer.IsValid())
	GPUScene.GPUSceneInstancePayloadData = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(GPUSceneInstancePayloadDataBuffer));

	check(GPUSceneLightDataBuffer.IsValid())
	GPUScene.GPUSceneLightData = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(GPUSceneLightDataBuffer));

	SceneUniforms.Set(SceneUB::GPUScene, GPUScene);
}

FCachedRayTracingSceneData::~FCachedRayTracingSceneData()
{
	// Release all static allocated SBT entries
	for (FRayTracingSBTAllocation* SBTAllocation : StaticSBTAllocations)
	{
		RaytracingSBT.FreeStaticRange(SBTAllocation);
	}

	RaytracingSBT.FlushAllocationsToClear(FRHICommandListImmediate::Get());

	// Move OwnedRayTracingInstanceTransforms to RHIThread and it will get destroyed when the lambda is released
	FRHICommandListImmediate::Get().EnqueueLambda([OwnedRayTracingInstanceTransforms = MoveTemp(OwnedRayTracingInstanceTransforms)](FRHICommandList&){});
}

BEGIN_SHADER_PARAMETER_STRUCT(FRayTracingSceneBuildPassParams, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
END_SHADER_PARAMETER_STRUCT()

bool FSceneRenderState::SetupRayTracingScene(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniforms, int32 LODIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SetupRayTracingScene);

	// Make sure the buffer is prepped
	SceneUniforms.GetBuffer(GraphBuilder);

#if RHI_RAYTRACING
	// Force build all the open build requests
	bool bBuildAll = true;
	GRayTracingGeometryManager->ProcessBuildRequests(GraphBuilder.RHICmdList, bBuildAll);
#endif // RHI_RAYTRACING

	if (CachedRayTracingScene.IsValid())
	{
		CachedRayTracingScene->RestoreCachedBuffers(GraphBuilder, SceneUniforms);
	}
	else
	{
		CachedRayTracingScene = MakeUnique<FCachedRayTracingSceneData>();

		CachedRayTracingScene->SetupViewAndSceneUniformBufferFromSceneRenderState(GraphBuilder, *this, SceneUniforms);
		CachedRayTracingScene->SetupFromSceneRenderState(*this);

		CalculateDistributionPrefixSumForAllLightmaps();
	}

	// If no LOD level is specified, select the first non empty level (and merge it with landscapes later)
	if (LODIndex == INDEX_NONE)
	{
		for (int32 NonEmptyLODIndex = 0; NonEmptyLODIndex < MAX_STATIC_MESH_LODS; NonEmptyLODIndex++)
		{
			if (CachedRayTracingScene->RayTracingGeometryInstancesPerLOD[NonEmptyLODIndex].Num() > 0)
			{
				LODIndex = NonEmptyLODIndex;
				break;
			}
		}
	}

#if 0 // Debug: verify cached ray tracing scene has up-to-date shader bindings
	TUniquePtr<FCachedRayTracingSceneData> VerificationRayTracingScene = MakeUnique<FCachedRayTracingSceneData>();
	VerificationRayTracingScene->SetupFromSceneRenderState(*this);

	check(CachedRayTracingScene->VisibleRayTracingMeshCommands.Num() == VerificationRayTracingScene->VisibleRayTracingMeshCommands.Num());
	check(CachedRayTracingScene->MeshCommandStorage.Num() == VerificationRayTracingScene->MeshCommandStorage.Num());

	for (int32 CommandIndex = 0; CommandIndex < CachedRayTracingScene->VisibleRayTracingMeshCommands.Num(); CommandIndex++)
	{
		const FVisibleRayTracingMeshCommand& VisibleMeshCommand = CachedRayTracingScene->VisibleRayTracingMeshCommands[CommandIndex];
		const FRayTracingMeshCommand& MeshCommand = *VisibleMeshCommand.RayTracingMeshCommand;
		const FRayTracingMeshCommand& VerificationMeshCommand = *VerificationRayTracingScene->VisibleRayTracingMeshCommands[CommandIndex].RayTracingMeshCommand;
		check(MeshCommand.ShaderBindings.GetDynamicInstancingHash() == VerificationMeshCommand.ShaderBindings.GetDynamicInstancingHash());
		MeshCommand.ShaderBindings.MatchesForDynamicInstancing(VerificationMeshCommand.ShaderBindings);
	}
#endif

	FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(
		nullptr,
		nullptr,
		FEngineShowFlags(ESFIM_Game))
		.SetTime(FGameTime()));

	const FIntRect ViewRect(FIntPoint(0, 0), FIntPoint(GPreviewLightmapPhysicalTileSize, GPreviewLightmapPhysicalTileSize));

	// make a temporary view
	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.ViewOrigin = FVector::ZeroVector;
	ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
	ViewInitOptions.ProjectionMatrix = FCanvas::CalcBaseTransform2D(GPreviewLightmapPhysicalTileSize, GPreviewLightmapPhysicalTileSize);
	ViewInitOptions.BackgroundColor = FLinearColor::Black;
	ViewInitOptions.OverlayColor = FLinearColor::White;

	ReferenceView = MakeShared<FViewInfo>(ViewInitOptions);
	FViewInfo& View = *ReferenceView;
	ViewFamily.Views.Add(ReferenceView.Get());
	View.ViewRect = View.UnscaledViewRect;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SetupViewBuffers);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SetupUniformBufferParameters);

			// Expanded version of View.InitRHIResources() - need to do SetupSkyIrradianceEnvironmentMapConstants manually because the estimation of skylight is dependent on GetSkySHDiffuse
			View.CachedViewUniformShaderParameters = MakeUnique<FViewUniformShaderParameters>();

			FBox UnusedVolumeBounds[TVC_MAX];
			View.SetupUniformBufferParameters(
				UnusedVolumeBounds,
				TVC_MAX,
				*View.CachedViewUniformShaderParameters);

			if (LightSceneRenderState.SkyLight.IsSet())
			{
				View.CachedViewUniformShaderParameters->SkyIrradianceEnvironmentMap = LightSceneRenderState.SkyLight->SkyIrradianceEnvironmentMap.SRV;
			}
			else
			{
				View.CachedViewUniformShaderParameters->SkyIrradianceEnvironmentMap = GIdentityPrimitiveBuffer.SkyIrradianceEnvironmentMapSRV;
			}

			View.ViewUniformBuffer = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(*View.CachedViewUniformShaderParameters, UniformBuffer_SingleFrame);
		}

		const_cast<TRange<int32>&>(View.DynamicPrimitiveCollector.GetPrimitiveIdRange()) = TRange<int32>(0,
			FMath::Max(StaticMeshInstanceRenderStates.Elements.Num(), FMath::Max(InstanceGroupRenderStates.Elements.Num(), LandscapeRenderStates.Elements.Num()))
			);
		View.DynamicPrimitiveCollector.Commit();
	}

	// Early out if there's nothing in the scene: no instance in the selected LOD level, or no landscape (which effectively exists on every LOD level)
	if ((LODIndex == INDEX_NONE || CachedRayTracingScene->RayTracingGeometryInstancesPerLOD[LODIndex].Num() == 0) && LandscapeRenderStates.Elements.Num() == 0)
	{
		return false;
	}

#if RHI_RAYTRACING

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RayTracingScene);

		FRayTracingSceneBuildPassParams* PassParameters = GraphBuilder.AllocParameters<FRayTracingSceneBuildPassParams>();
		PassParameters->View = ReferenceView->ViewUniformBuffer;
		PassParameters->Scene = SceneUniforms.GetBuffer(GraphBuilder);

		SCOPED_DRAW_EVENTF(GraphBuilder.RHICmdList, GPULightmassUpdateRayTracingScene, TEXT("GPULightmass UpdateRayTracingScene %d Instances"), StaticMeshInstanceRenderStates.Elements.Num());

		TArray<FRayTracingGeometryInstance, SceneRenderingAllocator>& RayTracingGeometryInstances = GraphBuilder.AllocArray<FRayTracingGeometryInstance>();
		if (LODIndex != INDEX_NONE)
		{
			RayTracingGeometryInstances.Append(CachedRayTracingScene->RayTracingGeometryInstancesPerLOD[LODIndex]);
		}
		
		uint32 RayTracingNumSegments = LODIndex != INDEX_NONE ? CachedRayTracingScene->RayTracingNumSegmentsPerLOD[LODIndex] : 0;
		FRayTracingShaderBindingTable& RayTracingSBT = CachedRayTracingScene->RaytracingSBT;
		RayTracingSBT.ResetDynamicAllocationData();

		int32 LandscapeStartOffset = RayTracingGeometryInstances.Num();
		for (FLandscapeRenderState& Landscape : LandscapeRenderStates.Elements)
		{
			for (int32 SubY = 0; SubY < Landscape.NumSubsections; SubY++)
			{
				for (int32 SubX = 0; SubX < Landscape.NumSubsections; SubX++)
				{
					RayTracingGeometryInstances.AddDefaulted(1);
				}
			}
		}

		FRayTracingShaderBindingDataOneFrameArray DynamicRTShaderBindings;
		FDynamicRayTracingMeshCommandStorage DynamicRayTracingMeshCommandStorage;

		TArray<TUniquePtr<FMatrix>> LandscapeTransforms;
		
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Landscapes);

			int32 NumLandscapeInstances = 0;

			for (int32 LandscapeIndex = 0; LandscapeIndex < LandscapeRenderStates.Elements.Num(); LandscapeIndex++)
			{
				FLandscapeRenderState& Landscape = LandscapeRenderStates.Elements[LandscapeIndex];

				for (int32 SubY = 0; SubY < Landscape.NumSubsections; SubY++)
				{
					for (int32 SubX = 0; SubX < Landscape.NumSubsections; SubX++)
					{
						const int8 SubSectionIdx = SubX + SubY * Landscape.NumSubsections;
						uint32 NumPrimitives = FMath::Square(Landscape.SubsectionSizeVerts - 1) * 2;

						int32 InstanceIndex = LandscapeStartOffset + NumLandscapeInstances;
						NumLandscapeInstances++;

						if (!Landscape.SectionRayTracingStates[SubSectionIdx].IsValid())
						{
							FRayTracingGeometryInitializer GeometryInitializer;
							GeometryInitializer.IndexBuffer = Landscape.SharedBuffers->ZeroOffsetIndexBuffers[0]->IndexBufferRHI;
							GeometryInitializer.TotalPrimitiveCount = NumPrimitives;
							GeometryInitializer.GeometryType = RTGT_Triangles;
							GeometryInitializer.bFastBuild = false;
							GeometryInitializer.bAllowUpdate = false;

							FRayTracingGeometrySegment Segment;
							Segment.VertexBuffer = nullptr;
							Segment.VertexBufferStride = sizeof(FVector3f);
							Segment.VertexBufferElementType = VET_Float3;
							Segment.MaxVertices = FMath::Square(Landscape.SubsectionSizeVerts);
							Segment.NumPrimitives = NumPrimitives;
							GeometryInitializer.Segments.Add(Segment);

							Landscape.SectionRayTracingStates[SubSectionIdx] = MakeUnique<FLandscapeRenderState::FLandscapeSectionRayTracingState>();
							Landscape.SectionRayTracingStates[SubSectionIdx]->Geometry.SetInitializer(GeometryInitializer);
							Landscape.SectionRayTracingStates[SubSectionIdx]->Geometry.InitResource(GraphBuilder.RHICmdList);

							FRayTracingDynamicGeometryUpdateManager& DynamicGeometryUpdateManager = *GraphBuilder.AllocObject<FRayTracingDynamicGeometryUpdateManager>();

							TArray<FMeshBatch> MeshBatches = Landscape.GetMeshBatchesForGBufferRendering(0);

							FLandscapeVertexFactoryMVFParameters UniformBufferParams;
							UniformBufferParams.SubXY = FIntPoint(SubX, SubY);
							Landscape.SectionRayTracingStates[SubSectionIdx]->UniformBuffer = FLandscapeVertexFactoryMVFUniformBufferRef::CreateUniformBufferImmediate(UniformBufferParams, UniformBuffer_MultiFrame);

							FLandscapeBatchElementParams& BatchElementParams = *(FLandscapeBatchElementParams*)MeshBatches[0].Elements[0].UserData;
							BatchElementParams.LandscapeVertexFactoryMVFUniformBuffer = Landscape.SectionRayTracingStates[SubSectionIdx]->UniformBuffer;

							MeshBatches[0].Elements[0].IndexBuffer = Landscape.SharedBuffers->ZeroOffsetIndexBuffers[0];
							MeshBatches[0].Elements[0].FirstIndex = 0;
							MeshBatches[0].Elements[0].NumPrimitives = NumPrimitives;
							MeshBatches[0].Elements[0].MinVertexIndex = 0;
							MeshBatches[0].Elements[0].MaxVertexIndex = 0;

							MeshBatches[0].Elements[0].DynamicPrimitiveIndex = LandscapeIndex;
							MeshBatches[0].Elements[0].DynamicPrimitiveIndex += StaticMeshInstanceRenderStates.Elements.Num();
							MeshBatches[0].Elements[0].DynamicPrimitiveIndex += InstanceGroupRenderStates.Elements.Num();

							for (FMeshBatch& MeshBatch : MeshBatches)
							{
								// Override with default material as we're not considering WPO in GPULM landscape creation
								MeshBatch.MaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
							}
							
							FRayTracingDynamicGeometryUpdateParams UpdateParams;
							UpdateParams.MeshBatchesView = MeshBatches;
							UpdateParams.NumVertices = (uint32)FMath::Square(Landscape.SubsectionSizeVerts);
							UpdateParams.VertexBufferSize = (uint32)FMath::Square(Landscape.SubsectionSizeVerts) * (uint32)sizeof(FVector3f);
							UpdateParams.NumTriangles = (uint32)FMath::Square(Landscape.SubsectionSizeVerts - 1) * 2;
							UpdateParams.Geometry = &Landscape.SectionRayTracingStates[SubSectionIdx]->Geometry;
							UpdateParams.Buffer = &Landscape.SectionRayTracingStates[SubSectionIdx]->RayTracingDynamicVertexBuffer;
							UpdateParams.bApplyWorldPositionOffset = false;

							DynamicGeometryUpdateManager.AddDynamicGeometryToUpdate(
								GraphBuilder.RHICmdList,
								Landscape.ComponentUObject->GetWorld()->Scene->GetRenderScene(),
								ReferenceView.Get(),
								nullptr,
								UpdateParams,
								StaticMeshInstanceRenderStates.Elements.Num() + InstanceGroupRenderStates.Elements.Num() + LandscapeIndex
							);
							
							DynamicGeometryUpdateManager.ScheduleUpdates(GraphBuilder, /*bUseTracingFeedback*/ false);

							FRDGBufferRef DynamicGeometryScratchBuffer;
							DynamicGeometryUpdateManager.AddDynamicGeometryUpdatePass(
								GraphBuilder,
								ERDGPassFlags::Compute,
								SceneUniforms.GetBuffer(GraphBuilder),
								ERHIPipeline::Graphics,
								DynamicGeometryScratchBuffer);

							// Landscape VF doesn't really use the vertex buffer in HitGroupSystemParameters
							// We can release after all related RHI cmds get dispatched onto the cmd list
							Landscape.SectionRayTracingStates[SubSectionIdx]->RayTracingDynamicVertexBuffer.Release();
						}

						TArray<FMeshBatch> MeshBatches = Landscape.GetMeshBatchesForGBufferRendering(0);
						ensure(MeshBatches.Num() == 1);

						const uint32 SegmentCount = Landscape.SectionRayTracingStates[SubSectionIdx]->Geometry.GetInitializer().Segments.Num();
						FRayTracingSBTAllocation* SBTAllocation = RayTracingSBT.AllocateDynamicRange(ERayTracingShaderBindingLayerMask::Base, SegmentCount);
						RayTracingNumSegments += SegmentCount;

						const FRayTracingGeometry& RayTracingGeometry = Landscape.SectionRayTracingStates[SubSectionIdx]->Geometry;

						FRayTracingGeometryInstance& RayTracingInstance = RayTracingGeometryInstances[InstanceIndex];
						RayTracingInstance.GeometryRHI = RayTracingGeometry.GetRHI();
						RayTracingInstance.Transforms = MakeArrayView(GraphBuilder.AllocObject<FMatrix>(Landscape.LocalToWorld), 1);
						RayTracingInstance.NumTransforms = 1;
						RayTracingInstance.InstanceContributionToHitGroupIndex = SBTAllocation->GetInstanceContributionToHitGroupIndex(ERayTracingShaderBindingLayer::Base);
						RayTracingInstance.DefaultUserData = (uint32)(StaticMeshInstanceRenderStates.Elements.Num() + InstanceGroupRenderStates.Elements.Num() + LandscapeIndex);

						FLandscapeBatchElementParams& BatchElementParams = *(FLandscapeBatchElementParams*)MeshBatches[0].Elements[0].UserData;
						BatchElementParams.LandscapeVertexFactoryMVFUniformBuffer = Landscape.SectionRayTracingStates[SubSectionIdx]->UniformBuffer;

						MeshBatches[0].Elements[0].IndexBuffer = Landscape.SharedBuffers->ZeroOffsetIndexBuffers[0];
						MeshBatches[0].Elements[0].FirstIndex = 0;
						MeshBatches[0].Elements[0].NumPrimitives = NumPrimitives;
						MeshBatches[0].Elements[0].MinVertexIndex = 0;
						MeshBatches[0].Elements[0].MaxVertexIndex = 0;

						for (const FMeshBatch& MeshBatch : MeshBatches)
						{
							FDynamicRayTracingMeshCommandContext CommandContext(DynamicRayTracingMeshCommandStorage, DynamicRTShaderBindings, RayTracingInstance.GeometryRHI, MeshBatch.SegmentIndex, RayTracingGeometry.GetInitializer().bAllowUpdate, SBTAllocation);
							FLightmapRayTracingMeshProcessor RayTracingMeshProcessor(&CommandContext);

							RayTracingMeshProcessor.AddMeshBatch(MeshBatch, 1, nullptr);
						}

						FRayTracingMaskAndStatus RayTracingMaskAndStatus;
						RayTracingMaskAndStatus.UpdateInstanceMaskAndStatus(FeatureLevel, MeshBatches);

						if (RayTracingMaskAndStatus.bAllSegmentsUnlit)
						{
							RayTracingInstance.Mask = 0;
						}
						else
						{
							RayTracingInstance.Mask = RayTracingMaskAndStatus.InstanceMask;
						}

						if (RayTracingMaskAndStatus.bAllSegmentsOpaque)
						{
							RayTracingInstance.Flags |= ERayTracingInstanceFlags::ForceOpaque;
						}
					}
				}
			}
		}

		if (IsRayTracingEnabled())
		{
			SCOPED_GPU_MASK(GraphBuilder.RHICmdList, FRHIGPUMask::All());

			FRayTracingInstanceBufferBuilder RayTracingInstanceBufferBuilder;
			RayTracingInstanceBufferBuilder.Init(RayTracingGeometryInstances, View.ViewMatrices.GetPreViewTranslation());

			{
				FRayTracingSceneInitializer Initializer;
				Initializer.DebugName = FName(TEXT("LightmapRendererRayTracingScene"));
				Initializer.MaxNumInstances = RayTracingInstanceBufferBuilder.GetMaxNumInstances();
				Initializer.BuildFlags = ERayTracingAccelerationStructureFlags::FastTrace;

				RayTracingScene = RHICreateRayTracingScene(MoveTemp(Initializer));
			}

			const FRayTracingSceneInitializer& SceneInitializer = RayTracingScene->GetInitializer();

			FRayTracingAccelerationStructureSize SizeInfo = RHICalcRayTracingSceneSize(SceneInitializer);

			const FRHIBufferCreateDesc SceneCreateDesc =
				FRHIBufferCreateDesc::Create(TEXT("LightmassRayTracingSceneBuffer"), SizeInfo.ResultSize, 0, EBufferUsageFlags::AccelerationStructure)
				.SetInitialState(ERHIAccess::BVHWrite);

			RayTracingSceneBuffer = GraphBuilder.RHICmdList.CreateBuffer(SceneCreateDesc);
			RayTracingSceneSRV = GraphBuilder.RHICmdList.CreateShaderResourceView(FShaderResourceViewInitializer(RayTracingSceneBuffer, RayTracingScene, 0));

			const FRHIBufferCreateDesc ScratchCreateDesc =
				FRHIBufferCreateDesc::Create(TEXT("LightmassRayTracingScratchBuffer"), SizeInfo.BuildScratchSize, GRHIRayTracingScratchBufferAlignment, EBufferUsageFlags::StructuredBuffer | EBufferUsageFlags::RayTracingScratch)
				.SetInitialState(ERHIAccess::UAVCompute);

			FBufferRHIRef ScratchBuffer = GraphBuilder.RHICmdList.CreateBuffer(ScratchCreateDesc);

			FRWBufferStructured InstanceBuffer;
			InstanceBuffer.Initialize(GraphBuilder.RHICmdList, TEXT("LightmassRayTracingInstanceBuffer"), GRHIRayTracingInstanceDescriptorSize, SceneInitializer.MaxNumInstances);
			
			FRWBufferStructured HitGroupContributionsBuffer;
			
			if(GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer)
			{
				HitGroupContributionsBuffer.Initialize(GraphBuilder.RHICmdList, TEXT("LightmassRayTracingHitGroupBuffer"), 4, SceneInitializer.MaxNumInstances);
			}

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("GPULM::RayTracingBuildScene"),
				PassParameters,
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				[
					RayTracingScene = RayTracingScene,
					RayTracingSceneBuffer = RayTracingSceneBuffer,
					ScratchBuffer = ScratchBuffer,
					MaxNumInstances = SceneInitializer.MaxNumInstances,
					InstanceBufferUAV = InstanceBuffer.UAV,
					InstanceBufferRHI = InstanceBuffer.Buffer,
					bRequiresSeparateHitGroupContributionsBuffer = GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer,
					HitGroupContributionsBufferUAV = HitGroupContributionsBuffer.UAV,
					HitGroupContributionsBufferRHI = HitGroupContributionsBuffer.Buffer,
					RayTracingInstanceBufferBuilder = MoveTemp(RayTracingInstanceBufferBuilder)](FRHICommandListImmediate& RHICmdList) mutable
				{
					RayTracingInstanceBufferBuilder.FillRayTracingInstanceUploadBuffer(RHICmdList);
					RayTracingInstanceBufferBuilder.FillAccelerationStructureAddressesBuffer(RHICmdList);

					RayTracingInstanceBufferBuilder.BuildRayTracingInstanceBuffer(
						RHICmdList,
						nullptr,
						nullptr,
						nullptr,
						InstanceBufferUAV,
						bRequiresSeparateHitGroupContributionsBuffer ? HitGroupContributionsBufferUAV : nullptr,
						MaxNumInstances,
						nullptr,
						false,
						nullptr,
						0,
						nullptr);

					FRayTracingSceneBuildParams BuildParams;
					BuildParams.Scene = RayTracingScene;
					BuildParams.ScratchBuffer = ScratchBuffer;
					BuildParams.ScratchBufferOffset = 0;
					BuildParams.InstanceBuffer = InstanceBufferRHI;
					BuildParams.InstanceBufferOffset = 0;

					if (bRequiresSeparateHitGroupContributionsBuffer)
					{
						BuildParams.HitGroupContributionsBuffer = HitGroupContributionsBufferRHI;
						BuildParams.HitGroupContributionsBufferOffset = 0;
					}

					BuildParams.ReferencedGeometries = RayTracingInstanceBufferBuilder.GetReferencedGeometries();
					BuildParams.NumInstances = RayTracingInstanceBufferBuilder.GetMaxNumInstances();

					RHICmdList.BindAccelerationStructureMemory(RayTracingScene, RayTracingSceneBuffer, 0);
					RHICmdList.BuildAccelerationStructure(BuildParams);
				});

			// Move LandscapeTransforms to RHIThread to extend its lifetime until RHI cmd execution
			GraphBuilder.RHICmdList.EnqueueLambda([LandscapeTransforms = MoveTemp(LandscapeTransforms)](FRHICommandList&){});
			
			FRayTracingPipelineStateInitializer PSOInitializer;

			PSOInitializer.MaxPayloadSizeInBytes = GetRayTracingPayloadTypeMaxSize(ERayTracingPayloadType::GPULightmass);

			FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(FeatureLevel);

			TArray<FRHIRayTracingShader*> RayGenShaderTable;
			{
				FLightmapPathTracingRGS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FLightmapPathTracingRGS::FUseFirstBounceRayGuiding>(Settings->bUseIrradianceCaching && Settings->bUseFirstBounceRayGuiding);
				PermutationVector.Set<FLightmapPathTracingRGS::FUseIrradianceCaching>(Settings->bUseIrradianceCaching);
				PermutationVector.Set<FLightmapPathTracingRGS::FUseICBackfaceDetection>(Settings->bUseIrradianceCaching && Settings->bUseIrradianceCacheBackfaceDetection);
				RayGenShaderTable.Add(GlobalShaderMap->GetShader<FLightmapPathTracingRGS>(PermutationVector).GetRayTracingShader());
			}
			{
				RayGenShaderTable.Add(GlobalShaderMap->GetShader<FStationaryLightShadowTracingRGS>().GetRayTracingShader());

				FStaticShadowDepthMapTracingRGS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FStaticShadowDepthMapTracingRGS::FLightType>(0);
				RayGenShaderTable.Add(GlobalShaderMap->GetShader<FStaticShadowDepthMapTracingRGS>(PermutationVector).GetRayTracingShader());
				PermutationVector.Set<FStaticShadowDepthMapTracingRGS::FLightType>(1);
				RayGenShaderTable.Add(GlobalShaderMap->GetShader<FStaticShadowDepthMapTracingRGS>(PermutationVector).GetRayTracingShader());
				PermutationVector.Set<FStaticShadowDepthMapTracingRGS::FLightType>(2);
				RayGenShaderTable.Add(GlobalShaderMap->GetShader<FStaticShadowDepthMapTracingRGS>(PermutationVector).GetRayTracingShader());
			}
			{
				FVolumetricLightmapPathTracingRGS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FVolumetricLightmapPathTracingRGS::FUseIrradianceCaching>(Settings->bUseIrradianceCaching);
				RayGenShaderTable.Add(GlobalShaderMap->GetShader<FVolumetricLightmapPathTracingRGS>(PermutationVector).GetRayTracingShader());
			}
			PSOInitializer.SetRayGenShaderTable(RayGenShaderTable);

			EShaderPlatform ShaderPlatform = View.GetShaderPlatform();

			auto DefaultClosestHitShader = GetGPULightmassDefaultOpaqueHitShader(GlobalShaderMap);
			TArray<FRHIRayTracingShader*> RayTracingHitGroupLibrary;
			FShaderMapResource::GetRayTracingHitGroupLibrary(ShaderPlatform, RayTracingHitGroupLibrary, DefaultClosestHitShader);

			FRHIRayTracingShader* HiddenMaterialShader = GetGPULightmassDefaultHiddenHitShader(View.ShaderMap);
			RayTracingHitGroupLibrary.Add(HiddenMaterialShader);

			PSOInitializer.SetHitGroupTable(RayTracingHitGroupLibrary);

			FRHIRayTracingShader* MissTable[] = { GetGPULightmassDefaultMissShader(GlobalShaderMap) };
			PSOInitializer.SetMissShaderTable(MissTable);

			RayTracingPipelineState = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(GraphBuilder.RHICmdList, PSOInitializer);

			RayTracingSBT.ResetStaticAllocationLock();

			SBT = RayTracingSBT.AllocateTransientRHI(GraphBuilder.RHICmdList, ERayTracingShaderBindingMode::RTPSO, ERayTracingHitGroupIndexingMode::Allow, PSOInitializer.GetMaxLocalBindingDataSize());

			const int32 HiddenMaterialIndex = FindRayTracingHitGroupIndex(RayTracingPipelineState, HiddenMaterialShader, true);

			TUniquePtr<FRayTracingLocalShaderBindingWriter> BindingWriter = MakeUnique<FRayTracingLocalShaderBindingWriter>();

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SetRayTracingShaderBindings);

				if (LODIndex != INDEX_NONE)
				{
					for (const FRayTracingShaderBindingData& ShaderBindingData : CachedRayTracingScene->ShaderBindingsPerLOD[LODIndex])
					{
						const FRayTracingMeshCommand& MeshCommand = *ShaderBindingData.RayTracingMeshCommand;

						const int32 MaterialShaderIndex = MeshCommand.bDecal ? HiddenMaterialIndex : MeshCommand.MaterialShaderIndex;

						MeshCommand.SetRayTracingShaderBindingsForHitGroup(BindingWriter.Get(),
							View.ViewUniformBuffer,
PRAGMA_DISABLE_DEPRECATION_WARNINGS
							SceneUniforms.GetBufferRHI(GraphBuilder),
PRAGMA_ENABLE_DEPRECATION_WARNINGS
							nullptr,
							ShaderBindingData.SBTRecordIndex + RAY_TRACING_SHADER_SLOT_MATERIAL,
							ShaderBindingData.RayTracingGeometry,
							MeshCommand.GeometrySegmentIndex,
							MaterialShaderIndex,
							ERayTracingLocalShaderBindingType::Transient);

						MeshCommand.SetRayTracingShaderBindingsForHitGroup(BindingWriter.Get(),
							View.ViewUniformBuffer,
PRAGMA_DISABLE_DEPRECATION_WARNINGS
							SceneUniforms.GetBufferRHI(GraphBuilder),
PRAGMA_ENABLE_DEPRECATION_WARNINGS
							nullptr,
							ShaderBindingData.SBTRecordIndex + RAY_TRACING_SHADER_SLOT_SHADOW,
							ShaderBindingData.RayTracingGeometry,
							MeshCommand.GeometrySegmentIndex,
							MaterialShaderIndex,
							ERayTracingLocalShaderBindingType::Transient);
					}
				}

				for (const FRayTracingShaderBindingData& ShaderBindingData : DynamicRTShaderBindings)
				{
					const FRayTracingMeshCommand& MeshCommand = *ShaderBindingData.RayTracingMeshCommand;

					const int32 MaterialShaderIndex = MeshCommand.bDecal ? HiddenMaterialIndex : MeshCommand.MaterialShaderIndex;

					MeshCommand.SetRayTracingShaderBindingsForHitGroup(BindingWriter.Get(),
						View.ViewUniformBuffer,
PRAGMA_DISABLE_DEPRECATION_WARNINGS
						SceneUniforms.GetBufferRHI(GraphBuilder),
PRAGMA_ENABLE_DEPRECATION_WARNINGS
						nullptr,
						ShaderBindingData.SBTRecordIndex + RAY_TRACING_SHADER_SLOT_MATERIAL,
						ShaderBindingData.RayTracingGeometry,
						MeshCommand.GeometrySegmentIndex,
						MaterialShaderIndex,
						ERayTracingLocalShaderBindingType::Transient);

					MeshCommand.SetRayTracingShaderBindingsForHitGroup(BindingWriter.Get(),
						View.ViewUniformBuffer,
PRAGMA_DISABLE_DEPRECATION_WARNINGS
						SceneUniforms.GetBufferRHI(GraphBuilder),
PRAGMA_ENABLE_DEPRECATION_WARNINGS
						nullptr,
						ShaderBindingData.SBTRecordIndex + RAY_TRACING_SHADER_SLOT_SHADOW,
						ShaderBindingData.RayTracingGeometry,
						MeshCommand.GeometrySegmentIndex,
						MaterialShaderIndex,
						ERayTracingLocalShaderBindingType::Transient);
				}

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("GPULM::SetRayTracingBindings"),
					PassParameters,
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					[BindingWriter = MoveTemp(BindingWriter), RayTracingPipelineState = RayTracingPipelineState, SBT = SBT](FRHICommandListImmediate& RHICmdList) mutable
					{
						uint32 NumTotalBindings = 0;

						for (const FRayTracingLocalShaderBindingWriter::FChunk* Chunk = BindingWriter->GetFirstChunk(); Chunk; Chunk = Chunk->Next)
						{
							NumTotalBindings += Chunk->Num;
						}

						FConcurrentLinearBulkObjectAllocator Allocator;

						const uint32 MergedBindingsSize = sizeof(FRayTracingLocalShaderBindings) * NumTotalBindings;
						FRayTracingLocalShaderBindings* MergedBindings = static_cast<FRayTracingLocalShaderBindings*>(RHICmdList.Bypass()
							? Allocator.Malloc(MergedBindingsSize, alignof(FRayTracingLocalShaderBindings))
							: RHICmdList.Alloc(MergedBindingsSize, alignof(FRayTracingLocalShaderBindings)));

						uint32 MergedBindingIndex = 0;
						for (const FRayTracingLocalShaderBindingWriter::FChunk* Chunk = BindingWriter->GetFirstChunk(); Chunk; Chunk = Chunk->Next)
						{
							const uint32 Num = Chunk->Num;
							for (uint32_t i = 0; i < Num; ++i)
							{
								MergedBindings[MergedBindingIndex] = Chunk->Bindings[i];
								MergedBindingIndex++;
							}
						}

						const bool bCopyDataToInlineStorage = false; // Storage is already allocated from RHICmdList, no extra copy necessary
						RHICmdList.SetRayTracingHitGroups(
							SBT,
							RayTracingPipelineState,
							NumTotalBindings, MergedBindings,
							bCopyDataToInlineStorage);

						// there is only one miss shader, so it must be at index 0 by definition
						RHICmdList.SetRayTracingMissShader(SBT, 0, RayTracingPipelineState, 0 /* ShaderIndexInPipeline */, 0, nullptr, 0);
						RHICmdList.CommitShaderBindingTable(SBT);

						// Move the ray tracing binding container ownership to the command list, so that memory will be
						// released on the RHI thread timeline, after the commands that reference it are processed.
						// TUniquePtr<> auto destructs when exiting the lambda
						RHICmdList.EnqueueLambda([BindingWriter = MoveTemp(BindingWriter)](FRHICommandListImmediate&) {});
					});
			}
		}
	}
#endif

	return true;
}

void FSceneRenderState::DestroyRayTracingScene()
{
	ReferenceView.Reset();
	SBT.SafeRelease();
}

void FSceneRenderState::CalculateDistributionPrefixSumForAllLightmaps()
{
	uint32 PrefixSum = 0;

	for (FLightmapRenderState& Lightmap : LightmapRenderStates.Elements)
	{
		Lightmap.DistributionPrefixSum = PrefixSum;
		PrefixSum += Lightmap.GetNumTilesAcrossAllMipmapLevels();
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FLightmapGBufferPassParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLightmapGBufferParams, PassUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FInstanceCullingGlobalUniforms, InstanceCulling)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

bool ClampTexelPositionAndOffsetTile(FIntPoint& SrcVirtualTexelPosition, FIntPoint& SrcTileToLoad, FIntPoint SizeInTiles)
{
	bool bLoadingOutOfBounds = false;

	if (SrcVirtualTexelPosition.X < 0)
	{
		SrcTileToLoad.X -= 1;

		if (SrcTileToLoad.X < 0)
		{
			bLoadingOutOfBounds = true;
		}

		SrcVirtualTexelPosition.X += GPreviewLightmapVirtualTileSize;
	}
	else if (SrcVirtualTexelPosition.X >= GPreviewLightmapVirtualTileSize)
	{
		SrcTileToLoad.X += 1;

		if (SrcTileToLoad.X >= SizeInTiles.X)
		{
			bLoadingOutOfBounds = true;
		}

		SrcVirtualTexelPosition.X -= GPreviewLightmapVirtualTileSize;
	}

	if (SrcVirtualTexelPosition.Y < 0)
	{
		SrcTileToLoad.Y -= 1;

		if (SrcTileToLoad.Y < 0)
		{
			bLoadingOutOfBounds = true;
		}

		SrcVirtualTexelPosition.Y += GPreviewLightmapVirtualTileSize;
	}
	else if (SrcVirtualTexelPosition.Y >= GPreviewLightmapVirtualTileSize)
	{
		SrcTileToLoad.Y += 1;

		if (SrcTileToLoad.Y >= SizeInTiles.Y)
		{
			bLoadingOutOfBounds = true;
		}

		SrcVirtualTexelPosition.Y -= GPreviewLightmapVirtualTileSize;
	}

	return bLoadingOutOfBounds;
}

void FLightmapRenderer::RenderMeshBatchesIntoGBuffer(
	FRHICommandList& RHICmdList,
	const FSceneView* View,
	int32 GPUScenePrimitiveId,
	TArray<FMeshBatch> MeshBatches, // Copy mesh batches over so we don't need to worry about their lifetime
	FVector4f VirtualTexturePhysicalTileCoordinateScaleAndBias,
	int32 RenderPassIndex,
	FIntPoint ScratchTilePoolOffset)
{
	TArray<FMeshBatch> MeshBatchesToDrawImmediately;
	TArray<FMeshBatch> MeshBatchesNeedingInstanceOffsetUpdates;

	for (auto& MeshBatch : MeshBatches)
	{
		FMeshBatchElement& Element = MeshBatch.Elements[0];
		Element.DynamicPrimitiveIndex = GPUScenePrimitiveId;
		if (Element.UserIndex == INDEX_NONE)
		{
			MeshBatchesToDrawImmediately.Add(MeshBatch);
		}
		else
		{
			MeshBatchesNeedingInstanceOffsetUpdates.Add(MeshBatch);
		}
	}

	DrawDynamicMeshPass(
		*View, RHICmdList,
		[
			View,
			MeshBatchesToDrawImmediately,
			VirtualTexturePhysicalTileCoordinateScaleAndBias,
			RenderPassIndex,
			ScratchTilePoolOffset
		](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
	{
		FLightmapGBufferMeshProcessor MeshProcessor(nullptr, View, DynamicMeshPassContext, VirtualTexturePhysicalTileCoordinateScaleAndBias, RenderPassIndex, ScratchTilePoolOffset);

		for (auto& MeshBatch : MeshBatchesToDrawImmediately)
		{
			MeshProcessor.AddMeshBatch(MeshBatch, ~0ull, nullptr);
		}
	});

	for (auto& MeshBatch : MeshBatchesNeedingInstanceOffsetUpdates)
	{
		FDynamicMeshDrawCommandStorage DynamicMeshDrawCommandStorage;
		FMeshCommandOneFrameArray VisibleMeshDrawCommands;
		FGraphicsMinimalPipelineStateSet GraphicsMinimalPipelineStateSet;
		bool NeedsShaderInitialisation;

		FDynamicPassMeshDrawListContext DynamicMeshPassContext(DynamicMeshDrawCommandStorage, VisibleMeshDrawCommands, GraphicsMinimalPipelineStateSet, NeedsShaderInitialisation);

		FLightmapGBufferMeshProcessor MeshProcessor(nullptr, View, &DynamicMeshPassContext, VirtualTexturePhysicalTileCoordinateScaleAndBias, RenderPassIndex, ScratchTilePoolOffset);

		MeshProcessor.AddMeshBatch(MeshBatch, ~0ull, nullptr);

		const uint32 InstanceFactor = 1;
		FRHIBuffer* PrimitiveIdVertexBuffer = nullptr;
		const bool bDynamicInstancing = false;
		const uint32 PrimitiveIdBufferStride = FInstanceCullingContext::GetInstanceIdBufferStride(View->GetShaderPlatform());

		for (FVisibleMeshDrawCommand& Cmd : VisibleMeshDrawCommands)
		{
			Cmd.PrimitiveIdInfo.DrawPrimitiveId = Scene->CachedRayTracingScene->InstanceDataOriginalOffsets[MeshBatch.Elements[0].DynamicPrimitiveIndex] + MeshBatch.Elements[0].UserIndex;
		}

		SortAndMergeDynamicPassMeshDrawCommands(*View, RHICmdList, VisibleMeshDrawCommands, DynamicMeshDrawCommandStorage, PrimitiveIdVertexBuffer, InstanceFactor, nullptr);

		FMeshDrawCommandSceneArgs SceneArgs;
		SceneArgs.PrimitiveIdsBuffer = PrimitiveIdVertexBuffer;
		SubmitMeshDrawCommands(VisibleMeshDrawCommands, GraphicsMinimalPipelineStateSet, SceneArgs, PrimitiveIdBufferStride, bDynamicInstancing, InstanceFactor, RHICmdList);
	}

	GPrimitiveIdVertexBufferPool.DiscardAll();
}

void ClearScratchTilePoolForMultipleTiles(
	FRDGBuilder& GraphBuilder,
	const TResourceArray<FIntPoint>& TilePositionsToClear,
	const TStaticArray<FRDGTextureUAVRef, 3>& ScratchTilePoolLayerUAVs,
	const FGlobalShaderMap* GlobalShaderMap)
{
	for (int ScratchLayerIndex = 0; ScratchLayerIndex < ScratchTilePoolLayerUAVs.Num(); ScratchLayerIndex++)
	{
		RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

		FRDGBufferDesc TilePositionsBufferDesc = FRDGBufferDesc::CreateBufferDesc(TilePositionsToClear.GetTypeSize(), TilePositionsToClear.Num());
		const FRDGBufferRef TilePositionsBuffer = CreateVertexBuffer(GraphBuilder, TEXT("TilePositionsBufferForClear"), TilePositionsBufferDesc, TilePositionsToClear.GetData(), TilePositionsToClear.GetResourceDataSize());
		const FRDGBufferSRVRef TilePositionsBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(TilePositionsBuffer, PF_R32G32_UINT));
		FMultiTileClearCS::FParameters* Parameters = GraphBuilder.AllocParameters<FMultiTileClearCS::FParameters>();
		Parameters->NumTiles = TilePositionsToClear.Num();
		Parameters->TileSize = GPreviewLightmapPhysicalTileSize;
		Parameters->TilePositions = TilePositionsBufferSRV;
		Parameters->TilePool = ScratchTilePoolLayerUAVs[ScratchLayerIndex];

		TShaderMapRef<FMultiTileClearCS> ComputeShader(GlobalShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("MultiTileClear"),
			ComputeShader,
			Parameters,
			FComputeShaderUtils::GetGroupCount(FIntPoint(GPreviewLightmapPhysicalTileSize * TilePositionsToClear.Num(), GPreviewLightmapPhysicalTileSize), FComputeShaderUtils::kGolden2DGroupSize));
	}
}
	
void FLightmapRenderer::Finalize(FRDGBuilder& GraphBuilder)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLightmapRenderer::Finalize);

	bool bIsCompilingShaders = GShaderCompilingManager && GShaderCompilingManager->IsCompiling();

	if (bIsCompilingShaders)
	{
		PendingTileRequests.Empty();
		return;
	}
	
	if (PendingTileRequests.Num() == 0)
	{
		return;
	}

	FRHICommandListImmediate& RHICmdList = GraphBuilder.RHICmdList;

	const auto HoldReference = [](FRDGBuilder& GraphBuilder, const FShaderResourceViewRHIRef& View) -> const FShaderResourceViewRHIRef&
	{
		return *GraphBuilder.AllocObject<FShaderResourceViewRHIRef>(View);
	};

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(Scene->FeatureLevel);

	// Upload & copy converged tiles directly
	{
		TArray<FLightmapTileRequest> TileUploadRequests = PendingTileRequests.FilterByPredicate(
			[CurrentRevision = CurrentRevision, bDenoiseDuringInteractiveBake = bDenoiseDuringInteractiveBake](const FLightmapTileRequest& Tile)
		{ 
			return Tile.RenderState->DoesTileHaveValidCPUData(Tile.VirtualCoordinates, CurrentRevision) || (bDenoiseDuringInteractiveBake && Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).OngoingReadbackRevision == CurrentRevision && Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).bCanBeDenoised);
		});

		if (TileUploadRequests.Num() > 0)
		{
			SCOPED_DRAW_EVENTF(RHICmdList, GPULightmassUploadConvergedTiles, TEXT("GPULightmass UploadConvergedTiles %d tiles"), TileUploadRequests.Num());

			int32 NewSize = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(TileUploadRequests.Num())));
			if (!UploadTilePoolGPU.IsValid() || UploadTilePoolGPU->SizeInTiles.X < NewSize)
			{
				UploadTilePoolGPU = MakeUnique<FLightmapTilePoolGPU>(GraphBuilder.RHICmdList, 4, FIntPoint(NewSize, NewSize), FIntPoint(GPreviewLightmapPhysicalTileSize, GPreviewLightmapPhysicalTileSize));
				UE_LOGF(LogGPULightmass, Log, "Resizing GPULightmass upload tile pool to (%d, %d) %dx%d", NewSize, NewSize, NewSize * GPreviewLightmapPhysicalTileSize, NewSize * GPreviewLightmapPhysicalTileSize);
			}

			{
				uint32 DstRowPitch;
				FLinearColor* Texture[4];
				check(UE_ARRAY_COUNT(Texture) == UploadTilePoolGPU->PooledRenderTargets.Num());
				for (int32 LayerIndex = 0; LayerIndex < UE_ARRAY_COUNT(Texture); LayerIndex++)
				{
					Texture[LayerIndex] = (FLinearColor*)RHICmdList.LockTexture2D(UploadTilePoolGPU->PooledRenderTargets[LayerIndex]->GetRHI(), 0, RLM_WriteOnly, DstRowPitch, false);
				}

				TSet<FVirtualTile> TilesToDecompress;

				FTileDataLayer::Evict();

				for (FLightmapTileRequest& Tile : TileUploadRequests)
				{
					FIntPoint Positions[] = {
						FIntPoint(0, 0),
						FIntPoint(0, GPreviewLightmapPhysicalTileSize - 1),
						FIntPoint(GPreviewLightmapPhysicalTileSize - 1, 0),
						FIntPoint(GPreviewLightmapPhysicalTileSize - 1, GPreviewLightmapPhysicalTileSize - 1),
						FIntPoint(GPreviewLightmapPhysicalTileSize / 2, GPreviewLightmapPhysicalTileSize / 2),
						FIntPoint(GPreviewLightmapPhysicalTileSize / 2, 0),
						FIntPoint(0, GPreviewLightmapPhysicalTileSize / 2),
						FIntPoint(GPreviewLightmapPhysicalTileSize / 2, GPreviewLightmapPhysicalTileSize - 1),
						FIntPoint(GPreviewLightmapPhysicalTileSize - 1, GPreviewLightmapPhysicalTileSize / 2)
					};

					for (FIntPoint Position : Positions)
					{
						FIntPoint SrcVirtualTexelPosition = Position - FIntPoint(GPreviewLightmapTileBorderSize, GPreviewLightmapTileBorderSize);
						FIntPoint SrcTileToLoad = Tile.VirtualCoordinates.Position;

						bool bLoadingOutOfBounds = ClampTexelPositionAndOffsetTile(SrcVirtualTexelPosition, SrcTileToLoad, Tile.RenderState->GetPaddedSizeInTilesAtMipLevel(Tile.VirtualCoordinates.MipLevel));

						FTileVirtualCoordinates SrcTileCoords(SrcTileToLoad, Tile.VirtualCoordinates.MipLevel);

						if (!bLoadingOutOfBounds)
						{
							if (!Tile.RenderState->DoesTileHaveValidCPUData(SrcTileCoords, CurrentRevision))
							{
								if (!bDenoiseDuringInteractiveBake)
								{
									bLoadingOutOfBounds = true;
								}
								else if (Tile.RenderState->RetrieveTileState(SrcTileCoords).OngoingReadbackRevision != CurrentRevision || !Tile.RenderState->RetrieveTileState(SrcTileCoords).bCanBeDenoised)
								{
									bLoadingOutOfBounds = true;
								}
							}
						}

						if (!bLoadingOutOfBounds)
						{
							for (int32 LayerIndex = 0; LayerIndex < UE_ARRAY_COUNT(Texture); LayerIndex++)
							{
								Tile.RenderState->TileStorage[SrcTileCoords].CPUTextureData[LayerIndex]->Decompress();
							}
						}
					}
				}

				ParallelFor(TileUploadRequests.Num(), [&](int32 TileIndex)
				{
					FIntPoint SrcTilePosition(TileUploadRequests[TileIndex].VirtualCoordinates.Position);
					FIntPoint DstTilePosition(TileIndex % UploadTilePoolGPU->SizeInTiles.X, TileIndex / UploadTilePoolGPU->SizeInTiles.X);

					const int32 SrcRowPitchInPixels = TileUploadRequests[TileIndex].RenderState->GetPaddedSizeAtMipLevel(TileUploadRequests[TileIndex].VirtualCoordinates.MipLevel).X;
					const int32 DstRowPitchInPixels = DstRowPitch / sizeof(FLinearColor);

					for (int32 Y = 0; Y < GPreviewLightmapPhysicalTileSize; Y++)
					{
						for (int32 X = 0; X < GPreviewLightmapPhysicalTileSize; X++)
						{
							bool bLoadingOutOfBounds = false;

							FIntPoint SrcVirtualTexelPosition = FIntPoint(X, Y) - FIntPoint(GPreviewLightmapTileBorderSize, GPreviewLightmapTileBorderSize);
							FIntPoint SrcTileToLoad = SrcTilePosition;

							bLoadingOutOfBounds = ClampTexelPositionAndOffsetTile(SrcVirtualTexelPosition, SrcTileToLoad, TileUploadRequests[TileIndex].RenderState->GetPaddedSizeInTilesAtMipLevel(TileUploadRequests[TileIndex].VirtualCoordinates.MipLevel));

							int32 SrcLinearIndex = SrcVirtualTexelPosition.Y * GPreviewLightmapVirtualTileSize + SrcVirtualTexelPosition.X;
							FIntPoint DstPixelPosition = DstTilePosition * GPreviewLightmapPhysicalTileSize + FIntPoint(X, Y);
							int32 DstLinearIndex = DstPixelPosition.Y * DstRowPitchInPixels + DstPixelPosition.X;

							FTileVirtualCoordinates SrcTileCoords(SrcTileToLoad, TileUploadRequests[TileIndex].VirtualCoordinates.MipLevel);

							if (!bLoadingOutOfBounds)
							{
								if (!TileUploadRequests[TileIndex].RenderState->DoesTileHaveValidCPUData(SrcTileCoords, CurrentRevision))
								{
									if (!bDenoiseDuringInteractiveBake)
									{
										bLoadingOutOfBounds = true;
									}
									else if (TileUploadRequests[TileIndex].RenderState->RetrieveTileState(SrcTileCoords).OngoingReadbackRevision != CurrentRevision || !TileUploadRequests[TileIndex].RenderState->RetrieveTileState(SrcTileCoords).bCanBeDenoised)
									{
										bLoadingOutOfBounds = true;
									}
								}
							}

							for (int32 LayerIndex = 0; LayerIndex < UE_ARRAY_COUNT(Texture); LayerIndex++)
							{
								Texture[LayerIndex][DstLinearIndex] = !bLoadingOutOfBounds ? TileUploadRequests[TileIndex].RenderState->TileStorage[SrcTileCoords].CPUTextureData[LayerIndex]->Data[SrcLinearIndex] : FLinearColor(0, 0, 0, 0);
							}
						}
					}
				});

				for (int32 LayerIndex = 0; LayerIndex < UE_ARRAY_COUNT(Texture); LayerIndex++)
				{
					RHICmdList.UnlockTexture2D(UploadTilePoolGPU->PooledRenderTargets[LayerIndex]->GetRHI(), 0, false);
				}
			}

			FGPUBatchedTileRequests GPUBatchedTileRequests;
			GPUBatchedTileRequests.BuildFromTileDescs(TileUploadRequests, LightmapTilePoolGPU, *ScratchTilePoolGPU);
			GPUBatchedTileRequests.Commit(0);

			IPooledRenderTarget* OutputRenderTargets[4] = { nullptr, nullptr, nullptr, nullptr };

			for (auto& Tile : TileUploadRequests)
			{
				for (int32 RenderTargetIndex = 0; RenderTargetIndex < UE_ARRAY_COUNT(OutputRenderTargets); RenderTargetIndex++)
				{
					if (Tile.OutputRenderTargets[RenderTargetIndex] != nullptr)
					{
						if (OutputRenderTargets[RenderTargetIndex] == nullptr)
						{
							OutputRenderTargets[RenderTargetIndex] = Tile.OutputRenderTargets[RenderTargetIndex];
						}
						else
						{
							ensure(OutputRenderTargets[RenderTargetIndex] == Tile.OutputRenderTargets[RenderTargetIndex]);
						}
					}
				}
			}

			FIntPoint DispatchResolution;
			DispatchResolution.X = GPreviewLightmapPhysicalTileSize * GPUBatchedTileRequests.BatchedTilesDesc.Num();
			DispatchResolution.Y = GPreviewLightmapPhysicalTileSize;

			FString StagingTextureNames[4] = { TEXT("StagingHQLayer0"), TEXT("StagingHQLayer1"), TEXT("StagingShadowMask"), TEXT("StagingSkyOcclusion") };
			
			for (int32 RenderTargetIndex = 0; RenderTargetIndex < UE_ARRAY_COUNT(OutputRenderTargets); RenderTargetIndex++)
			{
				if (OutputRenderTargets[RenderTargetIndex] != nullptr)
				{
					FRDGTextureRef StagingTexture = GraphBuilder.RegisterExternalTexture(UploadTilePoolGPU->PooledRenderTargets[RenderTargetIndex], *StagingTextureNames[RenderTargetIndex]);

					FBufferRHIRef SrcTilePositionsBuffer;
					FShaderResourceViewRHIRef SrcTilePositionsSRV;
					FBufferRHIRef DstTilePositionsBuffer;
					FShaderResourceViewRHIRef DstTilePositionsSRV;

					TResourceArray<FIntPoint> SrcTilePositions;
					TResourceArray<FIntPoint> DstTilePositions;
					
					for (int32 TileIndex = 0; TileIndex < TileUploadRequests.Num(); TileIndex++)
					{
						SrcTilePositions.Add(FIntPoint(TileIndex % UploadTilePoolGPU->SizeInTiles.X, TileIndex / UploadTilePoolGPU->SizeInTiles.X) * GPreviewLightmapPhysicalTileSize);
						DstTilePositions.Add(TileUploadRequests[TileIndex].OutputPhysicalCoordinates[RenderTargetIndex] * GPreviewLightmapPhysicalTileSize);
					}

					{
						const FRHIBufferCreateDesc CreateDesc =
							FRHIBufferCreateDesc::CreateStructured<FIntPoint>(TEXT("SrcTilePositionsBuffer"), SrcTilePositions.Num())
							.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::ShaderResource)
							.SetGPUMask(FRHIGPUMask::GPU0())
							.SetInitActionResourceArray(&SrcTilePositions)
							.DetermineInitialState();

						SrcTilePositionsBuffer = RHICmdList.CreateBuffer(CreateDesc);
						SrcTilePositionsSRV = RHICmdList.CreateShaderResourceView(SrcTilePositionsBuffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(SrcTilePositionsBuffer));
					}

					{
						const FRHIBufferCreateDesc CreateDesc =
							FRHIBufferCreateDesc::CreateStructured<FIntPoint>(TEXT("DstTilePositionsBuffer"), DstTilePositions.Num())
							.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::ShaderResource)
							.SetGPUMask(FRHIGPUMask::GPU0())
							.SetInitActionResourceArray(&DstTilePositions)
							.DetermineInitialState();

						DstTilePositionsBuffer = RHICmdList.CreateBuffer(CreateDesc);
						DstTilePositionsSRV = RHICmdList.CreateShaderResourceView(DstTilePositionsBuffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(DstTilePositionsBuffer));
					}

					{
						FString& DynamicDebugName = *GraphBuilder.AllocObject<FString>();
						DynamicDebugName = TEXT("GPULightmassRenderTargetTileAtlas_") + StagingTextureNames[RenderTargetIndex];
						FRDGTextureRef RenderTargetTileAtlas = GraphBuilder.RegisterExternalTexture(OutputRenderTargets[RenderTargetIndex], *DynamicDebugName);

						FUploadConvergedLightmapTilesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FUploadConvergedLightmapTilesCS::FParameters>();

						PassParameters->NumBatchedTiles = GPUBatchedTileRequests.BatchedTilesDesc.Num();
						PassParameters->SrcTexture = GraphBuilder.CreateUAV(StagingTexture);
						PassParameters->DstTexture = GraphBuilder.CreateUAV(RenderTargetTileAtlas);
						PassParameters->SrcTilePositions = HoldReference(GraphBuilder, SrcTilePositionsSRV);
						PassParameters->DstTilePositions = HoldReference(GraphBuilder, DstTilePositionsSRV);

						TShaderMapRef<FUploadConvergedLightmapTilesCS> ComputeShader(GlobalShaderMap);
						FComputeShaderUtils::AddPass(
							GraphBuilder,
							RDG_EVENT_NAME("UploadConvergedLightmapTiles"),
							ComputeShader,
							PassParameters,
							FComputeShaderUtils::GetGroupCount(DispatchResolution, FComputeShaderUtils::kGolden2DGroupSize));
					}
				}
			}
		}

		// Drop these converged requests, critical so that we won't perform readback repeatedly
		PendingTileRequests = PendingTileRequests.FilterByPredicate([CurrentRevision = CurrentRevision](const FLightmapTileRequest& Tile) { return !Tile.RenderState->DoesTileHaveValidCPUData(Tile.VirtualCoordinates, CurrentRevision); });
	}

	PendingTileRequests = PendingTileRequests.FilterByPredicate([CurrentRevision = CurrentRevision](const FLightmapTileRequest& Tile) { return Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).OngoingReadbackRevision != CurrentRevision; });

	if (!bInsideBackgroundTick && !bOnlyBakeWhatYouSee)
	{
		TArray<FLightmapTileRequest> RoundRobinFilteredRequests;
		if (PendingTileRequests.Num() > (128 * (int32)GNumExplicitGPUsForRendering))
		{
			int32 RoundRobinDivisor = PendingTileRequests.Num() / (128 * (int32)GNumExplicitGPUsForRendering);

			for (int32 Index = 0; Index < PendingTileRequests.Num(); Index++)
			{
				if (Index % RoundRobinDivisor == FrameNumber % RoundRobinDivisor)
				{
					RoundRobinFilteredRequests.Add(PendingTileRequests[Index]);
				}
			}

			PendingTileRequests = RoundRobinFilteredRequests;
		}
	}

	if (!bInsideBackgroundTick && bOnlyBakeWhatYouSee)
	{
		TArray<FLightmapTileRequest> ScreenOutputTiles = PendingTileRequests.FilterByPredicate([](const FLightmapTileRequest& Tile) { return Tile.IsScreenOutputTile(); });
		if (ScreenOutputTiles.Num() > 0)
		{
			TilesVisibleLastFewFrames[(FrameNumber - 1 + TilesVisibleLastFewFrames.Num()) % TilesVisibleLastFewFrames.Num()] = ScreenOutputTiles;
			
			if (bIsRecordingTileRequests)
			{
				for (FLightmapTileRequest& Tile : ScreenOutputTiles)
				{
					RecordedTileRequests.AddUnique(Tile);
				}
			}
		}

		if (!bIsRecordingTileRequests && RecordedTileRequests.Num() > 0)
		{
			PendingTileRequests = PendingTileRequests.FilterByPredicate([&RecordedTileRequests = RecordedTileRequests](const FLightmapTileRequest& Tile) { return RecordedTileRequests.Find(Tile) != INDEX_NONE; });
		}
	}

	PendingTileRequests.Sort([](const FLightmapTileRequest& A, const FLightmapTileRequest& B) { 
		return
			A.RenderState.GetElementId() < B.RenderState.GetElementId() || 
			A.RenderState.GetElementId() == B.RenderState.GetElementId() && A.VirtualCoordinates.GetVirtualAddress() < B.VirtualCoordinates.GetVirtualAddress();
	});

	// Alloc for tiles that need work
	{
		// Find which tiles are already resident
		TArray<FVirtualTile> TilesToQuery;
		for (auto& Tile : PendingTileRequests)
		{
			checkSlow(TilesToQuery.Find(FVirtualTile{ Tile.RenderState, Tile.VirtualCoordinates.MipLevel, (int32)Tile.VirtualCoordinates.GetVirtualAddress() }) == INDEX_NONE);
			TilesToQuery.Add(FVirtualTile{ Tile.RenderState, Tile.VirtualCoordinates.MipLevel, (int32)Tile.VirtualCoordinates.GetVirtualAddress() });
		}
		TArray<uint32> TileAddressIfResident;
		LightmapTilePoolGPU.QueryResidency(TilesToQuery, TileAddressIfResident);

		// We lock tiles that are resident and requested for current frame so that they won't be evicted by the following AllocAndLock
		TArray<FVirtualTile> NonResidentTilesToAllocate;
		TArray<int32> NonResidentTileRequestIndices;
		TArray<int32> ResidentTilesToLock;
		for (int32 TileIndex = 0; TileIndex < TileAddressIfResident.Num(); TileIndex++)
		{
			if (TileAddressIfResident[TileIndex] == ~0u)
			{
				NonResidentTilesToAllocate.Add(TilesToQuery[TileIndex]);
				NonResidentTileRequestIndices.Add(TileIndex);
			}
			else
			{
				ResidentTilesToLock.Add(TileAddressIfResident[TileIndex]);
				PendingTileRequests[TileIndex].TileAddressInWorkingSet = TileAddressIfResident[TileIndex];
			}
		}

		// All non-resident tiles need to be invalidated, whether they are successfully allocated later or not
		for (const FVirtualTile& Tile : NonResidentTilesToAllocate)
		{
			if (Tile.RenderState.IsValid())
			{
				Tile.RenderState->RetrieveTileState(FTileVirtualCoordinates(Tile.VirtualAddress, Tile.MipLevel)).Revision = -1;
				Tile.RenderState->RetrieveTileState(FTileVirtualCoordinates(Tile.VirtualAddress, Tile.MipLevel)).RenderPassIndex = 0;
			}
		}

		LightmapTilePoolGPU.Lock(ResidentTilesToLock);

		{
			TArray<int32> SuccessfullyAllocatedTiles;
			LightmapTilePoolGPU.AllocAndLock(NonResidentTilesToAllocate.Num(), SuccessfullyAllocatedTiles);

			// Map successfully allocated tiles, potentially evict some resident tiles to the lower cache tiers
			TArray<FVirtualTile> TilesToMap;
			for (int32 TileIndex = 0; TileIndex < SuccessfullyAllocatedTiles.Num(); TileIndex++)
			{
				TilesToMap.Add(NonResidentTilesToAllocate[TileIndex]);

				auto& Tile = PendingTileRequests[NonResidentTileRequestIndices[TileIndex]];
				Tile.TileAddressInWorkingSet = SuccessfullyAllocatedTiles[TileIndex];
			}

			// Till this point there might still be tiles with ~0u (which have failed allocation), they will be dropped later

			TArray<FVirtualTile> TilesEvicted;
			LightmapTilePoolGPU.Map(TilesToMap, SuccessfullyAllocatedTiles, TilesEvicted);

			// Invalidate evicted tiles' state as they can't be read back anymore
			// TODO: save to CPU and reload when appropriate
			for (const FVirtualTile& Tile : TilesEvicted)
			{
				if (Tile.RenderState.IsValid())
				{
					Tile.RenderState->RetrieveTileState(FTileVirtualCoordinates(Tile.VirtualAddress, Tile.MipLevel)).Revision = -1;
					Tile.RenderState->RetrieveTileState(FTileVirtualCoordinates(Tile.VirtualAddress, Tile.MipLevel)).RenderPassIndex = 0;
				}
			}

			LightmapTilePoolGPU.MakeAvailable(SuccessfullyAllocatedTiles, FrameNumber);
		}

		LightmapTilePoolGPU.MakeAvailable(ResidentTilesToLock, FrameNumber);

		{
			bool bScratchAllocationSucceeded = false;

			while (!bScratchAllocationSucceeded)
			{
				if (ScratchTilePoolGPU.IsValid())
				{
					TArray<int32> SuccessfullyAllocatedTiles;
					ScratchTilePoolGPU->AllocAndLock(TilesToQuery.Num(), SuccessfullyAllocatedTiles);

					if (SuccessfullyAllocatedTiles.Num() == TilesToQuery.Num())
					{
						for (int32 TileIndex = 0; TileIndex < SuccessfullyAllocatedTiles.Num(); TileIndex++)
						{
							auto& Tile = PendingTileRequests[TileIndex];
							Tile.TileAddressInScratch = SuccessfullyAllocatedTiles[TileIndex];
						}

						bScratchAllocationSucceeded = true;
					}

					ScratchTilePoolGPU->MakeAvailable(SuccessfullyAllocatedTiles, FrameNumber);
				}

				if (!bScratchAllocationSucceeded)
				{
					if (ScratchTilePoolGPU.IsValid() && ScratchTilePoolGPU->SizeInTiles.X >= 64)
					{
						// If we have reached our limit, don't retry and drop the requests.
						// Till this point there might still be tiles with ~0u (which have failed allocation), they will be dropped later
						break;
					}

					int32 NewSize = FMath::Min(FMath::CeilToInt(FMath::Sqrt(static_cast<float>(TilesToQuery.Num()))), 64);
					ScratchTilePoolGPU = MakeUnique<FLightmapTilePoolGPU>(GraphBuilder.RHICmdList, 3, FIntPoint(NewSize, NewSize), FIntPoint(GPreviewLightmapPhysicalTileSize, GPreviewLightmapPhysicalTileSize));
					UE_LOGF(LogGPULightmass, Log, "Resizing GPULightmass scratch tile pool to (%d, %d) %dx%d", NewSize, NewSize, NewSize * GPreviewLightmapPhysicalTileSize, NewSize * GPreviewLightmapPhysicalTileSize);
				}
			}
		}

		// Drop requests that have failed allocation
		PendingTileRequests = PendingTileRequests.FilterByPredicate([](const FLightmapTileRequest& TileRequest) { return TileRequest.TileAddressInWorkingSet != ~0u && TileRequest.TileAddressInScratch != ~0u; });
	}

	// If all tiles have failed allocation (unlikely but possible), return immediately
	if (PendingTileRequests.Num() == 0)
	{
		return;
	}

	const bool bIsViewportNonRealtime = !FGPULightmassModule::IsRealtimeOn();

	int32 MostCommonLODIndex = 0;

	int32 NumRequestsPerLOD[MAX_STATIC_MESH_LODS] = { 0 };

	for (const FLightmapTileRequest& Tile : PendingTileRequests)
	{
		NumRequestsPerLOD[Tile.RenderState->GeometryInstanceRef.LODIndex]++;
	}

	if (bInsideBackgroundTick && bIsViewportNonRealtime)
	{
		// Pick the most common LOD level
		for (int32 Index = 0; Index < MAX_STATIC_MESH_LODS; Index++)
		{
			if (NumRequestsPerLOD[Index] > NumRequestsPerLOD[MostCommonLODIndex])
			{
				MostCommonLODIndex = Index;
			}
		}
	}
	else
	{
		// Alternate between LOD levels when in realtime preview
		TArray<int32> NonZeroLODIndices;

		for (int32 Index = 0; Index < MAX_STATIC_MESH_LODS; Index++)
		{
			if (NumRequestsPerLOD[Index] > 0)
			{
				NonZeroLODIndices.Add(Index);
			}
		}

		check(NonZeroLODIndices.Num() > 0);

		MostCommonLODIndex = NonZeroLODIndices[FrameNumber % NonZeroLODIndices.Num()];
	}

	RectLightAtlas::UpdateAtlasTexture(GraphBuilder, Scene->FeatureLevel);
	IESAtlas::UpdateAtlasTexture(GraphBuilder, GetFeatureLevelShaderPlatform(Scene->FeatureLevel));

	FSceneUniformBuffer SceneUniforms{};
	if (!Scene->SetupRayTracingScene(GraphBuilder, SceneUniforms, MostCommonLODIndex))
	{
		return;
	}

	TStaticArray<FRDGTextureUAVRef, 3> ScratchTilePoolLayerUAVs;

	for (int32 Index = 0; Index < ScratchTilePoolLayerUAVs.Num(); ++Index)
	{
		ScratchTilePoolLayerUAVs[Index] = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(ScratchTilePoolGPU->PooledRenderTargets[Index]));
	}

	TRDGUniformBufferRef<FLightmapGBufferParams> PassUniformBuffer = nullptr;

	{
		auto* LightmapGBufferParameters = GraphBuilder.AllocParameters<FLightmapGBufferParams>();
		LightmapGBufferParameters->ScratchTilePoolLayer0 = ScratchTilePoolLayerUAVs[0];
		LightmapGBufferParameters->ScratchTilePoolLayer1 = ScratchTilePoolLayerUAVs[1];
		LightmapGBufferParameters->ScratchTilePoolLayer2 = ScratchTilePoolLayerUAVs[2];
		PassUniformBuffer = GraphBuilder.CreateUniformBuffer(LightmapGBufferParameters);
	}

	TRDGUniformBufferRef<FInstanceCullingGlobalUniforms> InstanceCullingUniformBuffer = nullptr;
	{
		FRDGBufferRef InstanceIdsIdentityBuffer = nullptr;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(InstanceIdsIdentityBuffer);

			TArray<uint32, SceneRenderingAllocator> InstanceIdsIdentity;
			for (uint32 Index = 0U; Index < FMath::Max(1U, uint32(Scene->CachedRayTracingScene->GPUSceneInstanceDataSOAStride)); ++Index)
			{
				InstanceIdsIdentity.Add(Index);
			}

			InstanceIdsIdentityBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("InstanceIdsIdentityBuffer"), InstanceIdsIdentity, ERDGInitialDataFlags::NoCopy);
		}
		FInstanceCullingGlobalUniforms* InstanceCullingUniforms = GraphBuilder.AllocParameters<FInstanceCullingGlobalUniforms>();

		InstanceCullingUniforms->InstanceIdsBuffer = GraphBuilder.CreateSRV(InstanceIdsIdentityBuffer);
		// Note redundant, but must have non-null reference even if not used it would seem
		InstanceCullingUniforms->PageInfoBuffer = GraphBuilder.CreateSRV(InstanceIdsIdentityBuffer);
		InstanceCullingUniforms->BufferCapacity = 0;
		InstanceCullingUniformBuffer = GraphBuilder.CreateUniformBuffer(InstanceCullingUniforms);
	}

	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::GPU0());

	IPooledRenderTarget* OutputRenderTargets[5] = { nullptr, nullptr, nullptr, nullptr, nullptr };
	static_assert(UE_ARRAY_COUNT(OutputRenderTargets) <= UE_ARRAY_COUNT(FLightmapTileRequest::OutputRenderTargets));
	
	for (auto& Tile : PendingTileRequests)
	{
		for (int32 RenderTargetIndex = 0; RenderTargetIndex < UE_ARRAY_COUNT(OutputRenderTargets); RenderTargetIndex++)
		{
			if (Tile.OutputRenderTargets[RenderTargetIndex] != nullptr)
			{
				if (OutputRenderTargets[RenderTargetIndex] == nullptr)
				{
					OutputRenderTargets[RenderTargetIndex] = Tile.OutputRenderTargets[RenderTargetIndex];
				}
				else
				{
					ensure(OutputRenderTargets[RenderTargetIndex] == Tile.OutputRenderTargets[RenderTargetIndex]);
				}
			}
		}
	}

	// Perform deferred invalidation
	{
		// Clear working set pools
		for (int PoolLayerIndex = 0; PoolLayerIndex < LightmapTilePoolGPU.PooledRenderTargets.Num(); PoolLayerIndex++)
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

			TArray<FVector4f, SceneRenderingAllocator> ViewportsToClear;

			for (auto& Tile : PendingTileRequests)
			{
				if (Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).Revision != CurrentRevision)
				{
					ViewportsToClear.Emplace(
						LightmapTilePoolGPU.GetPositionFromLinearAddress(Tile.TileAddressInWorkingSet).X * LightmapTilePoolGPU.LayerFormatAndTileSize[PoolLayerIndex].TileSize.X,
						LightmapTilePoolGPU.GetPositionFromLinearAddress(Tile.TileAddressInWorkingSet).Y * LightmapTilePoolGPU.LayerFormatAndTileSize[PoolLayerIndex].TileSize.Y,
						(LightmapTilePoolGPU.GetPositionFromLinearAddress(Tile.TileAddressInWorkingSet).X + 1) * LightmapTilePoolGPU.LayerFormatAndTileSize[PoolLayerIndex].TileSize.X,
						(LightmapTilePoolGPU.GetPositionFromLinearAddress(Tile.TileAddressInWorkingSet).Y + 1) * LightmapTilePoolGPU.LayerFormatAndTileSize[PoolLayerIndex].TileSize.Y);
				}
			}

			if (ViewportsToClear.Num())
			{
				FRDGTextureRef Texture = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[PoolLayerIndex]);

				auto* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
				PassParameters->RenderTargets[0] = FRenderTargetBinding(Texture, ERenderTargetLoadAction::ENoAction);

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("ClearLightmapTilePoolGPU"),
					PassParameters,
					ERDGPassFlags::Raster,
					[LocalViewportsToClear = MoveTemp(ViewportsToClear)](FRHICommandList& RHICmdList)
				{
					for (FVector4f Viewport : LocalViewportsToClear)
					{
						RHICmdList.SetViewport(Viewport.X, Viewport.Y, 0.0f, Viewport.Z, Viewport.W, 1.0f);
						DrawClearQuad(RHICmdList, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
					}
				});
			}
		}

		for (auto& Tile : PendingTileRequests)
		{
			if (Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).Revision != CurrentRevision)
			{
				{
					// Reset GI sample states
					Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).Invalidate();
				}

				{
					// Clear stationary light sample states
					Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantDirectionalLightSampleCount.Empty();
					Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantPointLightSampleCount.Empty();
					Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantSpotLightSampleCount.Empty();
					Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantRectLightSampleCount.Empty();

					for (FDirectionalLightRenderState& DirectionalLight : Scene->LightSceneRenderState.DirectionalLights.Elements)
					{
						if (DirectionalLight.bStationary)
						{
							Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantDirectionalLightSampleCount.Add(FDirectionalLightRenderStateRef(DirectionalLight, Scene->LightSceneRenderState.DirectionalLights), 0);
						}
					}

					for (FPointLightRenderStateRef& PointLight : Tile.RenderState->RelevantPointLights)
					{
						check(PointLight->bStationary);

						Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantPointLightSampleCount.Add(PointLight, 0);
					}

					for (FSpotLightRenderStateRef& SpotLight : Tile.RenderState->RelevantSpotLights)
					{
						check(SpotLight->bStationary);

						Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantSpotLightSampleCount.Add(SpotLight, 0);
					}

					for (FRectLightRenderStateRef& RectLight : Tile.RenderState->RelevantRectLights)
					{
						check(RectLight->bStationary);

						Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantRectLightSampleCount.Add(RectLight, 0);
					}
				}

				{
					// Last step: set invalidation state to 'valid'
					Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).Revision = CurrentRevision;
				}
			}
		}
	}

	int32 NumSamplesPerFrame = (bInsideBackgroundTick && bIsViewportNonRealtime) ? Scene->Settings->TilePassesInFullSpeedMode : Scene->Settings->TilePassesInSlowMode;
	NumSamplesPerFrame = FMath::Max(FMath::Min(NumSamplesPerFrame, NumTotalPassesToRender - 1), 0);

	{
#if RHI_RAYTRACING
		FLightmapPathTracingRGS::FParameters* PreviousPassParameters[MAX_NUM_GPUS] = {};
#endif

		// Render GI
		for (int32 SampleIndex = 0; SampleIndex < NumSamplesPerFrame; SampleIndex++)
		{
			TArray<FLightmapTileRequest> PendingGITileRequests =
				PendingTileRequests.FilterByPredicate([NumTotalPassesToRender = NumTotalPassesToRender, MostCommonLODIndex](const FLightmapTileRequest& Tile)
					{
					return !Tile.RenderState->IsTileGIConverged(Tile.VirtualCoordinates, NumTotalPassesToRender) && Tile.RenderState->GeometryInstanceRef.LODIndex == MostCommonLODIndex;
					});
			
			if (PendingGITileRequests.Num() > 0)
			{
				const int32 AAvsGIMultiplier = 8;

				if (SampleIndex % AAvsGIMultiplier == 0)
				{
					TArray<int32>& PendingGIRenderPassIndices = *GraphBuilder.AllocObject<TArray<int32>>();

					for (auto& Tile : PendingGITileRequests)
					{
						PendingGIRenderPassIndices.Add(Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).RenderPassIndex);
					}

					TResourceArray<FIntPoint> TilePositionsToClear;
					for (auto& Tile : PendingGITileRequests)
					{
						TilePositionsToClear.Add(ScratchTilePoolGPU->GetPositionFromLinearAddress(Tile.TileAddressInScratch));
					}
					
					ClearScratchTilePoolForMultipleTiles(GraphBuilder, TilePositionsToClear, ScratchTilePoolLayerUAVs, GlobalShaderMap);

					{
						for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
						{
							RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::FromIndex(GPUIndex));

							auto* PassParameters = GraphBuilder.AllocParameters<FLightmapGBufferPassParameters>();
							PassParameters->View = Scene->ReferenceView->ViewUniformBuffer;
							PassParameters->Scene = SceneUniforms.GetBuffer(GraphBuilder);
							PassParameters->PassUniformBuffer = PassUniformBuffer;
							PassParameters->InstanceCulling = InstanceCullingUniformBuffer;

							for (int32 Index = 0; Index < PendingGITileRequests.Num(); Index++)
							{
								const FLightmapTileRequest& Tile = PendingGITileRequests[Index];

								uint32 AssignedGPUIndex = (Tile.RenderState->DistributionPrefixSum + Tile.RenderState->RetrieveTileStateIndex(Tile.VirtualCoordinates)) % GNumExplicitGPUsForRendering;
								if (AssignedGPUIndex != GPUIndex) continue;

								float ScaleX = Tile.RenderState->GetPaddedSizeInTiles().X * GPreviewLightmapVirtualTileSize * 1.0f / (1 << Tile.VirtualCoordinates.MipLevel) / GPreviewLightmapPhysicalTileSize;
								float ScaleY = Tile.RenderState->GetPaddedSizeInTiles().Y * GPreviewLightmapVirtualTileSize * 1.0f / (1 << Tile.VirtualCoordinates.MipLevel) / GPreviewLightmapPhysicalTileSize;
								float BiasX = (1.0f * (-Tile.VirtualCoordinates.Position.X * GPreviewLightmapVirtualTileSize) - (-GPreviewLightmapTileBorderSize)) / GPreviewLightmapPhysicalTileSize;
								float BiasY = (1.0f * (-Tile.VirtualCoordinates.Position.Y * GPreviewLightmapVirtualTileSize) - (-GPreviewLightmapTileBorderSize)) / GPreviewLightmapPhysicalTileSize;

								FVector4f VirtualTexturePhysicalTileCoordinateScaleAndBias = FVector4f(ScaleX, ScaleY, BiasX, BiasY);

								TArray<FMeshBatch> MeshBatches = Tile.RenderState->GeometryInstanceRef.GetMeshBatchesForGBufferRendering(Tile.VirtualCoordinates);

								int32 EffectiveRenderPassIndex = PendingGIRenderPassIndices[Index];

								if (Scene->Settings->bUseIrradianceCaching)
								{
									if (EffectiveRenderPassIndex >= Scene->Settings->IrradianceCacheQuality)
									{
										EffectiveRenderPassIndex -= Scene->Settings->IrradianceCacheQuality;

										if (Scene->Settings->bUseFirstBounceRayGuiding)
										{
											if (EffectiveRenderPassIndex >= Scene->Settings->FirstBounceRayGuidingTrialSamples)
											{
												EffectiveRenderPassIndex -= Scene->Settings->FirstBounceRayGuidingTrialSamples;
											}
										}
									}
								}

								GraphBuilder.AddPass(
									RDG_EVENT_NAME("LightmapGBufferTile"),
									PassParameters,
									ERDGPassFlags::Raster,
									[
										this,
										ReferenceView = Scene->ReferenceView,
										PrimitiveId = Scene->GetPrimitiveIdForGPUScene(Tile.RenderState->GeometryInstanceRef),
										MeshBatches,
										VirtualTexturePhysicalTileCoordinateScaleAndBias,
										EffectiveRenderPassIndex,
										AAvsGIMultiplier,
										ScratchTilePoolOffset = ScratchTilePoolGPU->GetPositionFromLinearAddress(Tile.TileAddressInScratch) * GPreviewLightmapPhysicalTileSize
										](FRHICommandListImmediate& RHICmdList)
									{
										RHICmdList.SetViewport(0, 0, 0.0f, GPreviewLightmapPhysicalTileSize, GPreviewLightmapPhysicalTileSize, 1.0f);
										
										RenderMeshBatchesIntoGBuffer(
											RHICmdList,
											ReferenceView.Get(),
											PrimitiveId,
											MeshBatches,
											VirtualTexturePhysicalTileCoordinateScaleAndBias,
											EffectiveRenderPassIndex / AAvsGIMultiplier,
											ScratchTilePoolOffset);

										GPrimitiveIdVertexBufferPool.DiscardAll();
									});
							}
						}
					}
				}

#if RHI_RAYTRACING
				if (IsRayTracingEnabled())
				{
					for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
					{
						FGPUBatchedTileRequests GPUBatchedTileRequests;

						TArray<FLightmapTileRequest> TileRequestsThisGPU;
						
						for (const FLightmapTileRequest& Tile : PendingGITileRequests)
						{
							uint32 AssignedGPUIndex = (Tile.RenderState->DistributionPrefixSum + Tile.RenderState->RetrieveTileStateIndex(Tile.VirtualCoordinates)) % GNumExplicitGPUsForRendering;
							if (AssignedGPUIndex != GPUIndex) continue;

							TileRequestsThisGPU.Add(Tile);
							
							if (!Tile.RenderState->IsTileGIConverged(Tile.VirtualCoordinates, NumTotalPassesToRender))
							{
								Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).RenderPassIndex++;

								if (/*Tile.VirtualCoordinates.MipLevel == 0 && */SampleIndex == 0)
								{
									if (!bInsideBackgroundTick)
									{
										Mip0WorkDoneLastFrame++;
									}
								}
							}
						}

						GPUBatchedTileRequests.BuildFromTileDescs(TileRequestsThisGPU, LightmapTilePoolGPU, *ScratchTilePoolGPU);
						GPUBatchedTileRequests.Commit(GPUIndex);
						// Let GraphBuilder references the SRV
						GraphBuilder.AllocObject<FShaderResourceViewRHIRef>(GPUBatchedTileRequests.BatchedTilesSRV);

						RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::FromIndex(GPUIndex));

						if (TileRequestsThisGPU.Num() > 0)
						{
							FRDGTextureRef GBufferWorldPosition = GraphBuilder.RegisterExternalTexture(ScratchTilePoolGPU->PooledRenderTargets[0], TEXT("GBufferWorldPosition"));
							FRDGTextureRef GBufferWorldNormal = GraphBuilder.RegisterExternalTexture(ScratchTilePoolGPU->PooledRenderTargets[1], TEXT("GBufferWorldNormal"));
							FRDGTextureRef GBufferShadingNormal = GraphBuilder.RegisterExternalTexture(ScratchTilePoolGPU->PooledRenderTargets[2], TEXT("GBufferShadingNormal"));
							FRDGTextureRef IrradianceAndSampleCount = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[0], TEXT("IrradianceAndSampleCount"));
							FRDGTextureRef SHDirectionality = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[1], TEXT("SHDirectionality"));
							FRDGTextureRef SHCorrectionAndStationarySkyLightBentNormal = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[4], TEXT("SHCorrectionAndStationarySkyLightBentNormal"));

							FRDGTextureRef RayGuidingLuminance = nullptr;
							FRDGTextureRef RayGuidingCDFX = nullptr;
							FRDGTextureRef RayGuidingCDFY = nullptr;

							if (Scene->Settings->bUseFirstBounceRayGuiding)
							{
								RayGuidingLuminance = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[5], TEXT("RayGuidingLuminance"));
								RayGuidingCDFX = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[6], TEXT("RayGuidingCDFX"));
								RayGuidingCDFY = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[7], TEXT("RayGuidingCDFY"));
							}

							FIntPoint RayTracingResolution;
							RayTracingResolution.X = GPreviewLightmapPhysicalTileSize * GPUBatchedTileRequests.BatchedTilesDesc.Num();
							RayTracingResolution.Y = GPreviewLightmapPhysicalTileSize;

							// Path Tracing GI
							{
								{
									FLightmapPathTracingRGS::FParameters* PassParameters = GraphBuilder.AllocParameters<FLightmapPathTracingRGS::FParameters>();
									PassParameters->LastInvalidationFrame = LastInvalidationFrame;
									PassParameters->NumTotalSamples = NumTotalPassesToRender;
									PassParameters->TLAS = Scene->RayTracingSceneSRV;
									PassParameters->GBufferWorldPosition = GBufferWorldPosition;
									PassParameters->GBufferWorldNormal = GBufferWorldNormal;
									PassParameters->GBufferShadingNormal = GBufferShadingNormal;
									PassParameters->IrradianceAndSampleCount = GraphBuilder.CreateUAV(IrradianceAndSampleCount);
									PassParameters->SHCorrectionAndStationarySkyLightBentNormal = GraphBuilder.CreateUAV(SHCorrectionAndStationarySkyLightBentNormal);
									PassParameters->SHDirectionality = GraphBuilder.CreateUAV(SHDirectionality);

									if (Scene->Settings->bUseFirstBounceRayGuiding)
									{
										PassParameters->RayGuidingLuminance = GraphBuilder.CreateUAV(RayGuidingLuminance);
										PassParameters->RayGuidingCDFX = RayGuidingCDFX;
										PassParameters->RayGuidingCDFY = RayGuidingCDFY;
										PassParameters->NumRayGuidingTrialSamples = Scene->Settings->FirstBounceRayGuidingTrialSamples;
									}

									PassParameters->BatchedTiles = GPUBatchedTileRequests.BatchedTilesSRV;
									PassParameters->ViewUniformBuffer = Scene->ReferenceView->ViewUniformBuffer;
									PassParameters->Scene = SceneUniforms.GetBuffer(GraphBuilder);
									PassParameters->IrradianceCachingParameters = Scene->IrradianceCache->IrradianceCachingParametersUniformBuffer;

									if (PreviousPassParameters[GPUIndex] == nullptr)
									{
										SetupPathTracingLightParameters(Scene->LightSceneRenderState, GraphBuilder, *Scene->ReferenceView, PassParameters);
										// store the first pass parameters so we don't have to re-create certain resources constantly
										PreviousPassParameters[GPUIndex] = PassParameters;
									}
									else
									{
										PassParameters->LightGridParameters = PreviousPassParameters[GPUIndex]->LightGridParameters;
										PassParameters->SceneLightCount = PreviousPassParameters[GPUIndex]->SceneLightCount;
										PassParameters->SceneVisibleLightCount = PreviousPassParameters[GPUIndex]->SceneVisibleLightCount;
										PassParameters->SceneLights = PreviousPassParameters[GPUIndex]->SceneLights;
										PassParameters->SkylightTexture = PreviousPassParameters[GPUIndex]->SkylightTexture;
										PassParameters->SkylightTextureSampler = PreviousPassParameters[GPUIndex]->SkylightTextureSampler;
										PassParameters->SkylightPdf = PreviousPassParameters[GPUIndex]->SkylightPdf;
										PassParameters->SkylightInvResolution = PreviousPassParameters[GPUIndex]->SkylightInvResolution;
										PassParameters->SkylightMipCount = PreviousPassParameters[GPUIndex]->SkylightMipCount;
									}

									FLightmapPathTracingRGS::FPermutationDomain PermutationVector;
									PermutationVector.Set<FLightmapPathTracingRGS::FUseFirstBounceRayGuiding>(Scene->Settings->bUseIrradianceCaching && Scene->Settings->bUseFirstBounceRayGuiding);
									PermutationVector.Set<FLightmapPathTracingRGS::FUseIrradianceCaching>(Scene->Settings->bUseIrradianceCaching);
									PermutationVector.Set<FLightmapPathTracingRGS::FUseICBackfaceDetection>(Scene->Settings->bUseIrradianceCaching && Scene->Settings->bUseIrradianceCacheBackfaceDetection);
									auto RayGenerationShader = GlobalShaderMap->GetShader<FLightmapPathTracingRGS>(PermutationVector);
									ClearUnusedGraphResources(RayGenerationShader, PassParameters);

									GraphBuilder.AddPass(
										RDG_EVENT_NAME("LightmapPathTracing %dx%d", RayTracingResolution.X, RayTracingResolution.Y),
										PassParameters,
										ERDGPassFlags::Compute,
										[PassParameters, this, PipelineState = Scene->RayTracingPipelineState, SBT = Scene->SBT, RayGenerationShader, RayTracingResolution, GPUIndex](FRHICommandList& RHICmdList)
									{
										FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
										SetShaderParameters(GlobalResources, RayGenerationShader, *PassParameters);

										check(RHICmdList.GetGPUMask().HasSingleIndex());

										RHICmdList.RayTraceDispatch(PipelineState, RayGenerationShader.GetRayTracingShader(), SBT, GlobalResources, RayTracingResolution.X, RayTracingResolution.Y);
									});
								}

								if (Scene->Settings->bUseFirstBounceRayGuiding)
								{
									FFirstBounceRayGuidingCDFBuildCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFirstBounceRayGuidingCDFBuildCS::FParameters>();

									PassParameters->BatchedTiles = GPUBatchedTileRequests.BatchedTilesSRV;
									PassParameters->RayGuidingLuminance = GraphBuilder.CreateUAV(RayGuidingLuminance);
									PassParameters->RayGuidingCDFX = GraphBuilder.CreateUAV(RayGuidingCDFX);
									PassParameters->RayGuidingCDFY = GraphBuilder.CreateUAV(RayGuidingCDFY);
									PassParameters->RayGuidingEndPassIndex = Scene->Settings->FirstBounceRayGuidingTrialSamples - 1;

									if (Scene->Settings->bUseIrradianceCaching)
									{
										PassParameters->RayGuidingEndPassIndex += Scene->Settings->IrradianceCacheQuality;
									}

									TShaderMapRef<FFirstBounceRayGuidingCDFBuildCS> ComputeShader(GlobalShaderMap);
									FComputeShaderUtils::AddPass(
										GraphBuilder,
										RDG_EVENT_NAME("FirstBounceRayGuidingCDFBuild"),
										ComputeShader,
										PassParameters,
										FIntVector(GPUBatchedTileRequests.BatchedTilesDesc.Num() * 256, 1, 1));
								}
							}
						}
					}
				}
#endif
			}
		}
	}

	for (int32 SampleIndex = 0; SampleIndex < NumSamplesPerFrame; SampleIndex++)
	{
		// Render shadow mask
		{
			TArray<FLightmapTileRequest> PendingShadowTileRequestsOnAllGPUs = PendingTileRequests.FilterByPredicate([NumShadowSamples = Scene->Settings->StationaryLightShadowSamples, MostCommonLODIndex](const FLightmapTileRequest& Tile) { 
				return !Tile.RenderState->IsTileShadowConverged(Tile.VirtualCoordinates, NumShadowSamples) && Tile.RenderState->GeometryInstanceRef.LODIndex == MostCommonLODIndex;
			});

			if (PendingShadowTileRequestsOnAllGPUs.Num() > 0)
			{
				for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
				{
					RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::FromIndex(GPUIndex));

					TArray<FLightmapTileRequest> PendingShadowTileRequests = PendingShadowTileRequestsOnAllGPUs.FilterByPredicate(
						[GPUIndex](const FLightmapTileRequest& Tile)
					{
						uint32 AssignedGPUIndex = (Tile.RenderState->DistributionPrefixSum + Tile.RenderState->RetrieveTileStateIndex(Tile.VirtualCoordinates)) % GNumExplicitGPUsForRendering;
						return AssignedGPUIndex == GPUIndex;
					});

					if (PendingShadowTileRequests.Num() == 0) continue;

					TResourceArray<FIntPoint> TilePositionsToClear;
					for (auto& Tile : PendingShadowTileRequests)
					{
						TilePositionsToClear.Add(ScratchTilePoolGPU->GetPositionFromLinearAddress(Tile.TileAddressInScratch));
					}

					ClearScratchTilePoolForMultipleTiles(GraphBuilder, TilePositionsToClear, ScratchTilePoolLayerUAVs, GlobalShaderMap);

					FRDGTextureRef GBufferWorldPosition = GraphBuilder.RegisterExternalTexture(ScratchTilePoolGPU->PooledRenderTargets[0], TEXT("GBufferWorldPosition"));
					FRDGTextureRef GBufferWorldNormal = GraphBuilder.RegisterExternalTexture(ScratchTilePoolGPU->PooledRenderTargets[1], TEXT("GBufferWorldNormal"));
					FRDGTextureRef GBufferShadingNormal = GraphBuilder.RegisterExternalTexture(ScratchTilePoolGPU->PooledRenderTargets[2], TEXT("GBufferShadingNormal"));

					FRDGTextureRef ShadowMask = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[2], TEXT("ShadowMask"));
					FRDGTextureRef ShadowMaskSampleCount = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[3], TEXT("ShadowMaskSampleCount"));

					TResourceArray<int32> LightTypeArray;
					FBufferRHIRef LightTypeBuffer;
					FShaderResourceViewRHIRef LightTypeSRV;

					TResourceArray<int32> ChannelIndexArray;
					FBufferRHIRef ChannelIndexBuffer;
					FShaderResourceViewRHIRef ChannelIndexSRV;

					auto& LightSampleIndexArray = *GraphBuilder.AllocObject<TResourceArray<int32>>();
					FBufferRHIRef LightSampleIndexBuffer;
					FShaderResourceViewRHIRef LightSampleIndexSRV;

					TResourceArray<FLightShaderConstants> LightShaderParameterArray;
					FBufferRHIRef LightShaderParameterBuffer;
					FShaderResourceViewRHIRef LightShaderParameterSRV;

					for (const FLightmapTileRequest& Tile : PendingShadowTileRequests)
					{
						// Gather all unconverged lights, then pick one based on RoundRobinIndex
						TArray<int32> UnconvergedLightTypeArray;
						TArray<int32> UnconvergedChannelIndexArray;
						TArray<int32> UnconvergedLightSampleIndexArray;
						TArray<FLightShaderConstants> UnconvergedLightShaderParameterArray;

						for (auto& Pair : Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantDirectionalLightSampleCount)
						{
							if (Pair.Value < Scene->Settings->StationaryLightShadowSamples)
							{
								UnconvergedLightTypeArray.Add(0);
								UnconvergedChannelIndexArray.Add(Pair.Key->ShadowMapChannel);
								UnconvergedLightShaderParameterArray.Add(FLightShaderConstants(Pair.Key->GetLightShaderParameters()));
								UnconvergedLightSampleIndexArray.Add(Pair.Value);
							}
						}

						for (auto& Pair : Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantPointLightSampleCount)
						{
							if (Pair.Value < Scene->Settings->StationaryLightShadowSamples)
							{
								UnconvergedLightTypeArray.Add(1);
								UnconvergedChannelIndexArray.Add(Pair.Key->ShadowMapChannel);
								UnconvergedLightShaderParameterArray.Add(FLightShaderConstants(Pair.Key->GetLightShaderParameters()));
								UnconvergedLightSampleIndexArray.Add(Pair.Value);
							}
						}

						for (auto& Pair : Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantSpotLightSampleCount)
						{
							if (Pair.Value < Scene->Settings->StationaryLightShadowSamples)
							{
								UnconvergedLightTypeArray.Add(2);
								UnconvergedChannelIndexArray.Add(Pair.Key->ShadowMapChannel);
								UnconvergedLightShaderParameterArray.Add(FLightShaderConstants(Pair.Key->GetLightShaderParameters()));
								UnconvergedLightSampleIndexArray.Add(Pair.Value);
							}
						}

						for (auto& Pair : Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantRectLightSampleCount)
						{
							if (Pair.Value < Scene->Settings->StationaryLightShadowSamples)
							{
								UnconvergedLightTypeArray.Add(3);
								UnconvergedChannelIndexArray.Add(Pair.Key->ShadowMapChannel);
								UnconvergedLightShaderParameterArray.Add(FLightShaderConstants(Pair.Key->GetLightShaderParameters()));
								UnconvergedLightSampleIndexArray.Add(Pair.Value);
							}
						}

						int32 PickedLightIndex = Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RoundRobinIndex % UnconvergedLightTypeArray.Num();

						LightTypeArray.Add(UnconvergedLightTypeArray[PickedLightIndex]);
						ChannelIndexArray.Add(UnconvergedChannelIndexArray[PickedLightIndex]);
						LightSampleIndexArray.Add(UnconvergedLightSampleIndexArray[PickedLightIndex]);
						LightShaderParameterArray.Add(UnconvergedLightShaderParameterArray[PickedLightIndex]);

						Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RoundRobinIndex++;

						{
							int32 LightIndex = 0;
							bool bFoundPickedLight = false;

							for (auto& Pair : Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantDirectionalLightSampleCount)
							{
								if (Pair.Value < Scene->Settings->StationaryLightShadowSamples)
								{
									if (LightIndex == PickedLightIndex)
									{
										Pair.Value++;
										bFoundPickedLight = true;
										break;
									}
									LightIndex++;
								}
							}

							for (auto& Pair : Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantPointLightSampleCount)
							{
								if (Pair.Value < Scene->Settings->StationaryLightShadowSamples)
								{
									if (LightIndex == PickedLightIndex)
									{
										Pair.Value++;
										bFoundPickedLight = true;
										break;
									}
									LightIndex++;
								}
							}

							for (auto& Pair : Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantSpotLightSampleCount)
							{
								if (Pair.Value < Scene->Settings->StationaryLightShadowSamples)
								{
									if (LightIndex == PickedLightIndex)
									{
										Pair.Value++;
										bFoundPickedLight = true;
										break;
									}
									LightIndex++;
								}
							}

							for (auto& Pair : Tile.RenderState->RetrieveTileRelevantLightSampleState(Tile.VirtualCoordinates).RelevantRectLightSampleCount)
							{
								if (Pair.Value < Scene->Settings->StationaryLightShadowSamples)
								{
									if (LightIndex == PickedLightIndex)
									{
										Pair.Value++;
										bFoundPickedLight = true;
										break;
									}
									LightIndex++;
								}
							}

							check(bFoundPickedLight);
						}
					}

					check(PendingShadowTileRequests.Num() == LightTypeArray.Num());

					{
						const FRHIBufferCreateDesc CreateDesc =
							FRHIBufferCreateDesc::CreateVertex(TEXT("LightTypeBuffer"), LightTypeArray.GetResourceDataSize())
							.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::ShaderResource)
							.SetGPUMask(FRHIGPUMask::FromIndex(GPUIndex))
							.SetInitActionResourceArray(&LightTypeArray)
							.DetermineInitialState();

						LightTypeBuffer = RHICmdList.CreateBuffer(CreateDesc);
						LightTypeSRV = RHICmdList.CreateShaderResourceView(
							LightTypeBuffer, 
							FRHIViewDesc::CreateBufferSRV()
								.SetType(FRHIViewDesc::EBufferType::Typed)
								.SetFormat(PF_R32_SINT));
					}

					{
						const FRHIBufferCreateDesc CreateDesc =
							FRHIBufferCreateDesc::CreateVertex(TEXT("ChannelIndexBuffer"), ChannelIndexArray.GetResourceDataSize())
							.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::ShaderResource)
							.SetGPUMask(FRHIGPUMask::FromIndex(GPUIndex))
							.SetInitActionResourceArray(&ChannelIndexArray)
							.DetermineInitialState();

						ChannelIndexBuffer = RHICmdList.CreateBuffer(CreateDesc);
						ChannelIndexSRV = RHICmdList.CreateShaderResourceView(
							ChannelIndexBuffer, 
							FRHIViewDesc::CreateBufferSRV()
								.SetType(FRHIViewDesc::EBufferType::Typed)
								.SetFormat(PF_R32_SINT));
					}

					{
						const FRHIBufferCreateDesc CreateDesc =
							FRHIBufferCreateDesc::CreateVertex(TEXT("LightSampleIndexSRV"), LightSampleIndexArray.GetResourceDataSize())
							.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::ShaderResource)
							.SetGPUMask(FRHIGPUMask::FromIndex(GPUIndex))
							.SetInitActionResourceArray(&LightSampleIndexArray)
							.DetermineInitialState();

						LightSampleIndexBuffer = RHICmdList.CreateBuffer(CreateDesc);
						LightSampleIndexSRV = RHICmdList.CreateShaderResourceView(
							LightSampleIndexBuffer, 
							FRHIViewDesc::CreateBufferSRV()
								.SetType(FRHIViewDesc::EBufferType::Typed)
								.SetFormat(PF_R32_SINT));
					}

					{
						const FRHIBufferCreateDesc CreateDesc =
							FRHIBufferCreateDesc::CreateStructured<FLightShaderConstants>(TEXT("LightShaderParameterBuffer"), LightShaderParameterArray.Num())
							.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::ShaderResource)
							.SetGPUMask(FRHIGPUMask::FromIndex(GPUIndex))
							.SetInitActionResourceArray(&LightShaderParameterArray)
							.DetermineInitialState();

						LightShaderParameterBuffer = RHICmdList.CreateBuffer(CreateDesc);
						LightShaderParameterSRV = RHICmdList.CreateShaderResourceView(LightShaderParameterBuffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(LightShaderParameterBuffer));
					}

					// Render GBuffer
					{
						auto* PassParameters = GraphBuilder.AllocParameters<FLightmapGBufferPassParameters>();
						PassParameters->View = Scene->ReferenceView->ViewUniformBuffer;
						PassParameters->Scene = SceneUniforms.GetBuffer(GraphBuilder);
						PassParameters->PassUniformBuffer = PassUniformBuffer;
						PassParameters->InstanceCulling = InstanceCullingUniformBuffer;

						for (int32 TileIndex = 0; TileIndex < PendingShadowTileRequests.Num(); TileIndex++)
						{
							const FLightmapTileRequest& Tile = PendingShadowTileRequests[TileIndex];

							float ScaleX = Tile.RenderState->GetPaddedSizeInTiles().X * GPreviewLightmapVirtualTileSize * 1.0f / (1 << Tile.VirtualCoordinates.MipLevel) / GPreviewLightmapPhysicalTileSize;
							float ScaleY = Tile.RenderState->GetPaddedSizeInTiles().Y * GPreviewLightmapVirtualTileSize * 1.0f / (1 << Tile.VirtualCoordinates.MipLevel) / GPreviewLightmapPhysicalTileSize;
							float BiasX = (1.0f * (-Tile.VirtualCoordinates.Position.X * GPreviewLightmapVirtualTileSize) - (-GPreviewLightmapTileBorderSize)) / GPreviewLightmapPhysicalTileSize;
							float BiasY = (1.0f * (-Tile.VirtualCoordinates.Position.Y * GPreviewLightmapVirtualTileSize) - (-GPreviewLightmapTileBorderSize)) / GPreviewLightmapPhysicalTileSize;

							FVector4f VirtualTexturePhysicalTileCoordinateScaleAndBias = FVector4f(ScaleX, ScaleY, BiasX, BiasY);

							TArray<FMeshBatch> MeshBatches = Tile.RenderState->GeometryInstanceRef.GetMeshBatchesForGBufferRendering(Tile.VirtualCoordinates);

							GraphBuilder.AddPass(
								RDG_EVENT_NAME("LightmapGBuffer"),
								PassParameters,
								ERDGPassFlags::Raster,
								[
									this,
									ReferenceView = Scene->ReferenceView,
									PrimitiveId = Scene->GetPrimitiveIdForGPUScene(Tile.RenderState->GeometryInstanceRef),
									MeshBatches,
									VirtualTexturePhysicalTileCoordinateScaleAndBias,
									RenderPassIndex = LightSampleIndexArray[TileIndex],
									ScratchTilePoolOffset = ScratchTilePoolGPU->GetPositionFromLinearAddress(Tile.TileAddressInScratch) * GPreviewLightmapPhysicalTileSize
									](FRHICommandListImmediate& RHICmdList)
								{
									RHICmdList.SetViewport(0, 0, 0.0f, GPreviewLightmapPhysicalTileSize, GPreviewLightmapPhysicalTileSize, 1.0f);
									
									RenderMeshBatchesIntoGBuffer(
										RHICmdList,
										ReferenceView.Get(),
										PrimitiveId,
										MeshBatches,
										VirtualTexturePhysicalTileCoordinateScaleAndBias,
										RenderPassIndex,
										ScratchTilePoolOffset);

									GPrimitiveIdVertexBufferPool.DiscardAll();
								});
						}
					}

#if RHI_RAYTRACING
					if (IsRayTracingEnabled())
					{
						FGPUBatchedTileRequests GPUBatchedTileRequests;						
						GPUBatchedTileRequests.BuildFromTileDescs(PendingShadowTileRequests,LightmapTilePoolGPU, *ScratchTilePoolGPU);
						for (int32 TileIndex = 0; TileIndex < PendingShadowTileRequests.Num(); TileIndex++)
						{
							GPUBatchedTileRequests.BatchedTilesDesc[TileIndex].RenderPassIndex = LightSampleIndexArray[TileIndex];
						}
						GPUBatchedTileRequests.Commit(GPUIndex);
						
						// Let GraphBuilder references the SRV
						GraphBuilder.AllocObject<FShaderResourceViewRHIRef>(GPUBatchedTileRequests.BatchedTilesSRV);

						FIntPoint RayTracingResolution;
						RayTracingResolution.X = GPreviewLightmapPhysicalTileSize * GPUBatchedTileRequests.BatchedTilesDesc.Num();
						RayTracingResolution.Y = GPreviewLightmapPhysicalTileSize;

						FStationaryLightShadowTracingRGS::FParameters* PassParameters = GraphBuilder.AllocParameters<FStationaryLightShadowTracingRGS::FParameters>();
						PassParameters->ViewUniformBuffer = Scene->ReferenceView->ViewUniformBuffer;
						PassParameters->Scene = SceneUniforms.GetBuffer(GraphBuilder);
						PassParameters->TLAS = Scene->RayTracingSceneSRV;
						PassParameters->BatchedTiles = GPUBatchedTileRequests.BatchedTilesSRV;
						PassParameters->LightTypeArray = HoldReference(GraphBuilder, LightTypeSRV);
						PassParameters->ChannelIndexArray = HoldReference(GraphBuilder, ChannelIndexSRV);
						PassParameters->LightSampleIndexArray = HoldReference(GraphBuilder, LightSampleIndexSRV);
						PassParameters->LightShaderParametersArray = HoldReference(GraphBuilder, LightShaderParameterSRV);
						PassParameters->GBufferWorldPosition = GBufferWorldPosition;
						PassParameters->GBufferWorldNormal = GBufferWorldNormal;
						PassParameters->GBufferShadingNormal = GBufferShadingNormal;
						PassParameters->ShadowMask = GraphBuilder.CreateUAV(ShadowMask);
						PassParameters->ShadowMaskSampleCount = GraphBuilder.CreateUAV(ShadowMaskSampleCount);

						auto RayGenerationShader = GlobalShaderMap->GetShader<FStationaryLightShadowTracingRGS>();
						ClearUnusedGraphResources(RayGenerationShader, PassParameters);

						GraphBuilder.AddPass(
							RDG_EVENT_NAME("StationaryLightShadowTracing %dx%d", RayTracingResolution.X, RayTracingResolution.Y),
							PassParameters,
							ERDGPassFlags::Compute,
							[PassParameters, this, PipelineState = Scene->RayTracingPipelineState, SBT = Scene->SBT, RayGenerationShader, RayTracingResolution](FRHICommandList& RHICmdList)
						{
							FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
							SetShaderParameters(GlobalResources, RayGenerationShader, *PassParameters);

							RHICmdList.RayTraceDispatch(PipelineState, RayGenerationShader.GetRayTracingShader(), SBT, GlobalResources, RayTracingResolution.X, RayTracingResolution.Y);
						});
					}
#endif
				}
			}
		}
	}

	// Pull results from other GPUs using batched transfer if realtime
	if (!bInsideBackgroundTick)
	{
		TArray<FTransferResourceParams> Params;

		for (const FLightmapTileRequest& Tile : PendingTileRequests)
		{
			uint32 AssignedGPUIndex = (Tile.RenderState->DistributionPrefixSum + Tile.RenderState->RetrieveTileStateIndex(Tile.VirtualCoordinates)) % GNumExplicitGPUsForRendering;
			if (AssignedGPUIndex != 0)
			{
				auto TransferTexture = [&](int32 RenderTargetIndex) {
					FIntRect GPURect;
					GPURect.Min = LightmapTilePoolGPU.GetPositionFromLinearAddress(Tile.TileAddressInWorkingSet) * LightmapTilePoolGPU.LayerFormatAndTileSize[RenderTargetIndex].TileSize;
					GPURect.Max = GPURect.Min + LightmapTilePoolGPU.LayerFormatAndTileSize[RenderTargetIndex].TileSize;
					Params.Add(FTransferResourceParams(LightmapTilePoolGPU.PooledRenderTargets[RenderTargetIndex]->GetRHI(), GPURect, AssignedGPUIndex, 0, true, true));
				};

				TransferTexture(0);
				TransferTexture(1);
				TransferTexture(2);
				TransferTexture(3);
				TransferTexture(4);

				if (Scene->Settings->bUseFirstBounceRayGuiding)
				{
					TransferTexture(5);
					TransferTexture(6);
					TransferTexture(7);
				}
			}
		}

		AddPass(GraphBuilder, RDG_EVENT_NAME("TransferResources"), [LocalParams = MoveTemp(Params)](FRHICommandList& RHICmdList)
		{
			RHICmdList.TransferResources(LocalParams);
		});
	}

	// Output from working set to VT layers
	{
		FGPUBatchedTileRequests GPUBatchedTileRequests;
		GPUBatchedTileRequests.BuildFromTileDescs(PendingTileRequests, LightmapTilePoolGPU, *ScratchTilePoolGPU);
		GPUBatchedTileRequests.Commit(0);
		// Let GraphBuilder references the SRV
		GraphBuilder.AllocObject<FShaderResourceViewRHIRef>(GPUBatchedTileRequests.BatchedTilesSRV);

		{
			FRDGTextureRef IrradianceAndSampleCount = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[0], TEXT("IrradianceAndSampleCount"));
			FRDGTextureRef SHDirectionality = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[1], TEXT("SHDirectionality"));
			FRDGTextureRef ShadowMask = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[2], TEXT("ShadowMask"));
			FRDGTextureRef ShadowMaskSampleCount = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[3], TEXT("ShadowMaskSampleCount"));
			FRDGTextureRef SHCorrectionAndStationarySkyLightBentNormal = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[4], TEXT("SHCorrectionAndStationarySkyLightBentNormal"));

			FIntPoint RayTracingResolution;
			RayTracingResolution.X = GPreviewLightmapPhysicalTileSize * GPUBatchedTileRequests.BatchedTilesDesc.Num();
			RayTracingResolution.Y = GPreviewLightmapPhysicalTileSize;

			if (OutputRenderTargets[0] != nullptr)
			{
				FRDGTextureRef RenderTargetTileAtlas = GraphBuilder.RegisterExternalTexture(OutputRenderTargets[0], TEXT("GPULightmassRenderTargetTileAtlas0"));

				FSelectiveLightmapOutputCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FSelectiveLightmapOutputCS::FOutputLayerDim>(0);
				PermutationVector.Set<FSelectiveLightmapOutputCS::FDrawProgressBars>(Scene->Settings->bShowProgressBars);

				auto Shader = GlobalShaderMap->GetShader<FSelectiveLightmapOutputCS>(PermutationVector);

				FSelectiveLightmapOutputCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSelectiveLightmapOutputCS::FParameters>();
				PassParameters->NumBatchedTiles = GPUBatchedTileRequests.BatchedTilesDesc.Num();
				PassParameters->NumTotalSamples = NumTotalPassesToRender;
				PassParameters->NumIrradianceCachePasses = Scene->Settings->bUseIrradianceCaching ? Scene->Settings->IrradianceCacheQuality : 0;
				PassParameters->NumRayGuidingTrialSamples = Scene->Settings->bUseFirstBounceRayGuiding ? Scene->Settings->FirstBounceRayGuidingTrialSamples : 0;
				PassParameters->BatchedTiles = GPUBatchedTileRequests.BatchedTilesSRV;
				PassParameters->OutputTileAtlas = GraphBuilder.CreateUAV(RenderTargetTileAtlas);
				PassParameters->IrradianceAndSampleCount = GraphBuilder.CreateUAV(IrradianceAndSampleCount);
				PassParameters->SHDirectionality = GraphBuilder.CreateUAV(SHDirectionality);
				PassParameters->SHCorrectionAndStationarySkyLightBentNormal = GraphBuilder.CreateUAV(SHCorrectionAndStationarySkyLightBentNormal);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("SelectiveLightmapOutput 0"),
					Shader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(RayTracingResolution, FComputeShaderUtils::kGolden2DGroupSize));
			}

			if (OutputRenderTargets[1] != nullptr)
			{
				FRDGTextureRef RenderTargetTileAtlas = GraphBuilder.RegisterExternalTexture(OutputRenderTargets[1], TEXT("GPULightmassRenderTargetTileAtlas1"));

				FSelectiveLightmapOutputCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FSelectiveLightmapOutputCS::FOutputLayerDim>(1);
				PermutationVector.Set<FSelectiveLightmapOutputCS::FDrawProgressBars>(Scene->Settings->bShowProgressBars);

				auto Shader = GlobalShaderMap->GetShader<FSelectiveLightmapOutputCS>(PermutationVector);

				FSelectiveLightmapOutputCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSelectiveLightmapOutputCS::FParameters>();
				PassParameters->NumBatchedTiles = GPUBatchedTileRequests.BatchedTilesDesc.Num();
				PassParameters->NumTotalSamples = NumTotalPassesToRender;
				PassParameters->NumIrradianceCachePasses = Scene->Settings->bUseIrradianceCaching ? Scene->Settings->IrradianceCacheQuality : 0;
				PassParameters->NumRayGuidingTrialSamples = Scene->Settings->bUseFirstBounceRayGuiding ? Scene->Settings->FirstBounceRayGuidingTrialSamples : 0;
				PassParameters->BatchedTiles = GPUBatchedTileRequests.BatchedTilesSRV;
				PassParameters->OutputTileAtlas = GraphBuilder.CreateUAV(RenderTargetTileAtlas);
				PassParameters->IrradianceAndSampleCount = GraphBuilder.CreateUAV(IrradianceAndSampleCount);
				PassParameters->SHDirectionality = GraphBuilder.CreateUAV(SHDirectionality);
				PassParameters->SHCorrectionAndStationarySkyLightBentNormal = GraphBuilder.CreateUAV(SHCorrectionAndStationarySkyLightBentNormal);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("SelectiveLightmapOutput 1"),
					Shader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(RayTracingResolution, FComputeShaderUtils::kGolden2DGroupSize));
			}

			if (OutputRenderTargets[2] != nullptr)
			{
				FRDGTextureRef RenderTargetTileAtlas = GraphBuilder.RegisterExternalTexture(OutputRenderTargets[2], TEXT("GPULightmassRenderTargetTileAtlas2"));

				FSelectiveLightmapOutputCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FSelectiveLightmapOutputCS::FOutputLayerDim>(2);
				PermutationVector.Set<FSelectiveLightmapOutputCS::FDrawProgressBars>(false);

				auto Shader = GlobalShaderMap->GetShader<FSelectiveLightmapOutputCS>(PermutationVector);

				FSelectiveLightmapOutputCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSelectiveLightmapOutputCS::FParameters>();
				PassParameters->NumBatchedTiles = GPUBatchedTileRequests.BatchedTilesDesc.Num();
				PassParameters->NumTotalSamples = NumTotalPassesToRender;
				PassParameters->BatchedTiles = GPUBatchedTileRequests.BatchedTilesSRV;
				PassParameters->OutputTileAtlas = GraphBuilder.CreateUAV(RenderTargetTileAtlas);
				PassParameters->ShadowMask = GraphBuilder.CreateUAV(ShadowMask);
				PassParameters->ShadowMaskSampleCount = GraphBuilder.CreateUAV(ShadowMaskSampleCount);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("SelectiveLightmapOutput 2"),
					Shader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(RayTracingResolution, FComputeShaderUtils::kGolden2DGroupSize));
			}
			
			if (OutputRenderTargets[3] != nullptr)
			{
				FRDGTextureRef RenderTargetTileAtlas = GraphBuilder.RegisterExternalTexture(OutputRenderTargets[3], TEXT("GPULightmassRenderTargetTileAtlas3"));

				FSelectiveLightmapOutputCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FSelectiveLightmapOutputCS::FOutputLayerDim>(3);
				PermutationVector.Set<FSelectiveLightmapOutputCS::FDrawProgressBars>(false);

				auto Shader = GlobalShaderMap->GetShader<FSelectiveLightmapOutputCS>(PermutationVector);

				FSelectiveLightmapOutputCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSelectiveLightmapOutputCS::FParameters>();
				PassParameters->NumBatchedTiles = GPUBatchedTileRequests.BatchedTilesDesc.Num();
				PassParameters->BatchedTiles = GPUBatchedTileRequests.BatchedTilesSRV;
				PassParameters->OutputTileAtlas = GraphBuilder.CreateUAV(RenderTargetTileAtlas);
				PassParameters->IrradianceAndSampleCount = GraphBuilder.CreateUAV(IrradianceAndSampleCount);
				PassParameters->SHCorrectionAndStationarySkyLightBentNormal = GraphBuilder.CreateUAV(SHCorrectionAndStationarySkyLightBentNormal);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("SelectiveLightmapOutput 3"),
					Shader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(RayTracingResolution, FComputeShaderUtils::kGolden2DGroupSize));
			}
		}
	}

	GraphBuilder.AddPostExecuteCallback([this]
	{
		Scene->DestroyRayTracingScene();
	});

	// Perform readback on any potential converged tiles
	{
		auto ConvergedTileRequests = PendingTileRequests.FilterByPredicate(
			[
				NumGISamples = NumTotalPassesToRender,
				NumShadowSamples = Scene->Settings->StationaryLightShadowSamples,
				bOnlyBakeWhatYouSee = bOnlyBakeWhatYouSee, 
				bDenoiseDuringInteractiveBake = bDenoiseDuringInteractiveBake
			](const FLightmapTileRequest& TileRequest)
		{
			return
				(TileRequest.VirtualCoordinates.MipLevel == 0 || bDenoiseDuringInteractiveBake || bOnlyBakeWhatYouSee) && // Only mip 0 tiles will be saved
				TileRequest.RenderState->IsTileGIConverged(TileRequest.VirtualCoordinates, NumGISamples) && TileRequest.RenderState->IsTileShadowConverged(TileRequest.VirtualCoordinates, NumShadowSamples);
		}
		);

		if (ConvergedTileRequests.Num() > 0)
		{
			int32 NewSize = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(ConvergedTileRequests.Num())));

			for (const FLightmapTileRequest& Tile : ConvergedTileRequests)
			{
				Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).OngoingReadbackRevision = CurrentRevision;
			}

			for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
			{
				RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::FromIndex(GPUIndex));

				auto ConvergedTileRequestsOnCurrentGPU = ConvergedTileRequests.FilterByPredicate(
					[GPUIndex](const FLightmapTileRequest& Tile)
				{
					uint32 AssignedGPUIndex = (Tile.RenderState->DistributionPrefixSum + Tile.RenderState->RetrieveTileStateIndex(Tile.VirtualCoordinates)) % GNumExplicitGPUsForRendering;
					return AssignedGPUIndex == GPUIndex;
				});

				if (ConvergedTileRequestsOnCurrentGPU.Num() == 0) continue;

				FLightmapReadbackGroup* ReadbackGroupToUse = nullptr;

				for (TUniquePtr<FLightmapReadbackGroup>& ReadbackGroup : RecycledReadbacks)
				{
					if (ReadbackGroup->bIsFree && ReadbackGroup->ReadbackTilePoolGPU->SizeInTiles.X >= NewSize && ReadbackGroup->GPUIndex == GPUIndex)
					{
						ReadbackGroupToUse = ReadbackGroup.Get();
						break;
					}
				}

				if (ReadbackGroupToUse == nullptr)
				{
					int32 NewIndex = RecycledReadbacks.Add(MakeUnique<FLightmapReadbackGroup>());
					ReadbackGroupToUse = RecycledReadbacks[NewIndex].Get();
				}

				FLightmapReadbackGroup& LightmapReadbackGroup = *ReadbackGroupToUse;
				LightmapReadbackGroup.bIsFree = false;
				LightmapReadbackGroup.Revision = CurrentRevision;
				LightmapReadbackGroup.GPUIndex = GPUIndex;
				LightmapReadbackGroup.ConvergedTileRequests = ConvergedTileRequestsOnCurrentGPU;
				if (!LightmapReadbackGroup.ReadbackTilePoolGPU.IsValid())
				{
					LightmapReadbackGroup.ReadbackTilePoolGPU = MakeUnique<FLightmapTilePoolGPU>(GraphBuilder.RHICmdList, 4, FIntPoint(NewSize, NewSize), FIntPoint(GPreviewLightmapPhysicalTileSize, GPreviewLightmapPhysicalTileSize));
					LightmapReadbackGroup.StagingHQLayer0Readback = MakeUnique<FRHIGPUTextureReadback>(TEXT("StagingHQLayer0Readback"));
					LightmapReadbackGroup.StagingHQLayer1Readback = MakeUnique<FRHIGPUTextureReadback>(TEXT("StagingHQLayer1Readback"));
					LightmapReadbackGroup.StagingShadowMaskReadback = MakeUnique<FRHIGPUTextureReadback>(TEXT("StagingShadowMaskReadback"));
					LightmapReadbackGroup.StagingSkyOcclusionReadback = MakeUnique<FRHIGPUTextureReadback>(TEXT("StagingSkyOcclusionReadback"));
				}

				FGPUBatchedTileRequests GPUBatchedTileRequests;
				GPUBatchedTileRequests.BuildFromTileDescs(LightmapReadbackGroup.ConvergedTileRequests, LightmapTilePoolGPU, *ScratchTilePoolGPU);
				GPUBatchedTileRequests.Commit(GPUIndex);
				// Let GraphBuilder references the SRV
				GraphBuilder.AllocObject<FShaderResourceViewRHIRef>(GPUBatchedTileRequests.BatchedTilesSRV);

				FIntPoint DispatchResolution;
				DispatchResolution.X = GPreviewLightmapPhysicalTileSize * GPUBatchedTileRequests.BatchedTilesDesc.Num();
				DispatchResolution.Y = GPreviewLightmapPhysicalTileSize;

				FRDGTextureRef IrradianceAndSampleCount = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[0], TEXT("IrradianceAndSampleCount"));
				FRDGTextureRef SHDirectionality = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[1], TEXT("SHDirectionality"));
				FRDGTextureRef ShadowMask = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[2], TEXT("ShadowMask"));
				FRDGTextureRef ShadowMaskSampleCount = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[3], TEXT("ShadowMaskSampleCount"));
				FRDGTextureRef SHCorrectionAndStationarySkyLightBentNormal = GraphBuilder.RegisterExternalTexture(LightmapTilePoolGPU.PooledRenderTargets[4], TEXT("SHCorrectionAndStationarySkyLightBentNormal"));

				FRDGTextureRef StagingHQLayer0 = GraphBuilder.RegisterExternalTexture(LightmapReadbackGroup.ReadbackTilePoolGPU->PooledRenderTargets[0], TEXT("StagingHQLayer0"));
				FRDGTextureRef StagingHQLayer1 = GraphBuilder.RegisterExternalTexture(LightmapReadbackGroup.ReadbackTilePoolGPU->PooledRenderTargets[1], TEXT("StagingHQLayer1"));
				FRDGTextureRef StagingShadowMask = GraphBuilder.RegisterExternalTexture(LightmapReadbackGroup.ReadbackTilePoolGPU->PooledRenderTargets[2], TEXT("StagingShadowMask"));
				FRDGTextureRef StagingSkyOcclusion = GraphBuilder.RegisterExternalTexture(LightmapReadbackGroup.ReadbackTilePoolGPU->PooledRenderTargets[3], TEXT("StagingSkyOcclusion"));

				{
					FCopyConvergedLightmapTilesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCopyConvergedLightmapTilesCS::FParameters>();

					PassParameters->NumBatchedTiles = GPUBatchedTileRequests.BatchedTilesDesc.Num();
					PassParameters->StagingPoolSizeX = LightmapReadbackGroup.ReadbackTilePoolGPU->SizeInTiles.X;
					PassParameters->BatchedTiles = GPUBatchedTileRequests.BatchedTilesSRV;
					PassParameters->IrradianceAndSampleCount = GraphBuilder.CreateUAV(IrradianceAndSampleCount);
					PassParameters->SHDirectionality = GraphBuilder.CreateUAV(SHDirectionality);
					PassParameters->SHCorrectionAndStationarySkyLightBentNormal = GraphBuilder.CreateUAV(SHCorrectionAndStationarySkyLightBentNormal);
					PassParameters->ShadowMask = GraphBuilder.CreateUAV(ShadowMask);
					PassParameters->ShadowMaskSampleCount = GraphBuilder.CreateUAV(ShadowMaskSampleCount);
					PassParameters->StagingHQLayer0 = GraphBuilder.CreateUAV(StagingHQLayer0);
					PassParameters->StagingHQLayer1 = GraphBuilder.CreateUAV(StagingHQLayer1);
					PassParameters->StagingShadowMask = GraphBuilder.CreateUAV(StagingShadowMask);
					PassParameters->StagingSkyOcclusion = GraphBuilder.CreateUAV(StagingSkyOcclusion);

					TShaderMapRef<FCopyConvergedLightmapTilesCS> ComputeShader(GlobalShaderMap);
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("CopyConvergedLightmapTiles"),
						ComputeShader,
						PassParameters,
						FComputeShaderUtils::GetGroupCount(DispatchResolution, FComputeShaderUtils::kGolden2DGroupSize));
				}

				AddEnqueueCopyPass(GraphBuilder, LightmapReadbackGroup.StagingHQLayer0Readback.Get(), StagingHQLayer0);
				AddEnqueueCopyPass(GraphBuilder, LightmapReadbackGroup.StagingHQLayer1Readback.Get(), StagingHQLayer1);
				AddEnqueueCopyPass(GraphBuilder, LightmapReadbackGroup.StagingShadowMaskReadback.Get(), StagingShadowMask);
				AddEnqueueCopyPass(GraphBuilder, LightmapReadbackGroup.StagingSkyOcclusionReadback.Get(), StagingSkyOcclusion);

				OngoingReadbacks.Emplace(ReadbackGroupToUse);
			}
		}
	}

	PendingTileRequests.Empty();

	FrameNumber++;
}

const int32 DenoiseTileProximity = 3;

void FLightmapTileDenoiseAsyncTask::DoThreadedWork()
{
	TArray<FVector3f> SkyBentNormal;
	SkyBentNormal.AddZeroed(Size.X * Size.Y);
		
	for (int32 Y = 0 ; Y < Size.Y; Y++)
	{
		for (int32 X = 0 ; X < Size.X; X++)
		{
			int32 LinearIndex = Y * Size.X + X;
										
			FLinearColor SkyOcclusion = TextureData->Texture[3][LinearIndex];
										
			// Revert sqrt in LightmapEncoding.ush for preview
			float Length = SkyOcclusion.A * SkyOcclusion.A;
			FVector3f UnpackedBentNormalVector = FVector3f(SkyOcclusion) * 2.0f - 1.0f;
			SkyBentNormal[LinearIndex] = UnpackedBentNormalVector * Length;
		}
	}
	
	if (Denoiser == EGPULightmassDenoiser::SimpleFireflyRemover)
	{
		TLightSampleDataProvider<FLinearColor> GISampleData(Size, TextureData->Texture[0], TextureData->Texture[1]);
		SimpleFireflyFilter(GISampleData);

		TLightSampleDataProvider<FVector3f> SkyBentNormalSampleData(Size, TextureData->Texture[0], SkyBentNormal);
		SimpleFireflyFilter(SkyBentNormalSampleData);
	}
	else
	{
		static FDenoiserContext DenoiserContext;
		
		DenoiseRawData(
			Size,
			TextureData->Texture[0],
			TextureData->Texture[1],
			DenoiserContext);

		DenoiseSkyBentNormal(Size, TextureData->Texture[0], SkyBentNormal, DenoiserContext);
	}
		
	for (int32 Y = 0 ; Y < Size.Y; Y++)
	{
		for (int32 X = 0 ; X < Size.X; X++)
		{
			int32 LinearIndex = Y * Size.X + X;
				
			float Length = SkyBentNormal[LinearIndex].Length();
			FVector3f PackedVector = SkyBentNormal[LinearIndex].GetSafeNormal() * 0.5f + 0.5f;
				
			TextureData->Texture[3][LinearIndex].R = PackedVector.X;
			TextureData->Texture[3][LinearIndex].G = PackedVector.Y;
			TextureData->Texture[3][LinearIndex].B = PackedVector.Z;
			TextureData->Texture[3][LinearIndex].A = FMath::Sqrt(Length);
		}
	}

	FPlatformAtomics::AtomicStore(&TextureData->bDenoisingFinished, 1);
}

bool CompareMostSignificantBit(uint32 A, uint32 B)
{
	return A < B && A < (A ^ B);
}

bool MortonCompare(FUintVector A, FUintVector B)
{
	int32 MostSignificantDim = 0;
	for (int32 Dim = 1; Dim < FUintVector::Num(); Dim++)
	{
		if (CompareMostSignificantBit(A[MostSignificantDim] ^ B[MostSignificantDim], A[Dim] ^ B[Dim]))
		{
			MostSignificantDim = Dim;
		}
	}

	return A[MostSignificantDim] < B[MostSignificantDim];
}

void FSceneRenderState::BuildMortonSortedLightmapRefList()
{
	FBox SceneBounds(ForceInit);

	for (FLightmapRenderState& Lightmap : LightmapRenderStates.Elements)
	{
		SceneBounds += Lightmap.GeometryInstanceRef.GetOrigin();
	}

	for (FLightmapRenderState& Lightmap : LightmapRenderStates.Elements)
	{
		MortonSortedLightmapRefList.Add(FLightmapRenderStateRef(Lightmap, LightmapRenderStates));
	}

	if (SceneBounds.GetSize().GetMax() > UE_DOUBLE_SMALL_NUMBER)
	{
		MortonSortedLightmapRefList.Sort(
			[SceneBounds](const FLightmapRenderStateRef& A, const FLightmapRenderStateRef& B)
			{
				const FUintVector QuantizedPosA = FUintVector((A->GeometryInstanceRef.GetOrigin() - SceneBounds.Min) / SceneBounds.GetSize().GetMax() * UINT_MAX);
				const FUintVector QuantizedPosB = FUintVector((B->GeometryInstanceRef.GetOrigin() - SceneBounds.Min) / SceneBounds.GetSize().GetMax() * UINT_MAX);
				return MortonCompare(QuantizedPosA, QuantizedPosB);
			});
	}
}

void FLightmapRenderer::BackgroundTick(FRHICommandListImmediate& RHICmdList)
{
	{
		TArray<FLightmapTileDenoiseGroup> FilteredDenoiseGroups;

		FTileDataLayer::Evict();

		for (FLightmapTileDenoiseGroup& DenoiseGroup : OngoingDenoiseGroups)
		{
			bool bPipelineFinished = false;

			if (DenoiseGroup.Revision != CurrentRevision)
			{
				bPipelineFinished = true;
				continue;
			}

			if (DenoiseGroup.bShouldBeCancelled)
			{
				if (DenoisingThreadPool->RetractQueuedWork(DenoiseGroup.AsyncDenoisingWork))
				{
					delete DenoiseGroup.AsyncDenoisingWork;
					bPipelineFinished = true;
					continue;
				}
				else
				{
					// Failed to cancel async work, proceed as usual
					DenoiseGroup.bShouldBeCancelled = false;
				}
			}

			if (FPlatformAtomics::AtomicRead(&DenoiseGroup.TextureData->bDenoisingFinished) == 1)
			{
				FLightmapTileRequest& Tile = DenoiseGroup.TileRequest;

				FIntPoint SrcTilePosition(DenoiseTileProximity / 2, DenoiseTileProximity / 2);
				FIntPoint DstTilePosition(Tile.VirtualCoordinates.Position.X, Tile.VirtualCoordinates.Position.Y);

				const int32 DstRowPitchInPixels = GPreviewLightmapVirtualTileSize;
				const int32 SrcRowPitchInPixels = DenoiseTileProximity * GPreviewLightmapVirtualTileSize;

				// While the data will be overwritten immediately, we still need to decompress to inform the LRU cache management
				Tile.RenderState->TileStorage[Tile.VirtualCoordinates].CPUTextureData[0]->Decompress();
				Tile.RenderState->TileStorage[Tile.VirtualCoordinates].CPUTextureData[1]->Decompress();
				Tile.RenderState->TileStorage[Tile.VirtualCoordinates].CPUTextureData[2]->Decompress();
				Tile.RenderState->TileStorage[Tile.VirtualCoordinates].CPUTextureData[3]->Decompress();

				for (int32 Y = 0; Y < GPreviewLightmapVirtualTileSize; Y++)
				{
					for (int32 X = 0; X < GPreviewLightmapVirtualTileSize; X++)
					{
						FIntPoint SrcPixelPosition = SrcTilePosition * GPreviewLightmapVirtualTileSize + FIntPoint(X, Y);
						FIntPoint DstPixelPosition = FIntPoint(X, Y);

						int32 SrcLinearIndex = SrcPixelPosition.Y * SrcRowPitchInPixels + SrcPixelPosition.X;
						int32 DstLinearIndex = DstPixelPosition.Y * DstRowPitchInPixels + DstPixelPosition.X;

						Tile.RenderState->TileStorage[Tile.VirtualCoordinates].CPUTextureData[0]->Data[DstLinearIndex] = DenoiseGroup.TextureData->Texture[0][SrcLinearIndex];
						Tile.RenderState->TileStorage[Tile.VirtualCoordinates].CPUTextureData[1]->Data[DstLinearIndex] = DenoiseGroup.TextureData->Texture[1][SrcLinearIndex];
						Tile.RenderState->TileStorage[Tile.VirtualCoordinates].CPUTextureData[2]->Data[DstLinearIndex] = DenoiseGroup.TextureData->Texture[2][SrcLinearIndex];
						Tile.RenderState->TileStorage[Tile.VirtualCoordinates].CPUTextureData[3]->Data[DstLinearIndex] = DenoiseGroup.TextureData->Texture[3][SrcLinearIndex];
					}
				}

				DenoiseGroup.TileRequest.RenderState->RetrieveTileState(DenoiseGroup.TileRequest.VirtualCoordinates).CPURevision = CurrentRevision;
				DenoiseGroup.TileRequest.RenderState->RetrieveTileState(DenoiseGroup.TileRequest.VirtualCoordinates).OngoingReadbackRevision = -1;

				delete DenoiseGroup.AsyncDenoisingWork;

				bPipelineFinished = true;
			}

			if (!bPipelineFinished)
			{
				FilteredDenoiseGroups.Emplace(MoveTemp(DenoiseGroup));
			}
		}

		OngoingDenoiseGroups = MoveTemp(FilteredDenoiseGroups);
	}

	TArray<FLightmapReadbackGroup*> FilteredReadbackGroups;

	TArray<FLightmapTileRequest> TilesWaitingForDenoising;

	FTileDataLayer::Evict();

	for (int32 Index = 0; Index < OngoingReadbacks.Num(); Index++)
	{
		FLightmapReadbackGroup& ReadbackGroup = *OngoingReadbacks[Index];

		if (ReadbackGroup.Revision != CurrentRevision)
		{
			continue;
		}

		bool bPipelineFinished = false;

		if (ReadbackGroup.StagingHQLayer0Readback->IsReady(FRHIGPUMask::FromIndex(ReadbackGroup.GPUIndex)) &&
			ReadbackGroup.StagingHQLayer1Readback->IsReady(FRHIGPUMask::FromIndex(ReadbackGroup.GPUIndex)) &&
			ReadbackGroup.StagingShadowMaskReadback->IsReady(FRHIGPUMask::FromIndex(ReadbackGroup.GPUIndex)) &&
			ReadbackGroup.StagingSkyOcclusionReadback->IsReady(FRHIGPUMask::FromIndex(ReadbackGroup.GPUIndex)))
		{
			SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::FromIndex(ReadbackGroup.GPUIndex));

			ReadbackGroup.TextureData = MakeUnique<FLightmapReadbackGroup::FTextureData>();

			ReadbackGroup.TextureData->SizeInTiles = ReadbackGroup.ReadbackTilePoolGPU->SizeInTiles;

			// FLinearColor is in RGBA while the GPU texture is in ABGR
			// TODO: apply swizzling in the copy compute shader if this becomes a problem
			void* LockedData[4];
			LockedData[0] = ReadbackGroup.StagingHQLayer0Readback->Lock(ReadbackGroup.TextureData->RowPitchInPixels[0]); // This forces a GPU stall
			LockedData[1] = ReadbackGroup.StagingHQLayer1Readback->Lock(ReadbackGroup.TextureData->RowPitchInPixels[1]); // This forces a GPU stall
			LockedData[2] = ReadbackGroup.StagingShadowMaskReadback->Lock(ReadbackGroup.TextureData->RowPitchInPixels[2]); // This forces a GPU stall
			LockedData[3] = ReadbackGroup.StagingSkyOcclusionReadback->Lock(ReadbackGroup.TextureData->RowPitchInPixels[3]); // This forces a GPU stall

			ReadbackGroup.TextureData->Texture[0].AddUninitialized(ReadbackGroup.ReadbackTilePoolGPU->SizeInTiles.Y * GPreviewLightmapPhysicalTileSize * ReadbackGroup.TextureData->RowPitchInPixels[0]);
			ReadbackGroup.TextureData->Texture[1].AddUninitialized(ReadbackGroup.ReadbackTilePoolGPU->SizeInTiles.Y * GPreviewLightmapPhysicalTileSize * ReadbackGroup.TextureData->RowPitchInPixels[1]);
			ReadbackGroup.TextureData->Texture[2].AddUninitialized(ReadbackGroup.ReadbackTilePoolGPU->SizeInTiles.Y * GPreviewLightmapPhysicalTileSize * ReadbackGroup.TextureData->RowPitchInPixels[2]);
			ReadbackGroup.TextureData->Texture[3].AddUninitialized(ReadbackGroup.ReadbackTilePoolGPU->SizeInTiles.Y * GPreviewLightmapPhysicalTileSize * ReadbackGroup.TextureData->RowPitchInPixels[3]);
			FMemory::Memcpy(ReadbackGroup.TextureData->Texture[0].GetData(), LockedData[0], ReadbackGroup.ReadbackTilePoolGPU->SizeInTiles.Y * GPreviewLightmapPhysicalTileSize * ReadbackGroup.TextureData->RowPitchInPixels[0] * sizeof(FLinearColor));
			FMemory::Memcpy(ReadbackGroup.TextureData->Texture[1].GetData(), LockedData[1], ReadbackGroup.ReadbackTilePoolGPU->SizeInTiles.Y * GPreviewLightmapPhysicalTileSize * ReadbackGroup.TextureData->RowPitchInPixels[1] * sizeof(FLinearColor));
			FMemory::Memcpy(ReadbackGroup.TextureData->Texture[2].GetData(), LockedData[2], ReadbackGroup.ReadbackTilePoolGPU->SizeInTiles.Y * GPreviewLightmapPhysicalTileSize * ReadbackGroup.TextureData->RowPitchInPixels[2] * sizeof(FLinearColor));
			FMemory::Memcpy(ReadbackGroup.TextureData->Texture[3].GetData(), LockedData[3], ReadbackGroup.ReadbackTilePoolGPU->SizeInTiles.Y * GPreviewLightmapPhysicalTileSize * ReadbackGroup.TextureData->RowPitchInPixels[3] * sizeof(FLinearColor));

			ReadbackGroup.StagingHQLayer0Readback->Unlock();
			ReadbackGroup.StagingHQLayer1Readback->Unlock();
			ReadbackGroup.StagingShadowMaskReadback->Unlock();
			ReadbackGroup.StagingSkyOcclusionReadback->Unlock();

			for (int32 TileIndex = 0; TileIndex < ReadbackGroup.ConvergedTileRequests.Num(); TileIndex++)
			{
				FIntPoint SrcTilePosition(TileIndex % ReadbackGroup.ReadbackTilePoolGPU->SizeInTiles.X, TileIndex / ReadbackGroup.ReadbackTilePoolGPU->SizeInTiles.X);
				FIntPoint DstTilePosition(ReadbackGroup.ConvergedTileRequests[TileIndex].VirtualCoordinates.Position);

				check(ReadbackGroup.TextureData->RowPitchInPixels[0] == ReadbackGroup.TextureData->RowPitchInPixels[1]);
				const int32 SrcRowPitchInPixels = ReadbackGroup.TextureData->RowPitchInPixels[0];
				const int32 DstRowPitchInPixels = GPreviewLightmapVirtualTileSize;

				if (!ReadbackGroup.ConvergedTileRequests[TileIndex].RenderState->TileStorage.Contains(ReadbackGroup.ConvergedTileRequests[TileIndex].VirtualCoordinates))
				{
					ReadbackGroup.ConvergedTileRequests[TileIndex].RenderState->TileStorage.Add(ReadbackGroup.ConvergedTileRequests[TileIndex].VirtualCoordinates, FTileStorage{});
				}

				FTileStorage& TileStorage = ReadbackGroup.ConvergedTileRequests[TileIndex].RenderState->TileStorage[ReadbackGroup.ConvergedTileRequests[TileIndex].VirtualCoordinates];

				if (bDenoiseDuringInteractiveBake)
				{
					TileStorage.CPUTextureRawData[0]->Decompress();
					TileStorage.CPUTextureRawData[1]->Decompress();
					TileStorage.CPUTextureRawData[2]->Decompress();
					TileStorage.CPUTextureRawData[3]->Decompress();
				}

				TileStorage.CPUTextureData[0]->Decompress();
				TileStorage.CPUTextureData[1]->Decompress();
				TileStorage.CPUTextureData[2]->Decompress();
				TileStorage.CPUTextureData[3]->Decompress();

				for (int32 Y = 0; Y < GPreviewLightmapVirtualTileSize; Y++)
				{
					for (int32 X = 0; X < GPreviewLightmapVirtualTileSize; X++)
					{
						FIntPoint SrcPixelPosition = SrcTilePosition * GPreviewLightmapPhysicalTileSize + FIntPoint(X, Y) + FIntPoint(GPreviewLightmapTileBorderSize, GPreviewLightmapTileBorderSize);
						FIntPoint DstPixelPosition = FIntPoint(X, Y);

						int32 SrcLinearIndex = SrcPixelPosition.Y * SrcRowPitchInPixels + SrcPixelPosition.X;
						int32 DstLinearIndex = DstPixelPosition.Y * DstRowPitchInPixels + DstPixelPosition.X;

						if (bDenoiseDuringInteractiveBake)
						{
							TileStorage.CPUTextureRawData[0]->Data[DstLinearIndex] = ReadbackGroup.TextureData->Texture[0][SrcLinearIndex];
							TileStorage.CPUTextureRawData[1]->Data[DstLinearIndex] = ReadbackGroup.TextureData->Texture[1][SrcLinearIndex];
							TileStorage.CPUTextureRawData[2]->Data[DstLinearIndex] = ReadbackGroup.TextureData->Texture[2][SrcLinearIndex];
							TileStorage.CPUTextureRawData[3]->Data[DstLinearIndex] = ReadbackGroup.TextureData->Texture[3][SrcLinearIndex];
						}

						// Always write into display data so we have something to show before denoising completes
						TileStorage.CPUTextureData[0]->Data[DstLinearIndex] = ReadbackGroup.TextureData->Texture[0][SrcLinearIndex];
						TileStorage.CPUTextureData[1]->Data[DstLinearIndex] = ReadbackGroup.TextureData->Texture[1][SrcLinearIndex];

						// For shadow maps, pass through
						TileStorage.CPUTextureData[2]->Data[DstLinearIndex] = ReadbackGroup.TextureData->Texture[2][SrcLinearIndex];
						TileStorage.CPUTextureData[3]->Data[DstLinearIndex] = ReadbackGroup.TextureData->Texture[3][SrcLinearIndex];
					}
				}
			}

			for (FLightmapTileRequest& Tile : ReadbackGroup.ConvergedTileRequests)
			{
				Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).bCanBeDenoised = true;

				if (!bDenoiseDuringInteractiveBake)
				{
					Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).CPURevision = CurrentRevision;
					Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).OngoingReadbackRevision = -1;
				}
				else
				{
					TilesWaitingForDenoising.Add(Tile);

					for (int Dx = -(DenoiseTileProximity / 2); Dx <= (DenoiseTileProximity / 2); Dx++)
					{
						for (int Dy = -(DenoiseTileProximity / 2); Dy <= (DenoiseTileProximity / 2); Dy++)
						{
							FIntPoint TilePositionToLookAt(Tile.VirtualCoordinates.Position.X + Dx, Tile.VirtualCoordinates.Position.Y + Dy);
							TilePositionToLookAt.X = FMath::Clamp(TilePositionToLookAt.X, 0, Tile.RenderState->GetPaddedSizeInTilesAtMipLevel(Tile.VirtualCoordinates.MipLevel).X - 1);
							TilePositionToLookAt.Y = FMath::Clamp(TilePositionToLookAt.Y, 0, Tile.RenderState->GetPaddedSizeInTilesAtMipLevel(Tile.VirtualCoordinates.MipLevel).Y - 1);

							if (Tile.RenderState->RetrieveTileState(FTileVirtualCoordinates(TilePositionToLookAt, Tile.VirtualCoordinates.MipLevel)).bWasDenoisedWithoutProximity)
							{
								FLightmapTileRequest TileToDenoise(Tile.RenderState, FTileVirtualCoordinates(TilePositionToLookAt, Tile.VirtualCoordinates.MipLevel));

								TilesWaitingForDenoising.Add(TileToDenoise);

								Tile.RenderState->RetrieveTileState(TileToDenoise.VirtualCoordinates).CPURevision = -1;
								Tile.RenderState->RetrieveTileState(TileToDenoise.VirtualCoordinates).OngoingReadbackRevision = CurrentRevision;
							}
						}
					}
				}
			}
			
			ReadbackGroup.bIsFree = true;

			bPipelineFinished = true;
		}

		if (!bPipelineFinished)
		{
			FilteredReadbackGroups.Emplace(OngoingReadbacks[Index]);
		}
	}

	OngoingReadbacks = MoveTemp(FilteredReadbackGroups);

	{
		int32 NumFreeReadbackGroups = 0;

		for (int32 Index = 0; Index < RecycledReadbacks.Num(); Index++)
		{
			if (RecycledReadbacks[Index]->bIsFree)
			{
				NumFreeReadbackGroups++;
			}
		}

		const int32 MaxPooledFreeReadbackGroups = 100;
		int32 FreeReadbackGroupsToRemove = NumFreeReadbackGroups - MaxPooledFreeReadbackGroups;
		if (FreeReadbackGroupsToRemove > 0)
		{
			for (int32 Index = 0; Index < RecycledReadbacks.Num(); Index++)
			{
				if (RecycledReadbacks[Index]->bIsFree)
				{
					RecycledReadbacks.RemoveAt(Index);
					Index--;
					FreeReadbackGroupsToRemove--;

					if (FreeReadbackGroupsToRemove == 0) break;
				}
			}
		}
	}

	FTileDataLayer::Evict();

	{
		for (FLightmapTileRequest& Tile : TilesWaitingForDenoising)
		{
			auto AllTilesInProximityDenoised = [&Lightmap = Tile.RenderState, &PendingTileRequests = PendingTileRequests, &Tile = Tile](FTileVirtualCoordinates Coords) -> bool
			{
				bool bAll3x3TilesHaveBeenReadback = true;

				for (int Dx = -(DenoiseTileProximity / 2); Dx <= (DenoiseTileProximity / 2); Dx++)
				{
					for (int Dy = -(DenoiseTileProximity / 2); Dy <= (DenoiseTileProximity / 2); Dy++)
					{
						FIntPoint TilePositionToLookAt(Coords.Position.X + Dx, Coords.Position.Y + Dy);
						TilePositionToLookAt.X = FMath::Clamp(TilePositionToLookAt.X, 0, Lightmap->GetPaddedSizeInTilesAtMipLevel(Coords.MipLevel).X - 1);
						TilePositionToLookAt.Y = FMath::Clamp(TilePositionToLookAt.Y, 0, Lightmap->GetPaddedSizeInTilesAtMipLevel(Coords.MipLevel).Y - 1);

						if (!Lightmap->RetrieveTileState(FTileVirtualCoordinates(TilePositionToLookAt, Coords.MipLevel)).bCanBeDenoised)
						{
							bAll3x3TilesHaveBeenReadback = false;

							break;
						}
					}
				}

				return bAll3x3TilesHaveBeenReadback;
			};

			for (FLightmapTileDenoiseGroup& DenoiseGroup : OngoingDenoiseGroups)
			{
				if (DenoiseGroup.TileRequest == Tile)
				{
					DenoiseGroup.bShouldBeCancelled = true;
				}
			}

			FLightmapTileDenoiseGroup DenoiseGroup(Tile);
			DenoiseGroup.Revision = CurrentRevision;
			DenoiseGroup.TextureData = MakeShared<FLightmapTileDenoiseGroup::FTextureData, ESPMode::ThreadSafe>();

			DenoiseGroup.TextureData->Texture[0].AddZeroed(DenoiseTileProximity * DenoiseTileProximity * GPreviewLightmapVirtualTileSize * GPreviewLightmapVirtualTileSize);
			DenoiseGroup.TextureData->Texture[1].AddZeroed(DenoiseTileProximity * DenoiseTileProximity * GPreviewLightmapVirtualTileSize * GPreviewLightmapVirtualTileSize);
			DenoiseGroup.TextureData->Texture[2].AddZeroed(DenoiseTileProximity * DenoiseTileProximity * GPreviewLightmapVirtualTileSize * GPreviewLightmapVirtualTileSize);
			DenoiseGroup.TextureData->Texture[3].AddZeroed(DenoiseTileProximity * DenoiseTileProximity * GPreviewLightmapVirtualTileSize * GPreviewLightmapVirtualTileSize);

			for (int Dx = -(DenoiseTileProximity / 2); Dx <= (DenoiseTileProximity / 2); Dx++)
			{
				for (int Dy = -(DenoiseTileProximity / 2); Dy <= (DenoiseTileProximity / 2); Dy++)
				{
					FIntPoint SrcTilePosition(Tile.VirtualCoordinates.Position.X + Dx, Tile.VirtualCoordinates.Position.Y + Dy);
					SrcTilePosition.X = FMath::Clamp(SrcTilePosition.X, 0, Tile.RenderState->GetPaddedSizeInTilesAtMipLevel(Tile.VirtualCoordinates.MipLevel).X - 1);
					SrcTilePosition.Y = FMath::Clamp(SrcTilePosition.Y, 0, Tile.RenderState->GetPaddedSizeInTilesAtMipLevel(Tile.VirtualCoordinates.MipLevel).Y - 1);
					FIntPoint DstTilePosition(Dx + (DenoiseTileProximity / 2), Dy + (DenoiseTileProximity / 2));

					const int32 SrcRowPitchInPixels = GPreviewLightmapVirtualTileSize;
					const int32 DstRowPitchInPixels = DenoiseTileProximity * GPreviewLightmapVirtualTileSize;

					bool bShouldWriteZero = false;

					if (!Tile.RenderState->RetrieveTileState(FTileVirtualCoordinates(SrcTilePosition, Tile.VirtualCoordinates.MipLevel)).bCanBeDenoised)
					{
						bShouldWriteZero = true;
					}

					if (!bShouldWriteZero)
					{
						Tile.RenderState->TileStorage[FTileVirtualCoordinates(SrcTilePosition, Tile.VirtualCoordinates.MipLevel)].CPUTextureRawData[0]->Decompress();
						Tile.RenderState->TileStorage[FTileVirtualCoordinates(SrcTilePosition, Tile.VirtualCoordinates.MipLevel)].CPUTextureRawData[1]->Decompress();
						Tile.RenderState->TileStorage[FTileVirtualCoordinates(SrcTilePosition, Tile.VirtualCoordinates.MipLevel)].CPUTextureRawData[2]->Decompress();
						Tile.RenderState->TileStorage[FTileVirtualCoordinates(SrcTilePosition, Tile.VirtualCoordinates.MipLevel)].CPUTextureRawData[3]->Decompress();
					}

					for (int32 Y = 0; Y < GPreviewLightmapVirtualTileSize; Y++)
					{
						for (int32 X = 0; X < GPreviewLightmapVirtualTileSize; X++)
						{
							FIntPoint SrcPixelPosition = FIntPoint(X, Y);
							FIntPoint DstPixelPosition = DstTilePosition * GPreviewLightmapVirtualTileSize + FIntPoint(X, Y);

							int32 SrcLinearIndex = SrcPixelPosition.Y * SrcRowPitchInPixels + SrcPixelPosition.X;
							int32 DstLinearIndex = DstPixelPosition.Y * DstRowPitchInPixels + DstPixelPosition.X;

							DenoiseGroup.TextureData->Texture[0][DstLinearIndex] = !bShouldWriteZero ? Tile.RenderState->TileStorage[FTileVirtualCoordinates(SrcTilePosition, Tile.VirtualCoordinates.MipLevel)].CPUTextureRawData[0]->Data[SrcLinearIndex] : FLinearColor(0, 0, 0, 0);
							DenoiseGroup.TextureData->Texture[1][DstLinearIndex] = !bShouldWriteZero ? Tile.RenderState->TileStorage[FTileVirtualCoordinates(SrcTilePosition, Tile.VirtualCoordinates.MipLevel)].CPUTextureRawData[1]->Data[SrcLinearIndex] : FLinearColor(0, 0, 0, 0);
							DenoiseGroup.TextureData->Texture[2][DstLinearIndex] = !bShouldWriteZero ? Tile.RenderState->TileStorage[FTileVirtualCoordinates(SrcTilePosition, Tile.VirtualCoordinates.MipLevel)].CPUTextureRawData[2]->Data[SrcLinearIndex] : FLinearColor(0, 0, 0, 0);
							DenoiseGroup.TextureData->Texture[3][DstLinearIndex] = !bShouldWriteZero ? Tile.RenderState->TileStorage[FTileVirtualCoordinates(SrcTilePosition, Tile.VirtualCoordinates.MipLevel)].CPUTextureRawData[3]->Data[SrcLinearIndex] : FLinearColor(0, 0, 0, 0);
						}
					}
				}
			}

			DenoiseGroup.AsyncDenoisingWork = new FLightmapTileDenoiseAsyncTask();
			DenoiseGroup.AsyncDenoisingWork->Size = FIntPoint(DenoiseTileProximity * GPreviewLightmapVirtualTileSize, DenoiseTileProximity * GPreviewLightmapVirtualTileSize);
			DenoiseGroup.AsyncDenoisingWork->TextureData = DenoiseGroup.TextureData;
			DenoiseGroup.AsyncDenoisingWork->Denoiser = Scene->Settings->Denoiser;
			DenoisingThreadPool->AddQueuedWork(DenoiseGroup.AsyncDenoisingWork);

			OngoingDenoiseGroups.Add(MoveTemp(DenoiseGroup));

			Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).bWasDenoisedWithoutProximity = !AllTilesInProximityDenoised(Tile.VirtualCoordinates);
		}
	}

	const bool bIsViewportNonRealtime = !FGPULightmassModule::IsRealtimeOn();

	if (bIsViewportNonRealtime && !bWasRunningAtFullSpeed)
	{
		bWasRunningAtFullSpeed = true;
		UE_LOGF(LogGPULightmass, Log, "GPULightmass is now running at full speed");
	}

	if (!bIsViewportNonRealtime && bWasRunningAtFullSpeed)
	{
		bWasRunningAtFullSpeed = false;
		UE_LOGF(LogGPULightmass, Log, "GPULightmass is now throttled for realtime preview");
	}

	if (!bOnlyBakeWhatYouSee)
	{
		const int32 NumWorkPerFrame = !bIsViewportNonRealtime ? 32 : 128;

		if (Mip0WorkDoneLastFrame < NumWorkPerFrame)
		{
			int32 WorkToGenerate = NumWorkPerFrame - Mip0WorkDoneLastFrame;
			int32 WorkGenerated = 0;

			TArray<FString> SelectedLightmapNames;

			if (Scene->MortonSortedLightmapRefList.IsEmpty())
			{
				Scene->BuildMortonSortedLightmapRefList();
			}

			// We schedule VLM work to be after lightmaps (which forces LOD 0). Making LOD 0 last here reduces the chance of rebuilding cached scene
			for (int32 LODIndex = MAX_STATIC_MESH_LODS - 1; LODIndex >= 0; LODIndex--)
			{
				bool bLODIndexSelected = false;
				
				for (const FLightmapRenderStateRef& Lightmap : Scene->MortonSortedLightmapRefList)
				{
					if (Lightmap->GeometryInstanceRef.LODIndex != LODIndex)
					{
						continue;
					}

					bool bAnyTileSelected = false;

					for (int32 Y = 0; Y < Lightmap->GetPaddedSizeInTiles().Y; Y++)
					{
						for (int32 X = 0; X < Lightmap->GetPaddedSizeInTiles().X; X++)
						{
							FTileVirtualCoordinates VirtualCoordinates(FIntPoint(X, Y), 0);

							if (!Lightmap->DoesTileHaveValidCPUData(VirtualCoordinates, CurrentRevision) && Lightmap->RetrieveTileState(VirtualCoordinates).OngoingReadbackRevision != CurrentRevision)
							{
								bAnyTileSelected = true;

								FVTProduceTargetLayer TargetLayers[4];

								Lightmap->LightmapPreviewVirtualTexture->ProducePageData(
									RHICmdList,
									Scene->FeatureLevel,
									EVTProducePageFlags::None,
									FVirtualTextureProducerHandle(),
									0b1111,
									0,
									FMath::MortonCode2(X) | (FMath::MortonCode2(Y) << 1),
									0,
									TargetLayers);

								WorkGenerated++;

								if (WorkGenerated >= WorkToGenerate)
								{
									break;
								}
							}
						}

						if (WorkGenerated >= WorkToGenerate)
						{
							break;
						}
					}

					if (bAnyTileSelected)
					{
						bLODIndexSelected = true;
						SelectedLightmapNames.Add(Lightmap->Name);
					}

					if (WorkGenerated >= WorkToGenerate)
					{
						break;
					}
				}

				// Do not mix different LODs together
				if (bLODIndexSelected || WorkGenerated >= WorkToGenerate)
				{
					break;
				}
			}

			if (!SelectedLightmapNames.IsEmpty() && bIsViewportNonRealtime && FrameNumber % 100 == 0)
			{
				FString AllNames;
				for (FString& Name : SelectedLightmapNames)
				{
					AllNames += Name.RightChop(FString(TEXT("Lightmap_")).Len()) + TEXT(" ");
				}
				UE_LOGF(LogGPULightmass, Log, "Working on: %ls", *AllNames);
			}
		}

		Mip0WorkDoneLastFrame = 0;
	}

	if (bOnlyBakeWhatYouSee)
	{
		if (bIsViewportNonRealtime)
		{
			int32 WorkGenerated = 0;

			const int32 WorkToGenerate = 512;

			if (RecordedTileRequests.Num() > 0)
			{
				for (int32 LODIndex = MAX_STATIC_MESH_LODS - 1; LODIndex >= 0; LODIndex--)
				{
					for (FLightmapTileRequest& Tile : RecordedTileRequests)
					{
						if (Tile.RenderState->GeometryInstanceRef.LODIndex != LODIndex)
						{
							continue;
						}

						if (!Tile.RenderState->DoesTileHaveValidCPUData(Tile.VirtualCoordinates, CurrentRevision) && Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).OngoingReadbackRevision != CurrentRevision)
						{
							PendingTileRequests.AddUnique(Tile);

							WorkGenerated++;

							if (WorkGenerated >= WorkToGenerate)
							{
								break;
							}
						}
					}

					if (WorkGenerated >= WorkToGenerate)
					{
						break;
					}
				}
			}
			else
			{
				for (int32 LODIndex = MAX_STATIC_MESH_LODS - 1; LODIndex >= 0; LODIndex--)
				{
					for (TArray<FLightmapTileRequest>& FrameRequests : TilesVisibleLastFewFrames)
					{
						for (FLightmapTileRequest& Tile : FrameRequests)
						{
							if (Tile.RenderState->GeometryInstanceRef.LODIndex != LODIndex)
							{
								continue;
							}

							if (!Tile.RenderState->DoesTileHaveValidCPUData(Tile.VirtualCoordinates, CurrentRevision) && Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).OngoingReadbackRevision != CurrentRevision)
							{
								PendingTileRequests.AddUnique(Tile);

								WorkGenerated++;

								if (WorkGenerated >= WorkToGenerate)
								{
									break;
								}
							}
						}
					}

					if (WorkGenerated >= WorkToGenerate)
					{
						break;
					}
				}
			}
		}
	}

	bInsideBackgroundTick = true;

	// Render lightmap tiles
	{
		FRDGBuilder GraphBuilder(RHICmdList);
		Finalize(GraphBuilder);
		GraphBuilder.Execute();
	}

	bInsideBackgroundTick = false;

	if (bIsViewportNonRealtime) // Indicates that the viewport is non-realtime
	{
		// Purge resources when 'realtime' is not checked on editor viewport to avoid leak & slowing down
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
	}
}

void FLightmapRenderer::BumpRevision()
{
	FrameNumber = 0;
	CurrentRevision++;

	for (TArray<FLightmapTileRequest>& FrameRequests : TilesVisibleLastFewFrames)
	{
		FrameRequests.Empty();
	}	

	PendingTileRequests.Empty();
	RecordedTileRequests.Empty();

	LightmapTilePoolGPU.UnmapAll();

	Scene->MortonSortedLightmapRefList.Empty();
}

void FLightmapRenderer::DeduplicateRecordedTileRequests()
{
	RecordedTileRequests.Sort([](const FLightmapTileRequest& A, const FLightmapTileRequest& B) { return A.VirtualCoordinates.MipLevel > B.VirtualCoordinates.MipLevel; });

	int32 OldNum = RecordedTileRequests.Num();

	for (int32 Index = 0; Index < RecordedTileRequests.Num(); Index++)
	{
		FLightmapTileRequest& Tile = RecordedTileRequests[Index];
		if (
			Tile.VirtualCoordinates.MipLevel > 0 &&
			RecordedTileRequests.FindByPredicate([&Tile](const FLightmapTileRequest& Entry) {
			return Entry.VirtualCoordinates.MipLevel == Tile.VirtualCoordinates.MipLevel - 1
				&& Entry.RenderState == Tile.RenderState
				&& (Entry.VirtualCoordinates.Position.X == Tile.VirtualCoordinates.Position.X * 2 + 0)
				&& (Entry.VirtualCoordinates.Position.Y == Tile.VirtualCoordinates.Position.Y * 2 + 0)
				;
		}) &&
			RecordedTileRequests.FindByPredicate([&Tile](const FLightmapTileRequest& Entry) {
			return Entry.VirtualCoordinates.MipLevel == Tile.VirtualCoordinates.MipLevel - 1
				&& Entry.RenderState == Tile.RenderState
				&& (Entry.VirtualCoordinates.Position.X == Tile.VirtualCoordinates.Position.X * 2 + 0)
				&& (Entry.VirtualCoordinates.Position.Y == FMath::Min(Tile.VirtualCoordinates.Position.Y * 2 + 1, Tile.RenderState->GetPaddedSizeInTilesAtMipLevel(Tile.VirtualCoordinates.MipLevel - 1).Y - 1))
				;
		}) &&
			RecordedTileRequests.FindByPredicate([&Tile](const FLightmapTileRequest& Entry) {
			return Entry.VirtualCoordinates.MipLevel == Tile.VirtualCoordinates.MipLevel - 1
				&& Entry.RenderState == Tile.RenderState
				&& (Entry.VirtualCoordinates.Position.X == FMath::Min(Tile.VirtualCoordinates.Position.X * 2 + 1, Tile.RenderState->GetPaddedSizeInTilesAtMipLevel(Tile.VirtualCoordinates.MipLevel - 1).X - 1))
				&& (Entry.VirtualCoordinates.Position.Y == Tile.VirtualCoordinates.Position.Y * 2 + 0)
				;
		}) &&
			RecordedTileRequests.FindByPredicate([&Tile](const FLightmapTileRequest& Entry) {
			return Entry.VirtualCoordinates.MipLevel == Tile.VirtualCoordinates.MipLevel - 1
				&& Entry.RenderState == Tile.RenderState
				&& (Entry.VirtualCoordinates.Position.X == FMath::Min(Tile.VirtualCoordinates.Position.X * 2 + 1, Tile.RenderState->GetPaddedSizeInTilesAtMipLevel(Tile.VirtualCoordinates.MipLevel - 1).X - 1))
				&& (Entry.VirtualCoordinates.Position.Y == FMath::Min(Tile.VirtualCoordinates.Position.Y * 2 + 1, Tile.RenderState->GetPaddedSizeInTilesAtMipLevel(Tile.VirtualCoordinates.MipLevel - 1).Y - 1))
				;
		})
			)
		{
			RecordedTileRequests.RemoveAt(Index);
			Index--;
		}
	}

	UE_LOGF(LogGPULightmass, Log, "Tile deduplication removed %d tiles", OldNum - RecordedTileRequests.Num());
}

void FLightmapRenderer::RenderIrradianceCacheVisualization(FPostOpaqueRenderParameters& Parameters)
{
	if (!Scene->Settings->bVisualizeIrradianceCache)
	{
		return;
	}
	
	FRDGBuilder& GraphBuilder = *Parameters.GraphBuilder;
	const ERHIFeatureLevel::Type FeatureLevel = Scene->FeatureLevel;

	auto* PassParameters = GraphBuilder.AllocParameters<FVisualizeIrradianceCachePS::FParameters>();
	PassParameters->View = Parameters.View->ViewUniformBuffer;
	PassParameters->SceneTextures = Parameters.SceneTexturesUniformParams;
	PassParameters->IrradianceCachingParameters = Scene->IrradianceCache->IrradianceCachingParametersUniformBuffer;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(Parameters.ColorTexture, ERenderTargetLoadAction::ELoad);

	const FIntRect ViewportRect = Parameters.ViewportRect;
	const FIntPoint TextureExtent = Parameters.ColorTexture->Desc.Extent;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ClearIrradiance"),
		PassParameters,
		ERDGPassFlags::Raster,
		[ViewportRect, PassParameters, TextureExtent, FeatureLevel] (FRHICommandList& RHICmdList)
	{
		RHICmdList.SetViewport(0, 0, 0.0f, ViewportRect.Width(), ViewportRect.Height(), 1.0f);

		TShaderMapRef<FPostProcessVS> VertexShader(GetGlobalShaderMap(FeatureLevel));
		TShaderMapRef<FVisualizeIrradianceCachePS> PixelShader(GetGlobalShaderMap(FeatureLevel));

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

		UE::Renderer::PostProcess::DrawRectangle(
			RHICmdList,
			VertexShader,
			0, 0,
			ViewportRect.Width(), ViewportRect.Height(),
			0, 0,
			ViewportRect.Width(), ViewportRect.Height(),
			FIntPoint(ViewportRect.Width(), ViewportRect.Height()),
			TextureExtent
		);
	});
}

}
