// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "GPULightmassSettings.h"
#include "LightmapTilePool.h"
#include "Misc/QueuedThreadPool.h"
#include "RHIGPUReadback.h"
// TBB suffers from extreme fragmentation problem in editor
#include "Runtime/Core/Private/HAL/Allocators/AnsiAllocator.h"
#include "VirtualTexturing.h"

struct IPooledRenderTarget;

namespace GPULightmass
{

class FSceneRenderState;

struct FLightmapTileRequest
{
	FLightmapRenderStateRef RenderState;
	FTileVirtualCoordinates VirtualCoordinates;

	FLightmapTileRequest(
		FLightmapRenderStateRef RenderState,
		FTileVirtualCoordinates VirtualCoordinates)
		: RenderState(RenderState)
		, VirtualCoordinates(VirtualCoordinates)
	{}

	~FLightmapTileRequest() {} // this deletes the move constructor while keeps copy constructors

	FIntPoint OutputPhysicalCoordinates[8];
	IPooledRenderTarget* OutputRenderTargets[8] = { nullptr, nullptr, nullptr, nullptr, nullptr, nullptr, nullptr, nullptr };
	uint32 TileAddressInWorkingSet = ~0u;
	uint32 TileAddressInScratch = ~0u;

	bool operator==(const FLightmapTileRequest& Rhs) const
	{
		return RenderState == Rhs.RenderState && VirtualCoordinates == Rhs.VirtualCoordinates;
	}

	bool IsScreenOutputTile() const
	{
		for (IPooledRenderTarget* RenderTarget : OutputRenderTargets)
		{
			if (RenderTarget != nullptr)
			{
				return true;
			}
		}

		return false;
	}
};

struct FLightmapReadbackGroup
{
	bool bIsFree = true;
	int32 Revision;
	int32 GPUIndex = 0;
	TUniquePtr<FLightmapTilePoolGPU> ReadbackTilePoolGPU;
	TArray<FLightmapTileRequest> ConvergedTileRequests;
	TUniquePtr<FRHIGPUTextureReadback> StagingHQLayer0Readback;
	TUniquePtr<FRHIGPUTextureReadback> StagingHQLayer1Readback;
	TUniquePtr<FRHIGPUTextureReadback> StagingShadowMaskReadback;
	TUniquePtr<FRHIGPUTextureReadback> StagingSkyOcclusionReadback;

	struct FTextureData
	{
		// Duplicate some metadata so the async thread doesn't need to access FLightmapReadbackGroup
		FIntPoint SizeInTiles;
		int32 RowPitchInPixels[4];
		TArray<FLinearColor> Texture[4];

		volatile int32 bDenoisingFinished = 0;
	};

	TUniquePtr<FTextureData> TextureData;
};

struct FLightmapTileDenoiseGroup
{
	FLightmapTileDenoiseGroup(FLightmapTileRequest& TileRequest) : TileRequest(TileRequest) {}

	int32 Revision;
	FLightmapTileRequest TileRequest;
	bool bShouldBeCancelled = false;

	struct FTextureData
	{
		TArray<FLinearColor> Texture[4];

		volatile int32 bDenoisingFinished = 0;
	};

	TSharedPtr<FTextureData, ESPMode::ThreadSafe> TextureData;
	class FLightmapTileDenoiseAsyncTask* AsyncDenoisingWork = nullptr;
};

class FLightmapTileDenoiseAsyncTask : public IQueuedWork
{
public:
	FIntPoint Size;
	TSharedPtr<FLightmapTileDenoiseGroup::FTextureData, ESPMode::ThreadSafe> TextureData;
	EGPULightmassDenoiser Denoiser;

	virtual void DoThreadedWork();
	virtual void Abandon() {}
};

class FLightmapRenderer : public IVirtualTextureFinalizer
{
public:
	FLightmapRenderer(FRHICommandList& RHICmdList, FSceneRenderState* InScene);

	void AddRequest(FLightmapTileRequest TileRequest);

	void RenderMeshBatchesIntoGBuffer(
		FRHICommandList& RHICmdList,
		const FSceneView* View,
		int32 GPUScenePrimitiveId,
		TArray<FMeshBatch> MeshBatches,
		FVector4f VirtualTexturePhysicalTileCoordinateScaleAndBias,
		int32 RenderPassIndex,
		FIntPoint ScratchTilePoolOffset);

	virtual void Finalize(FRDGBuilder& GraphBuilder) override;

	virtual ~FLightmapRenderer();

	void BackgroundTick(FRHICommandListImmediate& RHICmdList);

	void BumpRevision();
	int32 GetCurrentRevision() { return CurrentRevision; }

	int32 FrameNumber = 0;

	int32 NumTotalPassesToRender = 0;

	bool bDenoiseDuringInteractiveBake = false;
	bool bOnlyBakeWhatYouSee = false;

	TArray<TArray<FLightmapTileRequest>> TilesVisibleLastFewFrames;

	bool bIsRecordingTileRequests = false;
	TArray<FLightmapTileRequest> RecordedTileRequests;
	void DeduplicateRecordedTileRequests();

private:
	int32 CurrentRevision = 0;
	int32 LastInvalidationFrame = 0;
	int32 Mip0WorkDoneLastFrame = 0;
	bool bIsExiting = false;
	bool bInsideBackgroundTick = false;
	bool bWasRunningAtFullSpeed = true;

	FSceneRenderState* Scene;

	TArray<FLightmapTileRequest> PendingTileRequests;

	FLightmapTilePoolGPU LightmapTilePoolGPU;

	TUniquePtr<FLightmapTilePoolGPU> ScratchTilePoolGPU;

	TArray<FLightmapReadbackGroup*> OngoingReadbacks;
	TArray<TUniquePtr<FLightmapReadbackGroup>> RecycledReadbacks;

	TArray<FLightmapTileDenoiseGroup> OngoingDenoiseGroups;

	TUniquePtr<FLightmapTilePoolGPU> UploadTilePoolGPU;

	FQueuedThreadPool* DenoisingThreadPool;

	void RenderIrradianceCacheVisualization(FPostOpaqueRenderParameters& Parameters);
	FDelegateHandle IrradianceCacheVisualizationDelegateHandle;
};

}
