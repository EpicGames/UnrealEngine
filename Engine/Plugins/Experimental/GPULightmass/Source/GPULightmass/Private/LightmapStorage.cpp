// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightmapStorage.h"
#include "UObject/Package.h"
#include "VT/VirtualTexture.h"
#include "GPULightmassCommon.h"
#include "EngineModule.h"
#include "Async/Async.h"
#include "Scene/Lights.h"
#include "LightmapRenderer.h"
#include "LightmapPreviewVirtualTexture.h"
#include "GPULightmassModule.h"
#include "Async/ParallelFor.h"
#include "Compression/OodleDataCompression.h"
#include "VirtualTexturing.h"

namespace GPULightmass
{

TDoubleLinkedList<FTileDataLayer*> FTileDataLayer::AllUncompressedTiles;

int64 FTileDataLayer::Compress(bool bParallelCompression)
{
	const int32 CompressMemoryBound = FOodleDataCompression::CompressedBufferSizeNeeded(sizeof(FLinearColor) * GPreviewLightmapVirtualTileSize * GPreviewLightmapVirtualTileSize);

	if (Data.Num() > 0)
	{
		CompressedData.Empty();
		CompressedData.AddUninitialized(CompressMemoryBound);
		int32 CompressedSize = CompressMemoryBound;

		CompressedSize = FOodleDataCompression::Compress(
			CompressedData.GetData(), CompressedSize,
			Data.GetData(), sizeof(FLinearColor) * GPreviewLightmapVirtualTileSize * GPreviewLightmapVirtualTileSize,
			FOodleDataCompression::ECompressor::Selkie, FOodleDataCompression::ECompressionLevel::VeryFast);
		
		CompressedData.SetNum(CompressedSize);
		Data.Empty();

		check(bNodeAddedToList);
		if (!bParallelCompression)
		{
			AllUncompressedTiles.RemoveNode(&Node, false);
		}
		bNodeAddedToList = false;

		return sizeof(FLinearColor) * GPreviewLightmapVirtualTileSize * GPreviewLightmapVirtualTileSize - CompressedSize;
	}
	else
	{
		check(CompressedData.Num() > 0);
		return 0;
	}
}

void FTileDataLayer::Decompress()
{
	// LRU by adding to the end
	if (bNodeAddedToList)
	{
		AllUncompressedTiles.RemoveNode(&Node, false);
	}

	AllUncompressedTiles.AddHead(&Node);
	bNodeAddedToList = true;

	if (CompressedData.Num() != 0)
	{
		Data.Empty();
		Data.AddUninitialized(GPreviewLightmapVirtualTileSize * GPreviewLightmapVirtualTileSize);		
		check(FOodleDataCompression::Decompress(
			Data.GetData(), sizeof(FLinearColor) * GPreviewLightmapVirtualTileSize * GPreviewLightmapVirtualTileSize,
			CompressedData.GetData(), CompressedData.Num()));
		CompressedData.Empty();
	}
	else
	{
		check(Data.Num() > 0);
	}
}

void FTileDataLayer::Evict()
{
	int32 OldNum = AllUncompressedTiles.Num();
	double StartTime = FPlatformTime::Seconds();

	TArray<FTileDataLayer*> TileDataLayersToCompress;

	while (AllUncompressedTiles.Num() > 65536) // 4 gig
	{
		FTileDataLayer* TileDataLayerToCompress = AllUncompressedTiles.GetTail()->GetValue();
		AllUncompressedTiles.RemoveNode(AllUncompressedTiles.GetTail(), false);
		TileDataLayersToCompress.Add(TileDataLayerToCompress);
	}

	int64 EvictedMemorySize = 0;

	ParallelFor(TileDataLayersToCompress.Num(), 
		[&TileDataLayersToCompress, &EvictedMemorySize](int32 Index)
	{
		FTileDataLayer* TileDataLayerToCompress = TileDataLayersToCompress[Index];
		FPlatformAtomics::InterlockedAdd(&EvictedMemorySize, TileDataLayerToCompress->Compress(true)); 
	});

	double EndTime = FPlatformTime::Seconds();

	if (OldNum > AllUncompressedTiles.Num() && EndTime - StartTime > 1.0)
	{
		UE_LOGF(LogGPULightmass, Log, "CPU tile management: evicted %d tiles in %ls, released %.2fMB", OldNum - AllUncompressedTiles.Num(), *FPlatformTime::PrettyTime(EndTime - StartTime), EvictedMemorySize / 1024.0f / 1024.0f);
	}
}

FLightmap::FLightmap(FString InName, FIntPoint InSize)
	: Name(InName)
	, Size(InSize)
{
	check(IsInGameThread());
}

void FLightmap::CreateGameThreadResources()
{
	TextureUObject = NewObject<ULightMapVirtualTexture2D>(GetTransientPackage(), FName(*Name));
	TextureUObject->VirtualTextureStreaming = true;
	TextureUObject->bPreviewLightmap = true;
	
	// TODO: add more layers based on layer settings
	{
		TextureUObject->SetLayerForType(ELightMapVirtualTextureType::LightmapLayer0, 0);
		TextureUObject->SetLayerForType(ELightMapVirtualTextureType::LightmapLayer1, 1);
		TextureUObject->SetLayerForType(ELightMapVirtualTextureType::ShadowMask, 2);
		TextureUObject->SetLayerForType(ELightMapVirtualTextureType::SkyOcclusion, 3);
	}

	TextureUObjectGuard = MakeUnique<FGCObjectScopeGuard>(TextureUObject);

	LightmapObject = new FLightMap2D();
	{
		LightmapObject->CoordinateScale.X = (float)(Size.X - 2) / (GetPaddedSizeInTiles().X * GPreviewLightmapVirtualTileSize);
		LightmapObject->CoordinateScale.Y = (float)(Size.Y - 2) / (GetPaddedSizeInTiles().Y * GPreviewLightmapVirtualTileSize);
		LightmapObject->CoordinateBias.X = 1.0f / (GetPaddedSizeInTiles().X * GPreviewLightmapVirtualTileSize);
		LightmapObject->CoordinateBias.Y = 1.0f / (GetPaddedSizeInTiles().Y * GPreviewLightmapVirtualTileSize);

		for (int32 CoefIndex = 0; CoefIndex < NUM_STORED_LIGHTMAP_COEF; CoefIndex++)
		{
			LightmapObject->ScaleVectors[CoefIndex] = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
			LightmapObject->AddVectors[CoefIndex] = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		}
	}
	LightmapObject->VirtualTextures[0] = TextureUObject;

	ResourceCluster = MakeUnique<FLightmapResourceCluster>();
	ResourceCluster->Input.LightMapVirtualTextures[0] = TextureUObject;

	MeshMapBuildData = MakeUnique<FMeshMapBuildData>();
	MeshMapBuildData->LightMap = LightmapObject;
	MeshMapBuildData->ResourceCluster = ResourceCluster.Get();
}

FLightmapRenderState::FLightmapRenderState(Initializer InInitializer, FGeometryInstanceRenderStateRef GeometryInstanceRef)
	: Name(InInitializer.Name)
	, ResourceCluster(InInitializer.ResourceCluster)
	, LightmapCoordinateScaleBias(InInitializer.LightmapCoordinateScaleBias)
	, GeometryInstanceRef(GeometryInstanceRef)
	, Size(InInitializer.Size)
	, MaxLevel(InInitializer.MaxLevel)
{
	for (int32 MipLevel = 0; MipLevel <= MaxLevel; MipLevel++)
	{
		TileStates.AddDefaulted(GetPaddedSizeInTilesAtMipLevel(MipLevel).X * GetPaddedSizeInTilesAtMipLevel(MipLevel).Y);
	}

	for (int32 MipLevel = 0; MipLevel <= MaxLevel; MipLevel++)
	{
		TileRelevantLightSampleCountStates.AddDefaulted(GetPaddedSizeInTilesAtMipLevel(MipLevel).X * GetPaddedSizeInTilesAtMipLevel(MipLevel).Y);
	}
}

bool FLightmapRenderState::IsTileGIConverged(FTileVirtualCoordinates Coords, int32 NumGISamples)
{
	return RetrieveTileState(Coords).RenderPassIndex >= NumGISamples;
}

bool FLightmapRenderState::IsTileShadowConverged(FTileVirtualCoordinates Coords, int32 NumShadowSamples)
{
	bool bConverged = true;
	for (auto& Pair : RetrieveTileRelevantLightSampleState(Coords).RelevantDirectionalLightSampleCount)
	{
		bConverged &= Pair.Value >= NumShadowSamples;
	}
	for (auto& Pair : RetrieveTileRelevantLightSampleState(Coords).RelevantPointLightSampleCount)
	{
		bConverged &= Pair.Value >= NumShadowSamples;
	}
	for (auto& Pair : RetrieveTileRelevantLightSampleState(Coords).RelevantSpotLightSampleCount)
	{
		bConverged &= Pair.Value >= NumShadowSamples;
	}
	for (auto& Pair : RetrieveTileRelevantLightSampleState(Coords).RelevantRectLightSampleCount)
	{
		bConverged &= Pair.Value >= NumShadowSamples;
	}
	return bConverged;
}
bool FLightmapRenderState::DoesTileHaveValidCPUData(FTileVirtualCoordinates Coords, int32 CurrentRevision)
{
	return RetrieveTileState(Coords).CPURevision == CurrentRevision;
}

void CreateLightmapPreviewVirtualTexture(FLightmapRenderStateRef LightmapRenderState, ERHIFeatureLevel::Type FeatureLevel, FLightmapRenderer* LightmapRenderer)
{
	LightmapRenderState->LightmapPreviewVirtualTexture = new FLightmapPreviewVirtualTexture(LightmapRenderState, LightmapRenderer);
	LightmapRenderState->ResourceCluster->AllocatedVT = LightmapRenderState->LightmapPreviewVirtualTexture->AllocatedVT;
	LightmapRenderState->ResourceCluster->InitResource(FRHICommandListImmediate::Get());

	{
		IAllocatedVirtualTexture* AllocatedVT = LightmapRenderState->LightmapPreviewVirtualTexture->AllocatedVT;

		check(AllocatedVT);

		AllocatedVT->GetPackedPageTableUniform(&LightmapRenderState->LightmapVTPackedPageTableUniform[0]);
		uint32 NumLightmapVTLayers = AllocatedVT->GetNumTextureLayers();
		for (uint32 LayerIndex = 0u; LayerIndex < NumLightmapVTLayers; ++LayerIndex)
		{
			AllocatedVT->GetPackedUniform(&LightmapRenderState->LightmapVTPackedUniform[LayerIndex], LayerIndex);
		}
		for (uint32 LayerIndex = NumLightmapVTLayers; LayerIndex < 5u; ++LayerIndex)
		{
			LightmapRenderState->LightmapVTPackedUniform[LayerIndex] = FUintVector4(ForceInitToZero);
		}
	}
				
	LightmapRenderState->ResourceCluster->SetFeatureLevelAndInitialize(FeatureLevel);
	LightmapRenderState->SetResourceCluster(nullptr);
}

void FLightmapRenderState::ReleasePreviewVirtualTexture()
{
	ResourceCluster->ReleaseResource();

	if (LightmapPreviewVirtualTexture != nullptr)
	{
		FVirtualTextureProducerHandle ProducerHandle = LightmapPreviewVirtualTexture->ProducerHandle;
		GetRendererModule().ReleaseVirtualTextureProducer(ProducerHandle);
	}
}

}
