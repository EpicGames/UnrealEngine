// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GPULightmassCommon.h"
#include "Scene/Lights.h"
#include "LightMap.h"
#include "UObject/GCObjectScopeGuard.h"
#include "VT/LightmapVirtualTexture.h"
#include "Engine/MapBuildDataRegistry.h"

namespace GPULightmass
{

struct FTileDataLayer
{
	TArray<FLinearColor> Data;
	TArray<uint8> CompressedData;

	TDoubleLinkedList<FTileDataLayer*>::TDoubleLinkedListNode Node;
	bool bNodeAddedToList = false;

	static TDoubleLinkedList<FTileDataLayer*> AllUncompressedTiles;

	FTileDataLayer() : Node(this)
	{
		Data.AddZeroed(GPreviewLightmapVirtualTileSize * GPreviewLightmapVirtualTileSize);

		AllUncompressedTiles.AddHead(&Node);
		bNodeAddedToList = true;
	}

	~FTileDataLayer()
	{
		if (bNodeAddedToList)
		{
			AllUncompressedTiles.RemoveNode(&Node, false);
		}
	}

	int64 Compress(bool bParallelCompression = false);
	void Decompress();
	void AllocateForWrite();
	static void Evict();
};

struct FTileStorage
{
	TUniquePtr<FTileDataLayer> CPUTextureData[(int32)ELightMapVirtualTextureType::Count];
	TUniquePtr<FTileDataLayer> CPUTextureRawData[(int32)ELightMapVirtualTextureType::Count];

	FTileStorage()
	{
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(CPUTextureData); Index++)
		{
			CPUTextureData[Index] = MakeUnique<FTileDataLayer>();
		}

		for (int32 Index = 0; Index < UE_ARRAY_COUNT(CPUTextureRawData); Index++)
		{
			CPUTextureRawData[Index] = MakeUnique<FTileDataLayer>();
		}
	}
};

class FLightmap
{
public:
	FLightmap(FString InName, FIntPoint InSize);

	void CreateGameThreadResources();

	FIntPoint GetPaddedSizeInTiles() const
	{
		return FIntPoint(
			FMath::DivideAndRoundUp(Size.X, GPreviewLightmapVirtualTileSize),
			FMath::DivideAndRoundUp(Size.Y, GPreviewLightmapVirtualTileSize));
	}
	FIntPoint GetPaddedSize() const { return GetPaddedSizeInTiles() * GPreviewLightmapVirtualTileSize; }

	FString Name;
	FIntPoint Size;

	TUniquePtr<FLightmapResourceCluster> ResourceCluster;
	TUniquePtr<FGCObjectScopeGuard> TextureUObjectGuard = nullptr;
	ULightMapVirtualTexture2D* TextureUObject = nullptr;
	TUniquePtr<FMeshMapBuildData> MeshMapBuildData;
	TRefCountPtr<FLightMap2D> LightmapObject;

	int32 NumStationaryLightsPerShadowChannel[4] = { 0, 0, 0, 0 };
};

using FLightmapRef = TEntityArray<FLightmap>::EntityRefType;

class FLightmapRenderState : public FLightCacheInterface
{
public:
	struct Initializer
	{
		FString Name;
		FIntPoint Size { EForceInit::ForceInitToZero };
		int32 MaxLevel = -1;
		FLightmapResourceCluster* ResourceCluster = nullptr;
		FVector4f LightmapCoordinateScaleBias { EForceInit::ForceInitToZero };

		bool IsValid()
		{
			return Size.X > 0 && Size.Y > 0 && MaxLevel >= 0 && ResourceCluster != nullptr;
		}
	};

	FLightmapRenderState(Initializer InInitializer, FGeometryInstanceRenderStateRef GeometryInstanceRef);

	FIntPoint GetSize() const { return Size; }
	int32 GetMaxLevel() const { return MaxLevel; }
	uint32 GetNumTilesAcrossAllMipmapLevels() const { return TileStates.Num(); }
	FIntPoint GetPaddedSizeInTiles() const
	{
		return FIntPoint(
			FMath::DivideAndRoundUp(Size.X, GPreviewLightmapVirtualTileSize),
			FMath::DivideAndRoundUp(Size.Y, GPreviewLightmapVirtualTileSize));
	}
	FIntPoint GetPaddedSize() const { return GetPaddedSizeInTiles() * GPreviewLightmapVirtualTileSize; }
	FIntPoint GetPaddedPhysicalSize() const { return GetPaddedSizeInTiles() * GPreviewLightmapPhysicalTileSize; }
	FIntPoint GetPaddedSizeAtMipLevel(int32 MipLevel) const { return GetPaddedSizeInTilesAtMipLevel(MipLevel) * GPreviewLightmapVirtualTileSize; }
	FIntPoint GetPaddedSizeInTilesAtMipLevel(int32 MipLevel) const
	{
		return FIntPoint(FMath::DivideAndRoundUp(GetPaddedSizeInTiles().X, 1 << MipLevel), FMath::DivideAndRoundUp(GetPaddedSizeInTiles().Y, 1 << MipLevel));
	}

	struct FTileState
	{
		int32 Revision = -1;
		int32 RenderPassIndex = 0;
		int32 CPURevision = -1;
		int32 OngoingReadbackRevision = -1;
		bool bCanBeDenoised = false;
		bool bWasDenoisedWithoutProximity = false;

		void Invalidate()
		{
			Revision = -1;
			RenderPassIndex = 0;
			InvalidateCPUData();
		}

		void InvalidateCPUData()
		{
			CPURevision = -1;
			bCanBeDenoised = false;
			bWasDenoisedWithoutProximity = false;
			OngoingReadbackRevision = -1;
		}
	};

	bool IsTileCoordinatesValid(FTileVirtualCoordinates Coords)
	{
		if (Coords.MipLevel > MaxLevel)
		{
			return false;
		}

		FIntPoint SizeAtMipLevel(FMath::DivideAndRoundUp(GetPaddedSizeInTiles().X, 1 << Coords.MipLevel), FMath::DivideAndRoundUp(GetPaddedSizeInTiles().Y, 1 << Coords.MipLevel));

		if (Coords.Position.X >= SizeAtMipLevel.X || Coords.Position.Y >= SizeAtMipLevel.Y)
		{
			return false;
		}

		return true;
	}

	FTileState& RetrieveTileState(FTileVirtualCoordinates Coords)
	{
		check(Coords.MipLevel <= MaxLevel);

		int32 MipOffset = 0;
		for (int32 MipLevel = 0; MipLevel < Coords.MipLevel; MipLevel++)
		{
			MipOffset += FMath::DivideAndRoundUp(GetPaddedSizeInTiles().X, 1 << MipLevel) * FMath::DivideAndRoundUp(GetPaddedSizeInTiles().Y, 1 << MipLevel);
		}

		int32 LinearIndex = MipOffset + Coords.Position.Y * FMath::DivideAndRoundUp(GetPaddedSizeInTiles().X, 1 << Coords.MipLevel) + Coords.Position.X;
		return TileStates[LinearIndex];
	}

	uint32 RetrieveTileStateIndex(FTileVirtualCoordinates Coords)
	{
		check(Coords.MipLevel <= MaxLevel);

		int32 MipOffset = 0;
		for (int32 MipLevel = 0; MipLevel < Coords.MipLevel; MipLevel++)
		{
			MipOffset += FMath::DivideAndRoundUp(GetPaddedSizeInTiles().X, 1 << MipLevel) * FMath::DivideAndRoundUp(GetPaddedSizeInTiles().Y, 1 << MipLevel);
		}

		int32 LinearIndex = MipOffset + Coords.Position.Y * FMath::DivideAndRoundUp(GetPaddedSizeInTiles().X, 1 << Coords.MipLevel) + Coords.Position.X;
		return LinearIndex;
	}

	struct FTileRelevantLightSampleCountState
	{
		uint32 RoundRobinIndex = 0;
		TMap<FDirectionalLightRenderStateRef, int32> RelevantDirectionalLightSampleCount;
		TMap<FPointLightRenderStateRef, int32> RelevantPointLightSampleCount;
		TMap<FSpotLightRenderStateRef, int32> RelevantSpotLightSampleCount;
		TMap<FRectLightRenderStateRef, int32> RelevantRectLightSampleCount;
	};

	FTileRelevantLightSampleCountState& RetrieveTileRelevantLightSampleState(FTileVirtualCoordinates Coords)
	{
		check(Coords.MipLevel <= MaxLevel);

		int32 MipOffset = 0;
		for (int32 MipLevel = 0; MipLevel < Coords.MipLevel; MipLevel++)
		{
			MipOffset += FMath::DivideAndRoundUp(GetPaddedSizeInTiles().X, 1 << MipLevel) * FMath::DivideAndRoundUp(GetPaddedSizeInTiles().Y, 1 << MipLevel);
		}

		int32 LinearIndex = MipOffset + Coords.Position.Y * FMath::DivideAndRoundUp(GetPaddedSizeInTiles().X, 1 << Coords.MipLevel) + Coords.Position.X;
		return TileRelevantLightSampleCountStates[LinearIndex];
	}

	bool IsTileGIConverged(FTileVirtualCoordinates Coords, int32 NumGISamples);
	bool IsTileShadowConverged(FTileVirtualCoordinates Coords, int32 NumShadowSamples);
	bool DoesTileHaveValidCPUData(FTileVirtualCoordinates Coords, int32 CurrentRevision);

	FString Name;
	TUniquePtr<FLightmapResourceCluster> ResourceCluster;
	FVector4f LightmapCoordinateScaleBias;
	uint32 DistributionPrefixSum = 0;

	// The virtual texture & producer that handles actual rendering
	// Can't use a unique ptr, because ReleaseVirtualTextureProducer deletes the pointer
	class FLightmapPreviewVirtualTexture* LightmapPreviewVirtualTexture = nullptr;

	// Cached VT uniforms to avoid surprisingly high cost
	FUintVector4 LightmapVTPackedPageTableUniform[2]; // VT (1 page table, 2x uint4)
	FUintVector4 LightmapVTPackedUniform[5]; // VT (5 layers, 1x uint4 per layer)

	TMap<FTileVirtualCoordinates, FTileStorage> TileStorage;

	FGeometryInstanceRenderStateRef GeometryInstanceRef;

	TArray<FPointLightRenderStateRef> RelevantPointLights;
	TArray<FSpotLightRenderStateRef> RelevantSpotLights;
	TArray<FRectLightRenderStateRef> RelevantRectLights;

	void AddRelevantLight(FDirectionalLightRenderStateRef Light)
	{
		// No-op as directional lights are always relevant, so the list is implied
	}

	void RemoveRelevantLight(FDirectionalLightRenderStateRef Light)
	{
		// No-op as directional lights are always relevant, so the list is implied
	}

	void AddRelevantLight(FPointLightRenderStateRef Light)
	{
		RelevantPointLights.Add(Light);
	}

	void RemoveRelevantLight(FPointLightRenderStateRef Light)
	{
		RelevantPointLights.Remove(Light);
	}

	void AddRelevantLight(FSpotLightRenderStateRef Light)
	{
		RelevantSpotLights.Add(Light);
	}

	void RemoveRelevantLight(FSpotLightRenderStateRef Light)
	{
		RelevantSpotLights.Remove(Light);
	}

	void AddRelevantLight(FRectLightRenderStateRef Light)
	{
		RelevantRectLights.Add(Light);
	}

	void RemoveRelevantLight(FRectLightRenderStateRef Light)
	{
		RelevantRectLights.Remove(Light);
	}

	virtual FLightInteraction GetInteraction(const class FLightSceneProxy* LightSceneProxy) const override
	{
		return FLightInteraction::LightMap();
	}

	void ReleasePreviewVirtualTexture();

private:
	FIntPoint Size;
	int32 MaxLevel;
	TArray<FTileState> TileStates;
	TArray<FTileRelevantLightSampleCountState> TileRelevantLightSampleCountStates;
};

using FLightmapRenderStateRef = TEntityArray<FLightmapRenderState>::EntityRefType;

void CreateLightmapPreviewVirtualTexture(FLightmapRenderStateRef LightmapRenderState, ERHIFeatureLevel::Type FeatureLevel, class FLightmapRenderer* LightmapRenderer);

}

static uint32 GetTypeHash(const GPULightmass::FLightmapRenderStateRef& Ref)
{
	return GetTypeHash(Ref.GetElementId());
}
