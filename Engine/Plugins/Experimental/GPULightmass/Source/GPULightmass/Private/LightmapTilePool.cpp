// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightmapTilePool.h"
#include "GPULightmassModule.h"

FLightmapTilePool::FLightmapTilePool(int32 InNumTotalTiles)
	: NumTotalTiles(InNumTotalTiles)
{
	for (int32 TileIndex = 0; TileIndex < NumTotalTiles; TileIndex++)
	{
		FreeHeap.Add(0, TileIndex);
	}
}

void FLightmapTilePool::AllocAndLock(int32 NumTilesToAllocate, TArray<int32>& OutSuccessfullyAllocatedTiles)
{
	// Allocate and temporarily lock tiles, they will be returned by MakeAvailable soon
	// This is our way to prevent the same tile being rewritten multiple times on the same frame -- when that happens the allocation fails
	for (; NumTilesToAllocate > 0; NumTilesToAllocate--)
	{
		if (FreeHeap.Num() == 0)
		{
			break;
		}

		uint32 TileIndex = FreeHeap.Top();
		OutSuccessfullyAllocatedTiles.Add(TileIndex);
		FreeHeap.Pop();
	}
}

void FLightmapTilePool::Lock(const TArrayView<int32> InTiles)
{
	for (int32 TileIndex = 0; TileIndex < InTiles.Num(); TileIndex++)
	{
		checkSlow(FreeHeap.IsPresent(InTiles[TileIndex]));
		FreeHeap.Remove(InTiles[TileIndex]);
	}
}

void FLightmapTilePool::MakeAvailable(const TArrayView<int32> InSuccessfullyAllocatedTiles, const uint32 NewTileKey)
{
	for (int32 TileIndex = 0; TileIndex < InSuccessfullyAllocatedTiles.Num(); TileIndex++)
	{
		FreeHeap.Add(NewTileKey, InSuccessfullyAllocatedTiles[TileIndex]);
	}
}

void FLightmapTilePool::QueryResidency(const TArrayView<FVirtualTile> InTilesToQuery, TArray<uint32>& OutTileIndexIfResident)
{
	for (int32 TileIndex = 0; TileIndex < InTilesToQuery.Num(); TileIndex++)
	{
		if (VirtualTileToTileIndexMap.Find(InTilesToQuery[TileIndex]) != nullptr)
		{
			checkSlow(OutTileIndexIfResident.Find(VirtualTileToTileIndexMap[InTilesToQuery[TileIndex]]) == INDEX_NONE);
			OutTileIndexIfResident.Add(VirtualTileToTileIndexMap[InTilesToQuery[TileIndex]]);
		}
		else
		{
			OutTileIndexIfResident.Add(~0u);
		}
	}
}

void FLightmapTilePool::Map(const TArrayView<FVirtualTile> InVirtualTiles, const TArrayView<int32> InTiles, TArray<FVirtualTile>& OutVirtualTilesEvicted)
{
	ensure(InVirtualTiles.Num() == InTiles.Num());

	for (int32 TileIndex = 0; TileIndex < InVirtualTiles.Num(); TileIndex++)
	{
		checkSlow(VirtualTileToTileIndexMap.Find(InVirtualTiles[TileIndex]) == nullptr);

		if (TileIndexToVirtualTileMap.Find(InTiles[TileIndex]) != nullptr)
		{
			OutVirtualTilesEvicted.Add(TileIndexToVirtualTileMap[InTiles[TileIndex]]);
			VirtualTileToTileIndexMap.Remove(TileIndexToVirtualTileMap[InTiles[TileIndex]]);
			TileIndexToVirtualTileMap[InTiles[TileIndex]] = InVirtualTiles[TileIndex];
		}
		else
		{
			TileIndexToVirtualTileMap.Add(InTiles[TileIndex], InVirtualTiles[TileIndex]);
		}

		VirtualTileToTileIndexMap.Add(InVirtualTiles[TileIndex], InTiles[TileIndex]);
	}
}

void FLightmapTilePool::Unmap(const TArrayView<FVirtualTile> InVirtualTiles)
{
	for (int32 TileIndex = 0; TileIndex < InVirtualTiles.Num(); TileIndex++)
	{
		if (VirtualTileToTileIndexMap.Find(InVirtualTiles[TileIndex]) != nullptr)
		{
			VirtualTileToTileIndexMap.Remove(InVirtualTiles[TileIndex]);
		}
	}
}

void FLightmapTilePool::UnmapAll()
{
	TileIndexToVirtualTileMap.Empty();
	VirtualTileToTileIndexMap.Empty();
		
	FreeHeap.Free();	
	for (int32 TileIndex = 0; TileIndex < NumTotalTiles; TileIndex++)
	{
		FreeHeap.Add(0, TileIndex);
	}
}

FLightmapTilePoolGPU::FLightmapTilePoolGPU(FRHICommandListBase& RHICmdList, int32 NumLayers, FIntPoint InSizeInTiles, FIntPoint TileSize)
	: FLightmapTilePool(InSizeInTiles.X * InSizeInTiles.Y)
	, SizeInTiles(InSizeInTiles)
{
	ensure(SizeInTiles.X == SizeInTiles.Y);

	FIntPoint TextureSize(SizeInTiles.X * TileSize.X, SizeInTiles.Y * TileSize.Y);

	EPixelFormat RenderTargetFormat = PF_A32B32G32R32F;

	PooledRenderTargets.AddDefaulted(NumLayers);

	int64 MemorySize = 0;

	PooledRenderTargetDebugNames.AddDefaulted(NumLayers);

	for (int32 RenderTargetIndex = 0; RenderTargetIndex < NumLayers; RenderTargetIndex++)
	{
		LayerFormatAndTileSize.Add( { RenderTargetFormat, TileSize } );

		const FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DDesc(
			TextureSize,
			RenderTargetFormat,
			FClearValueBinding::None,
			TexCreate_None,
			TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV,
			false);

		PooledRenderTargetDebugNames[RenderTargetIndex] = FString::Printf(TEXT("LightmapTilePoolGPU%d"), RenderTargetIndex);
		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, PooledRenderTargets[RenderTargetIndex], *PooledRenderTargetDebugNames[RenderTargetIndex]);

		MemorySize += PooledRenderTargets[RenderTargetIndex]->ComputeMemorySize();

		ensure(PooledRenderTargets[RenderTargetIndex].IsValid());
	}

	// UE_LOGF(LogGPULightmass, Log, "LightmapTilePool created with %.2fMB", MemorySize / 1024.0f / 1024.0f);
}

void FLightmapTilePoolGPU::Initialize(FRHICommandListBase& RHICmdList, TArray<FLayerFormatAndTileSize> Layers)
{
	ensure(SizeInTiles.X == SizeInTiles.Y);

	PooledRenderTargets.AddDefaulted(Layers.Num());

	int64 MemorySize = 0;

	PooledRenderTargetDebugNames.AddDefaulted(Layers.Num());
	
	for (int32 RenderTargetIndex = 0; RenderTargetIndex < Layers.Num(); RenderTargetIndex++)
	{
		LayerFormatAndTileSize.Add(Layers[RenderTargetIndex]);

		FIntPoint TextureSize(SizeInTiles.X * Layers[RenderTargetIndex].TileSize.X, SizeInTiles.Y * Layers[RenderTargetIndex].TileSize.Y);

		const FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DDesc(
			TextureSize,
			Layers[RenderTargetIndex].Format,
			FClearValueBinding::None,
			TexCreate_None,
			TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV,
			false);

		PooledRenderTargetDebugNames[RenderTargetIndex] = FString::Printf(TEXT("LightmapTilePoolGPU%d"), RenderTargetIndex);
		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, PooledRenderTargets[RenderTargetIndex], *PooledRenderTargetDebugNames[RenderTargetIndex]);

		MemorySize += PooledRenderTargets[RenderTargetIndex]->ComputeMemorySize();

		ensure(PooledRenderTargets[RenderTargetIndex].IsValid());
	}

	UE_LOGF(LogGPULightmass, Log, "LightmapTilePool created with %.2fMB", MemorySize / 1024.0f / 1024.0f);
}

FIntPoint FLightmapTilePoolGPU::GetPositionFromLinearAddress(uint32 Address)
{
	return FIntPoint(Address % SizeInTiles.X, Address / SizeInTiles.X);
}

void FLightmapTilePoolGPU::ReleaseRenderTargets()
{
	for (int32 RenderTargetIndex = 0; RenderTargetIndex < PooledRenderTargets.Num(); RenderTargetIndex++)
	{
		PooledRenderTargets[RenderTargetIndex].SafeRelease();
	}
}
