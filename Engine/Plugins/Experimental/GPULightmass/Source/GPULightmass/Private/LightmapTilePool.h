// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/BinaryHeap.h"
#include "RenderTargetPool.h"
#include "LightmapStorage.h"

struct FVirtualTile
{
	// Comparison of FLightmapRenderStateRef is solely based on ElementId, which means it won't survive array changes (which means element swaps)
	GPULightmass::FLightmapRenderStateRef RenderState;
	int32 MipLevel = 0;
	int32 VirtualAddress = 0;
};

inline uint32 GetTypeHash(const FVirtualTile& TypeVar)
{
	return HashCombine(GetTypeHash(TypeVar.RenderState), HashCombine(GetTypeHash(TypeVar.MipLevel), GetTypeHash(TypeVar.VirtualAddress)));
}

inline bool operator==(const FVirtualTile& LHS, const FVirtualTile& RHS)
{
	return LHS.RenderState == RHS.RenderState && LHS.MipLevel == RHS.MipLevel && LHS.VirtualAddress == RHS.VirtualAddress;
}

class FLightmapTilePool
{
public:
	FLightmapTilePool(int32 InNumTotalTiles);

	int32 NumTotalTiles;

	void AllocAndLock(int32 NumTilesToAllocate, TArray<int32>& OutSuccessfullyAllocatedTiles);
	void Lock(const TArrayView<int32> InTiles);
	void MakeAvailable(const TArrayView<int32> InSuccessfullyAllocatedTiles, const uint32 NewTileKey);
	void QueryResidency(const TArrayView<FVirtualTile> InTilesToQuery, TArray<uint32>& OutTileIndexIfResident);
	void Map(const TArrayView<FVirtualTile> InVirtualTiles, const TArrayView<int32> InTiles, TArray<FVirtualTile>& OutVirtualTilesEvicted);
	void Unmap(const TArrayView<FVirtualTile> InVirtualTiles);
	void UnmapAll();

private:
	FBinaryHeap<uint32, uint32> FreeHeap;
	TMap<uint32, FVirtualTile> TileIndexToVirtualTileMap;
	TMap<FVirtualTile, uint32> VirtualTileToTileIndexMap;
};

class FLightmapTilePoolGPU : public FLightmapTilePool
{
public:
	struct FLayerFormatAndTileSize
	{
		EPixelFormat Format;
		FIntPoint TileSize;
	};

	FLightmapTilePoolGPU(FIntPoint InSizeInTiles) : FLightmapTilePool(InSizeInTiles.X * InSizeInTiles.Y), SizeInTiles(InSizeInTiles) {}
	FLightmapTilePoolGPU(FRHICommandListBase& RHICmdList, int32 NumLayers, FIntPoint InSizeInTiles, FIntPoint TileSize);

	void Initialize(FRHICommandListBase& RHICmdList, TArray<FLayerFormatAndTileSize> Layers);

	FIntPoint SizeInTiles;

	TArray<FLayerFormatAndTileSize> LayerFormatAndTileSize;
	TArray<TRefCountPtr<IPooledRenderTarget>> PooledRenderTargets;
	TArray<FString> PooledRenderTargetDebugNames;

	FIntPoint GetPositionFromLinearAddress(uint32 Address);

	void ReleaseRenderTargets();
};
