// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RayTracingDefinitions.h"
#include "PathTracingDefinitions.h"

#if RHI_RAYTRACING

#include "SystemTextures.h"

RENDERER_API void PrepareLightGrid(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FPathTracingLightGrid* LightGridParameters, const FPathTracingLight* Lights, uint32 NumLights, uint32 NumInfiniteLights, FRDGBufferSRV* LightsSRV);

static uint32 EncodeToF16x2(const FVector2f& In)
{
	return FFloat16(In.X).Encoded | (FFloat16(In.Y).Encoded << 16);
}

template<typename PassParameterType>
void SetupPathTracingLightParameters(
	const GPULightmass::FLightSceneRenderState& LightScene,
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	PassParameterType* PassParameters)
{
	TArray<FPathTracingLight> Lights;

	if (LightScene.SkyLight.IsSet())
	{
		FPathTracingLight& DestLight = Lights.AddDefaulted_GetRef();
		DestLight.Color = FVector3f(LightScene.SkyLight->Color);
		DestLight.Flags = PATHTRACER_FLAG_TRANSMISSION_MASK;
		DestLight.Flags |= PATHTRACER_FLAG_LIGHTING_CHANNEL_MASK;
		DestLight.Flags |= LightScene.SkyLight->bCastShadow ? PATHTRACER_FLAG_CAST_SHADOW_MASK : 0;
		bool SkyLightIsStationary = LightScene.SkyLight->bStationary;
		DestLight.Flags |= SkyLightIsStationary ? PATHTRACER_FLAG_STATIONARY_MASK : 0;
		DestLight.Flags |= PATHTRACING_LIGHT_SKY;

		PassParameters->SkylightTexture = GraphBuilder.RegisterExternalTexture(LightScene.SkyLight->PathTracingSkylightTexture, TEXT("PathTracer.Skylight"));
		PassParameters->SkylightTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->SkylightPdf = GraphBuilder.RegisterExternalTexture(LightScene.SkyLight->PathTracingSkylightPdf, TEXT("PathTracer.SkylightPdf"));
		PassParameters->SkylightInvResolution = LightScene.SkyLight->SkylightInvResolution;
		PassParameters->SkylightMipCount = LightScene.SkyLight->SkylightMipCount;
	}
	else
	{
		PassParameters->SkylightTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		PassParameters->SkylightTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->SkylightPdf = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		PassParameters->SkylightInvResolution = 0;
		PassParameters->SkylightMipCount = 0;
	}


	for (auto Light : LightScene.DirectionalLights.Elements)
	{
		FPathTracingLight& DestLight = Lights.AddDefaulted_GetRef();

		DestLight.Normal = (FVector3f)-Light.Direction;
		DestLight.Color = FVector3f(Light.Color);
		DestLight.Dimensions = FVector2f(
			FMath::Sin(0.5f * FMath::DegreesToRadians(Light.LightSourceAngle)),
			0.0f);
		DestLight.Attenuation = 1.0;
		DestLight.IESAtlasIndex = INDEX_NONE;

		DestLight.Flags = PATHTRACER_FLAG_TRANSMISSION_MASK;
		DestLight.Flags |= PATHTRACER_FLAG_LIGHTING_CHANNEL_MASK;
		DestLight.Flags |= Light.bCastShadow ? PATHTRACER_FLAG_CAST_SHADOW_MASK : 0;
		DestLight.Flags |= Light.bStationary ? PATHTRACER_FLAG_STATIONARY_MASK : 0;
		DestLight.Flags |= PATHTRACING_LIGHT_DIRECTIONAL;
	}

	uint32 NumInfiniteLights = Lights.Num();

	for (auto Light : LightScene.PointLights.Elements)
	{
		FPathTracingLight& DestLight = Lights.AddDefaulted_GetRef();

		DestLight.TranslatedWorldPosition = FVector3f(Light.Position + View.ViewMatrices.GetPreViewTranslation());
		DestLight.Color = (FVector3f)(Light.Color);
		DestLight.Normal = (FVector3f)Light.Direction;
		DestLight.Tangent = (FVector3f)Light.Tangent;

		DestLight.Dimensions = FVector2f(Light.SourceRadius, Light.SourceLength);
		DestLight.Attenuation = 1.0f / Light.AttenuationRadius;
		DestLight.FalloffExponent = Light.FalloffExponent;
		DestLight.IESAtlasIndex = Light.GetIESAtlasIndex();

		DestLight.Flags = PATHTRACER_FLAG_TRANSMISSION_MASK;
		DestLight.Flags |= PATHTRACER_FLAG_LIGHTING_CHANNEL_MASK;
		DestLight.Flags |= Light.bCastShadow ? PATHTRACER_FLAG_CAST_SHADOW_MASK : 0;
		DestLight.Flags |= Light.IsInverseSquared ? 0 : PATHTRACER_FLAG_NON_INVERSE_SQUARE_FALLOFF_MASK;
		DestLight.Flags |= Light.bStationary ? PATHTRACER_FLAG_STATIONARY_MASK : 0;
		DestLight.Flags |= PATHTRACING_LIGHT_POINT;
	}

	for (auto Light : LightScene.SpotLights.Elements)
	{
		FPathTracingLight& DestLight = Lights.AddDefaulted_GetRef();

		DestLight.TranslatedWorldPosition = FVector3f(Light.Position + View.ViewMatrices.GetPreViewTranslation());
		DestLight.Normal = (FVector3f)Light.Direction;
		DestLight.Tangent = (FVector3f)Light.Tangent;
		DestLight.Color = FVector3f(Light.Color);
		DestLight.Dimensions = FVector2f(Light.SourceRadius, Light.SourceLength);
		DestLight.Shaping = FVector2f(Light.SpotAngles);
		DestLight.Attenuation = 1.0f / Light.AttenuationRadius;
		DestLight.FalloffExponent = Light.FalloffExponent;
		DestLight.IESAtlasIndex = Light.GetIESAtlasIndex();

		DestLight.Flags = PATHTRACER_FLAG_TRANSMISSION_MASK;
		DestLight.Flags |= PATHTRACER_FLAG_LIGHTING_CHANNEL_MASK;
		DestLight.Flags |= Light.bCastShadow ? PATHTRACER_FLAG_CAST_SHADOW_MASK : 0;
		DestLight.Flags |= Light.IsInverseSquared ? 0 : PATHTRACER_FLAG_NON_INVERSE_SQUARE_FALLOFF_MASK;
		DestLight.Flags |= Light.bStationary ? PATHTRACER_FLAG_STATIONARY_MASK : 0;
		DestLight.Flags |= PATHTRACING_LIGHT_SPOT;
	}

	for (auto Light : LightScene.RectLights.Elements)
	{
		FPathTracingLight& DestLight = Lights.AddDefaulted_GetRef();

		DestLight.TranslatedWorldPosition = FVector3f(Light.Position + View.ViewMatrices.GetPreViewTranslation());
		DestLight.Normal = (FVector3f)Light.Direction;
		DestLight.Tangent = (FVector3f)Light.Tangent;

		FLinearColor LightColor = Light.Color;
		LightColor /= 0.5f * Light.SourceWidth * Light.SourceHeight;
		DestLight.Color = FVector3f(LightColor);

		DestLight.Dimensions = FVector2f(Light.SourceWidth, Light.SourceHeight);
		DestLight.Attenuation = 1.0f / Light.AttenuationRadius;
		DestLight.Shaping = FVector2f(FMath::Cos(FMath::DegreesToRadians(Light.BarnDoorAngle)), Light.BarnDoorLength);
		DestLight.IESAtlasIndex = Light.GetIESAtlasIndex();

		DestLight.Flags = PATHTRACER_FLAG_TRANSMISSION_MASK;
		DestLight.Flags |= PATHTRACER_FLAG_LIGHTING_CHANNEL_MASK;
		DestLight.Flags |= Light.bCastShadow ? PATHTRACER_FLAG_CAST_SHADOW_MASK : 0;
		DestLight.Flags |= Light.bStationary ? PATHTRACER_FLAG_STATIONARY_MASK : 0;
		DestLight.Flags |= PATHTRACING_LIGHT_RECT;

		DestLight.RectLightAtlasUVOffset = Light.RectLightAtlasUVOffset;
		DestLight.RectLightAtlasUVScale  = Light.RectLightAtlasUVScale;
		if (Light.RectLightAtlasMaxLevel < 16)
		{
			DestLight.Flags |= PATHTRACER_FLAG_HAS_RECT_TEXTURE_MASK;
		}
	}

	PassParameters->SceneLightCount = Lights.Num();
	{
		// Upload the buffer of lights to the GPU
		// need at least one since zero-sized buffers are not allowed
		if (Lights.Num() == 0)
		{
			Lights.AddDefaulted();
		}
		PassParameters->SceneLights = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(CreateStructuredBuffer(GraphBuilder, TEXT("PathTracer.LightsBuffer"), sizeof(FPathTracingLight), Lights.Num(), Lights.GetData(), sizeof(FPathTracingLight) * Lights.Num())));
	}

	PrepareLightGrid(GraphBuilder, View.GetFeatureLevel(), &PassParameters->LightGridParameters, Lights.GetData(), PassParameters->SceneLightCount, NumInfiniteLights, PassParameters->SceneLights);
}

#endif  // RHI_RAYTRACING
