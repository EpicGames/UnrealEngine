// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryInterface.h"
#include "MeshBatch.h"

namespace GPULightmass
{

TArray<FMeshBatch> FGeometryInstanceRenderStateRef::GetMeshBatchesForGBufferRendering(FTileVirtualCoordinates CoordsForCulling)
{
	return Collection.GetMeshBatchesForGBufferRendering(*this, CoordsForCulling);
}

FVector FGeometryInstanceRenderStateRef::GetOrigin() const
{
	return Collection.Get(GetElementIdChecked()).LocalToWorld.GetOrigin();
}

}