// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GPULightmassCommon.h"
#include "EntityArray.h"
#include "Components/PrimitiveComponent.h"
#include "MeshBatch.h"

namespace GPULightmass
{

using FLightmapRef = TEntityArray<class FLightmap>::EntityRefType;
using FLightmapRenderStateRef = TEntityArray<class FLightmapRenderState>::EntityRefType;

struct FGeometryRenderState
{
	FMatrix LocalToWorld;

	FBoxSphereBounds WorldBounds;
	FVector ActorPosition;
	FBoxSphereBounds LocalBounds;

	bool bCastShadow;

	TArray<FLightmapRenderStateRef> LODLightmapRenderStates;
};

struct FGeometry
{
	virtual ~FGeometry() {}
	FGeometry() = default;
	FGeometry(FGeometry&& In) = default; // Default move constructor is killed by the custom destructor

	FBoxSphereBounds WorldBounds;

	bool bCastShadow;
	bool bLODsShareStaticLighting;

	TArray<FLightmapRef> LODLightmaps;

	virtual UPrimitiveComponent* GetComponentUObject() const = 0;
};

class FGeometryInstanceRenderStateCollectionBase;

class FGeometryInstanceRenderStateRef : public FGenericEntityRef
{
public:
	FGeometryInstanceRenderStateRef(
		FGeometryInstanceRenderStateCollectionBase& Collection,
		TArray<TSet<RefAddr>>& Refs,
		TSparseArray<int32>& RefAllocator,
		int32 ElementId, 
		int32 LODIndex)
		: FGenericEntityRef(ElementId, Refs, RefAllocator)
		, LODIndex(LODIndex)
		, Collection(Collection)
	{}

	TArray<FMeshBatch> GetMeshBatchesForGBufferRendering(FTileVirtualCoordinates CoordsForCulling = FTileVirtualCoordinates{});
	FVector GetOrigin() const;

	int32 LODIndex;
private:
	FGeometryInstanceRenderStateCollectionBase& Collection;

	template<typename GeometryType>
	friend class TGeometryInstanceRenderStateCollection;
};

class FGeometryInstanceRenderStateCollectionBase
{
public:
	virtual ~FGeometryInstanceRenderStateCollectionBase() {}
	virtual TArray<FMeshBatch> GetMeshBatchesForGBufferRendering(const FGeometryInstanceRenderStateRef& GeometryInstanceRef, FTileVirtualCoordinates CoordsForCulling) = 0;
	virtual FGeometryRenderState& Get(int32 ElementId) = 0;
};

template<typename GeometryType>
class TGeometryInstanceRenderStateCollection : public FGeometryInstanceRenderStateCollectionBase, public TEntityArray<GeometryType>
{
public:
	virtual TArray<FMeshBatch> GetMeshBatchesForGBufferRendering(const FGeometryInstanceRenderStateRef& GeometryInstanceRef, FTileVirtualCoordinates CoordsForCulling) override;

	FGeometryInstanceRenderStateRef CreateGeometryInstanceRef(const GeometryType& Element, int32 LODIndex)
	{
		return FGeometryInstanceRenderStateRef(*this, this->Refs, this->RefAllocator, &Element - this->Elements.GetData(), LODIndex);
	}

	GeometryType& ResolveGeometryInstanceRef(const FGeometryInstanceRenderStateRef& GeometryInstanceRef)
	{
		return this->Elements[GeometryInstanceRef.GetElementIdChecked()];
	}

	bool Contains(const FGeometryInstanceRenderStateRef& GeometryInstanceRef) const
	{
		return &GeometryInstanceRef.Collection == this;
	}

	virtual FGeometryRenderState& Get(int32 ElementId) override
	{
		return this->Elements[ElementId];
	}
};
class FGeometryArrayBase
{
public:
	virtual ~FGeometryArrayBase() {}
	virtual int32 Num() const = 0;
	virtual FGeometry& Get(int32 Index) = 0;
	void LinkRenderStateArray(FGeometryInstanceRenderStateCollectionBase& InRenderStateArray) { RenderStateArray = &InRenderStateArray; }
	FGeometryInstanceRenderStateCollectionBase* GetRenderStateArray() { return RenderStateArray; }

private:
	FGeometryInstanceRenderStateCollectionBase* RenderStateArray = nullptr;
};

template<typename GeometryType>
class TGeometryArray : public FGeometryArrayBase, public TEntityArray<GeometryType>
{
public:
	virtual int32 Num() const override { return this->Elements.Num(); }
	virtual FGeometry& Get(int32 Index) override { return this->Elements[Index]; }
};

struct FGeometryAndItsArray
{
	int32 Index;
	FGeometryArrayBase& Array;

	FGeometry& GetGeometry() { return Array.Get(Index); }
};

struct FGeometryIterator
{
	int32 IndexInCurrentArray;
	TArray<FGeometryArrayBase*> ArraysToIterate;
	int32 ArrayIndex;

	FGeometryIterator& operator++()
	{
		check(ArrayIndex < ArraysToIterate.Num());
		check(IndexInCurrentArray < ArraysToIterate[ArrayIndex]->Num());

		++IndexInCurrentArray;

		while (ArrayIndex < ArraysToIterate.Num() && IndexInCurrentArray == ArraysToIterate[ArrayIndex]->Num())
		{
			++ArrayIndex;
			if (ArrayIndex < ArraysToIterate.Num())
			{
				IndexInCurrentArray = 0;
			}
		}

		return *this;
	}

	bool operator!=(const FGeometryIterator& Other)
	{
		// We don't check the list of arrays
		return IndexInCurrentArray != Other.IndexInCurrentArray || ArrayIndex != Other.ArrayIndex;
	}

	FGeometryAndItsArray operator*() const
	{
		return FGeometryAndItsArray{ IndexInCurrentArray, *ArraysToIterate[ArrayIndex] };
	}
};

struct FGeometryRenderStateToken
{
	int32 ElementId;
	FGeometryInstanceRenderStateCollectionBase* RenderStateArray;
};

}
