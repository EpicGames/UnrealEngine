// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedStaticMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/InstancedStaticMesh.h"
#include "LightmapGBuffer.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "GameFramework/WorldSettings.h"

namespace GPULightmass
{

FInstanceGroup::FInstanceGroup(UInstancedStaticMeshComponent* ComponentUObject)
	: ComponentUObject(ComponentUObject)
{

}

const FMeshMapBuildData* FInstanceGroup::GetMeshMapBuildDataForLODIndex(int32 LODIndex)
{
	// ISM shares LOD0 lightmap with other LODs
	return (LODLightmaps.Num() > 0 && LODLightmaps[0].IsValid()) ? LODLightmaps[0]->MeshMapBuildData.Get() : nullptr;
}

void FInstanceGroup::AllocateLightmaps(TEntityArray<FLightmap>& LightmapContainer)
{
	for (int32 LODIndex = 0; LODIndex < ComponentUObject->GetStaticMesh()->GetRenderData()->LODResources.Num(); LODIndex++)
	{
		FStaticMeshLODResources& LODModel = ComponentUObject->GetStaticMesh()->GetRenderData()->LODResources[LODIndex];

		int32 LightMapWidth = 0;
		int32 LightMapHeight = 0;
		ComponentUObject->GetLightMapResolution(LightMapWidth, LightMapHeight);

		bool bFit = false;
		bool bReduced = false;
		while (1)
		{
			const int32 OneLessThanMaximumSupportedResolution = 1 << (GMaxTextureMipCount - 2);

			const int32 MaxInstancesInMaxSizeLightmap = (OneLessThanMaximumSupportedResolution / LightMapWidth) * ((OneLessThanMaximumSupportedResolution / 2) / LightMapHeight);
			if (ComponentUObject->PerInstanceSMData.Num() > MaxInstancesInMaxSizeLightmap)
			{
				if (LightMapWidth < 4 || LightMapHeight < 4)
				{
					break;
				}
				LightMapWidth /= 2;
				LightMapHeight /= 2;
				bReduced = true;
			}
			else
			{
				bFit = true;
				break;
			}
		}

		if (!bFit)
		{
			FMessageLog("LightingResults").Message(EMessageSeverity::Error)
				->AddToken(FUObjectToken::Create(ComponentUObject))
				->AddToken(FTextToken::Create(NSLOCTEXT("InstancedStaticMesh", "FailedStaticLightingWarning", "The total lightmap size for this InstancedStaticMeshComponent is too big no matter how much we reduce the per-instance size, the number of mesh instances in this component must be reduced")));
		}
		if (bReduced)
		{
			FMessageLog("LightingResults").Message(EMessageSeverity::Warning)
				->AddToken(FUObjectToken::Create(ComponentUObject))
				->AddToken(FTextToken::Create(NSLOCTEXT("InstancedStaticMesh", "ReducedStaticLightingWarning", "The total lightmap size for this InstancedStaticMeshComponent was too big and it was automatically reduced. Consider reducing the component's lightmap resolution or number of mesh instances in this component")));
		}

		const int32 LightMapSize = ComponentUObject->GetWorld()->GetWorldSettings()->PackedLightAndShadowMapTextureSize;
		const int32 MaxInstancesInDefaultSizeLightmap = (LightMapSize / LightMapWidth) * ((LightMapSize / 2) / LightMapHeight);
		if (ComponentUObject->PerInstanceSMData.Num() > MaxInstancesInDefaultSizeLightmap)
		{
			FMessageLog("LightingResults").Message(EMessageSeverity::Warning)
				->AddToken(FUObjectToken::Create(ComponentUObject))
				->AddToken(FTextToken::Create(NSLOCTEXT("InstancedStaticMesh", "LargeStaticLightingWarning", "The total lightmap size for this InstancedStaticMeshComponent is large, consider reducing the component's lightmap resolution or number of mesh instances in this component")));
		}

#if 0
		// TODO: Support separate static lighting in LODs for instanced meshes.

		if (!ComponentUObject->GetStaticMesh()->CanLODsShareStaticLighting())
		{
			//TODO: Detect if the UVs for all sub-LODs overlap the base LOD UVs and omit this warning if they do.
			FMessageLog("LightingResults").Message(EMessageSeverity::Warning)
				->AddToken(FUObjectToken::Create(ComponentUObject))
				->AddToken(FTextToken::Create(NSLOCTEXT("InstancedStaticMesh", "UniqueStaticLightingForLODWarning", "Instanced meshes don't yet support unique static lighting for each LOD. Lighting on LOD 1+ may be incorrect unless lightmap UVs are the same for all LODs.")));
		}
#endif

		bool bValidTextureMap = false;
		if (bFit 
			&& LightMapWidth > 0
			&& LightMapHeight > 0
			&& ComponentUObject->GetStaticMesh()->GetLightMapCoordinateIndex() >= 0
			&& (uint32)ComponentUObject->GetStaticMesh()->GetLightMapCoordinateIndex() < LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords())
		{
			bValidTextureMap = true;
		}

		// ISM shares LOD0 lightmap with other LODs
		if (bValidTextureMap && LODIndex == 0 && ComponentUObject->GetLightmapType() != ELightmapType::ForceVolumetric)
		{
			check(LightMapWidth == LightMapHeight);

			int32 TotalLightmapRes = LightMapWidth * FMath::CeilToInt(FMath::Sqrt(static_cast<float>(ComponentUObject->PerInstanceSMData.Num())));

			// Shrink LOD texture lightmaps by half for each LOD level
			const int32 TotalLightMapWidth = /*LODIndex > 0 ? FMath::Max(TotalLightmapRes / (2 << (LODIndex - 1)), 32) : */ TotalLightmapRes;
			const int32 TotalLightMapHeight = /*LODIndex > 0 ? FMath::Max(TotalLightmapRes / (2 << (LODIndex - 1)), 32) : */ TotalLightmapRes;

			FString LightmapName = TEXT("Lightmap_") + (ComponentUObject->GetOwner() ? ComponentUObject->GetOwner()->GetActorLabel() : FString());

			LODLightmaps.Add(LightmapContainer.Emplace(LightmapName, FIntPoint(TotalLightMapWidth, TotalLightMapHeight)));
			LODPerInstanceLightmapSize.Add(FIntPoint(LightMapWidth, LightMapHeight));
		}
		else
		{
			LODLightmaps.Add(LightmapContainer.CreateNullRef());
			LODPerInstanceLightmapSize.Add(FIntPoint(0, 0));
		}
	}
}

TArray<FMeshBatch> FInstanceGroupRenderState::GetMeshBatchesForGBufferRendering(int32 LODIndex, FTileVirtualCoordinates CoordsForCulling)
{
	TArray<FMeshBatch> MeshBatches;

	FInstancedStaticMeshVFLooseUniformShaderParameters LooseParameters{};
	FUniformBufferRHIRef LooseParametersUniformBuffer = CreateUniformBufferImmediate(LooseParameters, EUniformBufferUsage::UniformBuffer_SingleFrame);

	// TODO: potential race conditions between GT & RT everywhere in the following code
	FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];
	for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
	{
		const FStaticMeshSection& Section = LODModel.Sections[SectionIndex];
		if (Section.NumTriangles == 0)
		{
			continue;
		}

		FMeshBatch MeshBatch;

		FMeshBatchElement& MeshBatchElement = MeshBatch.Elements[0];

		if (CoordsForCulling.MipLevel == -1 || !InstancedRenderData.IsValid())
		{
			// No culling, should be for ray tracing scene
			// YW: While it seems more 'correct' to use FISMVF, what it contains is basically view-dependent instance fading loose parameters,
			// which isn't particularly useful for lightmap baking.
			// Most persistent data, like per instance transforms and randoms, has been moved to GPUScene.
			// This is also consistent with the behavior of FInstancedStaticMeshSceneProxy::GetDynamicRayTracingInstances() which
			// uses the static mesh VF as well, so that GPULM can benefit from bug fixes there
			MeshBatch.VertexFactory = &RenderData->LODVertexFactories[LODIndex].VertexFactory;
		}
		else
		{
			MeshBatch.VertexFactory = &InstancedRenderData->VertexFactories[LODIndex];
		}
		
		MeshBatchElement.VertexFactoryUserData = RenderData->LODVertexFactories[LODIndex].VertexFactory.GetUniformBuffer();
		MeshBatchElement.IndexBuffer = &LODModel.IndexBuffer;
		MeshBatchElement.FirstIndex = Section.FirstIndex;
		MeshBatchElement.NumPrimitives = Section.NumTriangles;
		MeshBatchElement.MinVertexIndex = Section.MinVertexIndex;
		MeshBatchElement.MaxVertexIndex = Section.MaxVertexIndex;
		MeshBatchElement.PrimitiveIdMode = PrimID_DynamicPrimitiveShaderData;
		MeshBatchElement.LooseParametersUniformBuffer = LooseParametersUniformBuffer;

		MeshBatchElement.InstancedLODIndex = LODIndex;
		MeshBatch.LODIndex = LODIndex;
		MeshBatch.SegmentIndex = SectionIndex;
		MeshBatch.CastShadow = bCastShadow && Section.bCastShadow;
		MeshBatch.LCI = LODLightmapRenderStates[LODIndex].IsValid() ? LODLightmapRenderStates[LODIndex].GetUnderlyingAddress_Unsafe() : nullptr;

		if (ComponentUObject->GetMaterial(Section.MaterialIndex) != nullptr)
		{
			MeshBatch.MaterialRenderProxy = ComponentUObject->GetMaterial(Section.MaterialIndex)->GetRenderProxy();

			if (CoordsForCulling.MipLevel == -1)
			{
				// No culling, should be for ray tracing scene
				MeshBatchElement.UserIndex = INDEX_NONE;
				MeshBatchElement.NumInstances = NumInstances;

				MeshBatches.Add(MeshBatch);
			}
			else
			{
				if (LODLightmapRenderStates[LODIndex].IsValid())
				{
					if ((LODPerInstanceLightmapSize[LODIndex].X >> CoordsForCulling.MipLevel) > 0 && (LODPerInstanceLightmapSize[LODIndex].Y >> CoordsForCulling.MipLevel) > 0)
					{
						FIntPoint Size = LODLightmapRenderStates[LODIndex]->GetPaddedSizeInTilesAtMipLevel(CoordsForCulling.MipLevel) * GPreviewLightmapVirtualTileSize;
						FIntPoint Min = (CoordsForCulling.Position * GPreviewLightmapVirtualTileSize - FIntPoint(GPreviewLightmapTileBorderSize, GPreviewLightmapTileBorderSize)).ComponentMax(FIntPoint(0, 0));
						FIntPoint Max = ((CoordsForCulling.Position + FIntPoint(1, 1)) * GPreviewLightmapVirtualTileSize + FIntPoint(GPreviewLightmapTileBorderSize, GPreviewLightmapTileBorderSize)).ComponentMin(Size);
						FIntPoint MinInInstanceTile(FMath::DivideAndRoundDown(Min.X, LODPerInstanceLightmapSize[LODIndex].X >> CoordsForCulling.MipLevel), FMath::DivideAndRoundDown(Min.Y, LODPerInstanceLightmapSize[LODIndex].Y >> CoordsForCulling.MipLevel));
						FIntPoint MaxInInstanceTile(FMath::DivideAndRoundUp(Max.X, LODPerInstanceLightmapSize[LODIndex].X >> CoordsForCulling.MipLevel), FMath::DivideAndRoundUp(Max.Y, LODPerInstanceLightmapSize[LODIndex].Y >> CoordsForCulling.MipLevel));

						int32 InstancesPerRow = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(NumInstances)));

						for (int32 Y = MinInInstanceTile.Y; Y < MaxInInstanceTile.Y; Y++)
						{
							int32 MinInstanceIndex = Y * InstancesPerRow + MinInInstanceTile.X;
							int32 MaxInstanceIndex = Y * InstancesPerRow + MaxInInstanceTile.X;

							MinInstanceIndex = FMath::Min(MinInstanceIndex, (int32)NumInstances - 1);
							
							MeshBatchElement.UserIndex = MinInstanceIndex;							
							MeshBatchElement.NumInstances = FMath::Min(MaxInstanceIndex - MinInstanceIndex, NumInstances - MinInstanceIndex);

							MeshBatches.Add(MeshBatch);
						}
					}
				}
			}
		}
	}

	return MeshBatches;
}

template<>
TArray<FMeshBatch> TGeometryInstanceRenderStateCollection<FInstanceGroupRenderState>::GetMeshBatchesForGBufferRendering(const FGeometryInstanceRenderStateRef& GeometryInstanceRef, FTileVirtualCoordinates CoordsForCulling)
{
	FInstanceGroupRenderState& Instance = ResolveGeometryInstanceRef(GeometryInstanceRef);

	return Instance.GetMeshBatchesForGBufferRendering(GeometryInstanceRef.LODIndex, CoordsForCulling);
}

}
