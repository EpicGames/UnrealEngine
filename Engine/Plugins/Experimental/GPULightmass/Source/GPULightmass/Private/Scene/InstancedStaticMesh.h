// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightmapStorage.h"
#include "GeometryInterface.h"
#include "Engine/InstancedStaticMesh.h"

namespace GPULightmass
{

class FInstanceGroupRenderState : public FGeometryRenderState
{
public:
	UInstancedStaticMeshComponent* ComponentUObject;
	FStaticMeshRenderData* RenderData;
	TUniquePtr<FInstancedStaticMeshRenderData> InstancedRenderData;

	TArray<FIntPoint> LODPerInstanceLightmapSize;
	int32 NumInstances = 0;
	TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> InstanceDataSceneProxy;
	const FInstanceSceneDataBuffers *InstanceSceneDataBuffers = nullptr;

	TArray<FMeshBatch> GetMeshBatchesForGBufferRendering(int32 LODIndex, FTileVirtualCoordinates CoordsForCulling);
};

using FInstanceGroupRenderStateRef = TEntityArray<FInstanceGroupRenderState>::EntityRefType;

class FInstanceGroup : public FGeometry
{
public:
	FInstanceGroup(UInstancedStaticMeshComponent* ComponentUObject);

	const class FMeshMapBuildData* GetMeshMapBuildDataForLODIndex(int32 LODIndex);

	TArray<FIntPoint> LODPerInstanceLightmapSize;

	void AllocateLightmaps(TEntityArray<FLightmap>& LightmapContainer);

	UInstancedStaticMeshComponent* ComponentUObject;
	virtual UPrimitiveComponent* GetComponentUObject() const override { return ComponentUObject; }
};

using FInstanceGroupRef = TEntityArray<FInstanceGroup>::EntityRefType;

}
