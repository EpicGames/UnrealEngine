// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/Landscape.h"
#include "Framework/Application/SlateApplication.h"
#include "Scene/Lights.h"
#include "LightmapGBuffer.h"
#include "LandscapeProxy.h"

float GetTerrainExpandPatchCount(float LightMapRes, int32& X, int32& Y, int32 ComponentSize, int32 LightmapSize, int32& DesiredSize, uint32 LightingLOD)
{
	if (LightMapRes <= 0) return 0.f;

	// Assuming DXT_1 compression at the moment...
	int32 PixelPaddingX = GPixelFormats[PF_DXT1].BlockSizeX; // "/2" ?
	int32 PixelPaddingY = GPixelFormats[PF_DXT1].BlockSizeY;
	int32 PatchExpandCountX = (LightMapRes >= 1.f) ? (PixelPaddingX) / LightMapRes : (PixelPaddingX);
	int32 PatchExpandCountY = (LightMapRes >= 1.f) ? (PixelPaddingY) / LightMapRes : (PixelPaddingY);

	X = FMath::Max<int32>(1, PatchExpandCountX >> LightingLOD);
	Y = FMath::Max<int32>(1, PatchExpandCountY >> LightingLOD);

	DesiredSize = (LightMapRes >= 1.f) ? FMath::Min<int32>((int32)((ComponentSize + 1) * LightMapRes), 4096) : FMath::Min<int32>((int32)((LightmapSize)*LightMapRes), 4096);
	int32 CurrentSize = (LightMapRes >= 1.f) ? FMath::Min<int32>((int32)((2 * (X << LightingLOD) + ComponentSize + 1) * LightMapRes), 4096) : FMath::Min<int32>((int32)((2 * (X << LightingLOD) + LightmapSize) * LightMapRes), 4096);

	// Find proper Lightmap Size
	if (CurrentSize > DesiredSize)
	{
		// Find maximum bit
		int32 PriorSize = DesiredSize;
		while (DesiredSize > 0)
		{
			PriorSize = DesiredSize;
			DesiredSize = DesiredSize & ~(DesiredSize & ~(DesiredSize - 1));
		}

		DesiredSize = PriorSize << 1; // next bigger size
		if (CurrentSize * CurrentSize <= ((PriorSize * PriorSize) << 1))
		{
			DesiredSize = PriorSize;
		}
	}

	int32 DestSize = (float)DesiredSize / CurrentSize * (ComponentSize * LightMapRes);
	float LightMapRatio = (float)DestSize / (ComponentSize * LightMapRes) * CurrentSize / DesiredSize;
	return LightMapRatio;
}

namespace GPULightmass
{

FLandscape::FLandscape(ULandscapeComponent* ComponentUObject)
	: ComponentUObject(ComponentUObject)
{

}

const FMeshMapBuildData* FLandscape::GetMeshMapBuildDataForLODIndex(int32 LODIndex)
{
	return LODLightmaps[0].IsValid() ? LODLightmaps[0]->MeshMapBuildData.Get() : nullptr;
}

void FLandscape::AllocateLightmaps(TEntityArray<FLightmap>& LightmapContainer)
{
	const float LightMapRes = ComponentUObject->StaticLightingResolution > 0.f ? ComponentUObject->StaticLightingResolution : ComponentUObject->GetLandscapeProxy()->StaticLightingResolution;
	const int32 LightingLOD = ComponentUObject->GetLandscapeProxy()->StaticLightingLOD;

	int32 PatchExpandCountX = 0;
	int32 PatchExpandCountY = 0;
	int32 DesiredSize = 1;
	const float LightMapRatio = GetTerrainExpandPatchCount(LightMapRes, PatchExpandCountX, PatchExpandCountY, ComponentUObject->ComponentSizeQuads, (ComponentUObject->NumSubsections * (ComponentUObject->SubsectionSizeQuads + 1)), DesiredSize, LightingLOD);

	int32 SizeX = DesiredSize;
	int32 SizeY = DesiredSize;

	if (SizeX > 0 && SizeY > 0)
	{
		FString LightmapName = TEXT("Lightmap_") + (ComponentUObject->GetOwner() ? ComponentUObject->GetOwner()->GetActorLabel() : FString());
		LODLightmaps.Add(LightmapContainer.Emplace(LightmapName, FIntPoint(SizeX, SizeY)));
	}
}

TArray<FMeshBatch> FLandscapeRenderState::GetMeshBatchesForGBufferRendering(int32 InLODIndex)
{
	// Force landscape LOD to be 0 when rendering GBuffer
	int32 LODIndex = 0;

	TArray<FMeshBatch> MeshBatches;

	FMeshBatch MeshBatch;

	MeshBatch.VertexFactory = SharedBuffers->FixedGridVertexFactory;
	MeshBatch.MaterialRenderProxy = MaterialInterface->GetRenderProxy();

	MeshBatch.LCI = LODLightmapRenderStates[LODIndex].IsValid() ? LODLightmapRenderStates[LODIndex].GetUnderlyingAddress_Unsafe() : nullptr;
	MeshBatch.CastShadow = true;
	MeshBatch.bUseForDepthPass = true;
	MeshBatch.bUseAsOccluder = true;
	MeshBatch.bUseForMaterial = true;
	MeshBatch.Type = PT_TriangleList;
	MeshBatch.DepthPriorityGroup = SDPG_World;
	MeshBatch.LODIndex = LODIndex;
	MeshBatch.bDitheredLODTransition = false;
	MeshBatch.SegmentIndex = 0;

	// Combined batch element
	FMeshBatchElement& BatchElement = MeshBatch.Elements[0];

	BatchElementParams.LandscapeUniformShaderParametersResource = LandscapeUniformShaderParameters.Get();
	BatchElementParams.FixedGridUniformShaderParameters = &LandscapeFixedGridUniformShaderParameters;
	BatchElementParams.SceneProxy = nullptr;
	BatchElementParams.CurrentLOD = LODIndex;

	BatchElement.UserData = &BatchElementParams;
	BatchElement.IndexBuffer = SharedBuffers->IndexBuffers[LODIndex];
	BatchElement.NumPrimitives = FMath::Square((SubsectionSizeVerts >> LODIndex) - 1) * FMath::Square(NumSubsections) * 2;
	BatchElement.FirstIndex = 0;
	BatchElement.MinVertexIndex = SharedBuffers->IndexRanges[LODIndex].MinIndexFull;
	BatchElement.MaxVertexIndex = SharedBuffers->IndexRanges[LODIndex].MaxIndexFull;
	BatchElement.PrimitiveIdMode = PrimID_DynamicPrimitiveShaderData;

	MeshBatches.Add(MeshBatch);

	return MeshBatches;
}

template<>
TArray<FMeshBatch> TGeometryInstanceRenderStateCollection<FLandscapeRenderState>::GetMeshBatchesForGBufferRendering(const FGeometryInstanceRenderStateRef& GeometryInstanceRef, FTileVirtualCoordinates CoordsForCulling)
{
	FLandscapeRenderState& Instance = ResolveGeometryInstanceRef(GeometryInstanceRef);

	return Instance.GetMeshBatchesForGBufferRendering(GeometryInstanceRef.LODIndex);
}

}
