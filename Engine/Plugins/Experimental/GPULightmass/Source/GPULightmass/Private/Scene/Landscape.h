// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightmapStorage.h"
#include "GeometryInterface.h"
#include "LandscapeComponent.h"
#include "LandscapeRender.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "RayTracingGeometry.h"

namespace GPULightmass
{

class FLandscapeRenderState : public FGeometryRenderState
{
public:
	struct Initializer
	{
		int32       SubsectionSizeQuads;
		int32       NumSubsections;
		int32       ComponentSizeQuads;
		int32       ComponentSizeVerts;
		int32       SubsectionSizeVerts;
		float       StaticLightingResolution;
		int32       StaticLightingLOD;
		float       InvLODBlendRange;
		FIntPoint   ComponentBase;
		FIntPoint   SectionBase;
		UTexture2D* HeightmapTexture;
		float       HeightmapSubsectionOffsetU;
		float       HeightmapSubsectionOffsetV;
		FVector4f    HeightmapScaleBias;
		FVector4f    WeightmapScaleBias;
		float       WeightmapSubsectionOffset;
	};

	struct FLandscapeSectionRayTracingState
	{
		int8 CurrentLOD;

		FRayTracingGeometry Geometry;
		FRWBuffer RayTracingDynamicVertexBuffer;
		FLandscapeVertexFactoryMVFUniformBufferRef UniformBuffer;

		FLandscapeSectionRayTracingState() : CurrentLOD(-1) {}
	};

	TStaticArray<TUniquePtr<FLandscapeSectionRayTracingState>, 4> SectionRayTracingStates;

	ULandscapeComponent* ComponentUObject;

	int32 SubsectionSizeVerts;
	int32 NumSubsections;
	uint32 SharedBuffersKey;
	FLandscapeSharedBuffers* SharedBuffers;
	UMaterialInterface* MaterialInterface;
	FMatrix LocalToWorldNoScaling;
	FLandscapeBatchElementParams BatchElementParams;
	TUniquePtr<TUniformBuffer<FLandscapeUniformShaderParameters>> LandscapeUniformShaderParameters;
	TArray<TUniformBuffer<FLandscapeFixedGridUniformShaderParameters>> LandscapeFixedGridUniformShaderParameters;

	TArray<FMeshBatch> GetMeshBatchesForGBufferRendering(int32 LODIndex);
};

using FLandscapeRenderStateRef = TEntityArray<FLandscapeRenderState>::EntityRefType;

class FLandscape : public FGeometry
{
public:
	FLandscape(ULandscapeComponent* ComponentUObject);

	const class FMeshMapBuildData* GetMeshMapBuildDataForLODIndex(int32 LODIndex);

	void AllocateLightmaps(TEntityArray<FLightmap>& LightmapContainer);

	ULandscapeComponent* ComponentUObject;
	virtual UPrimitiveComponent* GetComponentUObject() const override { return ComponentUObject; }
};

using FLandscapeRef = TEntityArray<FLandscape>::EntityRefType;

}
