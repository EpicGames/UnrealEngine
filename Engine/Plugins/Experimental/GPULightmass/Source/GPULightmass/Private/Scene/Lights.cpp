// Copyright Epic Games, Inc. All Rights Reserved.

#include "Lights.h"

#include "Engine/TextureLightProfile.h"
#include "LightmapRayTracing.h"
#include "Rendering/SkyAtmosphereCommonData.h"
#include "Scene.h"
#include "RenderGraphBuilder.h"
#include "ReflectionEnvironment.h"
#include "RectLightTextureManager.h"
#include "IESTextureManager.h"
#include "Components/SkyAtmosphereComponent.h"
#include "TextureResource.h"
#include "UObject/UObjectIterator.h"
#include "RenderGraphUtils.h"

RENDERER_API void PrepareSkyTexture_Internal(
	FRDGBuilder& GraphBuilder,
	ERHIFeatureLevel::Type FeatureLevel,
	FReflectionUniformParameters& Parameters,
	uint32 Size,
	FLinearColor SkyColor,
	bool UseMISCompensation,

	// Out
	FRDGTextureRef& SkylightTexture,
	FRDGTextureRef& SkylightPdf,
	float& SkylightInvResolution,
	int32& SkylightMipCount
);

namespace GPULightmass
{

void FLightBuildInfoRef::RemoveFromAray()
{
	LightArray.Remove(*this);

	check(!IsValid());
}

FLocalLightBuildInfo& FLightBuildInfoRef::Resolve()
{
	return LightArray.ResolveAsLocalLightBuildInfo(*this);
}

FLocalLightRenderState& FLightRenderStateRef::Resolve()
{
	return LightRenderStateArray.ResolveAsLocalLightRenderState(*this);
}

FLocalLightBuildInfo::FLocalLightBuildInfo(ULightComponent* LightComponent)
{
	bStationary = LightComponent->Mobility == EComponentMobility::Stationary;
	bCastShadow = LightComponent->CastShadows && LightComponent->CastStaticShadows;
	ShadowMapChannel = LightComponent->PreviewShadowMapChannel;
	LightComponentMapBuildData = MakeShared<FLightComponentMapBuildData>();
	LightComponentMapBuildData->ShadowMapChannel = ShadowMapChannel;
}

FLocalLightRenderState::FLocalLightRenderState(ULightComponent* LightComponent)
{
	bStationary = LightComponent->Mobility == EComponentMobility::Stationary;
	bCastShadow = LightComponent->CastShadows && LightComponent->CastStaticShadows;
	ShadowMapChannel = LightComponent->PreviewShadowMapChannel;
	IESTexture = LightComponent->IESTexture ? LightComponent->IESTexture.Get() : nullptr;
}

void FLocalLightRenderState::RenderThreadInit()
{
	if (IESTexture)
	{
		IESAtlasId = IESAtlas::AddTexture(IESTexture);
	}
}

float FLocalLightRenderState::GetIESAtlasIndex() const
{
	if (IESAtlasId != INDEX_NONE)
	{
		return IESAtlas::GetAtlasSlot(IESAtlasId);
	}
	return float(INDEX_NONE);
}

void FLocalLightRenderState::RenderThreadFinalize()
{
	if (IESTexture)
	{
		IESAtlas::RemoveTexture(IESAtlasId);
	}
}

FDirectionalLightBuildInfo::FDirectionalLightBuildInfo(UDirectionalLightComponent* DirectionalLightComponent)
	: FLocalLightBuildInfo(DirectionalLightComponent)
{
	ComponentUObject = DirectionalLightComponent;
}

FDirectionalLightRenderState::FDirectionalLightRenderState(UDirectionalLightComponent* DirectionalLightComponent)
	: FLocalLightRenderState(DirectionalLightComponent)
{
	Color = DirectionalLightComponent->GetColoredLightBrightness();
	
	if (DirectionalLightComponent->IsUsedAsAtmosphereSunLight())
	{
		for (USkyAtmosphereComponent* SkyAtmosphere : TObjectRange<USkyAtmosphereComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
		{
			if (SkyAtmosphere->GetWorld() == DirectionalLightComponent->GetWorld())
			{
				FAtmosphereSetup AtmosphereSetup(*SkyAtmosphere);
				FLinearColor SunLightAtmosphereTransmittance = AtmosphereSetup.GetTransmittanceAtGroundLevel(-DirectionalLightComponent->GetDirection());
				Color *= SunLightAtmosphereTransmittance;
				break; // We only register the first we find
			}
		}
	}
	
	Direction = DirectionalLightComponent->GetDirection();
	LightSourceAngle = DirectionalLightComponent->LightSourceAngle;
	LightSourceSoftAngle = DirectionalLightComponent->LightSourceSoftAngle;
}

FPointLightBuildInfo::FPointLightBuildInfo(UPointLightComponent* PointLightComponent)
	: FLocalLightBuildInfo(PointLightComponent)
{
	ComponentUObject = PointLightComponent;
	Position = PointLightComponent->GetLightPosition();
	AttenuationRadius = PointLightComponent->AttenuationRadius;
}

bool FPointLightBuildInfo::AffectsBounds(const FBoxSphereBounds& InBounds) const
{
	return (InBounds.Origin - Position).SizeSquared() <= FMath::Square(AttenuationRadius + InBounds.SphereRadius);
}

FPointLightRenderState::FPointLightRenderState(UPointLightComponent* PointLightComponent)
	: FLocalLightRenderState(PointLightComponent)
{
	Color = PointLightComponent->GetColoredLightBrightness();
	Position = PointLightComponent->GetLightPosition();
	Direction = PointLightComponent->GetDirection();
	{
		FMatrix LightToWorld = PointLightComponent->GetComponentTransform().ToMatrixNoScale();
		Tangent = FVector(LightToWorld.M[2][0], LightToWorld.M[2][1], LightToWorld.M[2][2]);
	}
	AttenuationRadius = PointLightComponent->AttenuationRadius;
	SourceRadius = PointLightComponent->SourceRadius;
	SourceSoftRadius = PointLightComponent->SoftSourceRadius;
	SourceLength = PointLightComponent->SourceLength;
	ShadowMapChannel = PointLightComponent->PreviewShadowMapChannel;
	FalloffExponent = PointLightComponent->LightFalloffExponent;
	IsInverseSquared = PointLightComponent->bUseInverseSquaredFalloff;
}

FSpotLightBuildInfo::FSpotLightBuildInfo(USpotLightComponent* SpotLightComponent)
	: FLocalLightBuildInfo(SpotLightComponent)
{
	ComponentUObject = SpotLightComponent;
	Position = SpotLightComponent->GetLightPosition();
	Direction = SpotLightComponent->GetDirection();
	AttenuationRadius = SpotLightComponent->AttenuationRadius;

	const FVector2f ClampedConeAngles = SpotLightComponent->GetClampedConeAngles();
	InnerConeAngle = ClampedConeAngles.X;
	OuterConeAngle = ClampedConeAngles.Y;
}

bool FSpotLightBuildInfo::AffectsBounds(const FBoxSphereBounds& InBounds) const
{
	if ((InBounds.Origin - Position).SizeSquared() <= FMath::Square(AttenuationRadius + InBounds.SphereRadius))
	{
		float Sin = FMath::Sin(OuterConeAngle);
		float Cos = FMath::Cos(OuterConeAngle);

		FVector	U = Position - (InBounds.SphereRadius / Sin) * Direction;
		FVector	D = InBounds.Origin - U;
		float dsqr = D | D;
		float E = Direction | D;
		if (E > 0.0f && E * E >= dsqr * FMath::Square(Cos))
		{
			D = InBounds.Origin - Position;
			dsqr = D | D;
			E = -(Direction | D);
			if (E > 0.0f && E * E >= dsqr * FMath::Square(Sin))
				return dsqr <= FMath::Square(InBounds.SphereRadius);
			else
				return true;
		}
	}

	return false;
}

FSpotLightRenderState::FSpotLightRenderState(USpotLightComponent* SpotLightComponent)
	: FLocalLightRenderState(SpotLightComponent)
{
	Color = SpotLightComponent->GetColoredLightBrightness();
	Position = SpotLightComponent->GetLightPosition();
	Direction = SpotLightComponent->GetDirection();
	{
		const FVector2f ClampedConeAngles = SpotLightComponent->GetClampedConeAngles();
		const float CosOuterCone = FMath::Cos(ClampedConeAngles.Y);
		const float CosInnerCone = FMath::Cos(ClampedConeAngles.X);
		const float InvCosConeDifference = 1.0f / (CosInnerCone - CosOuterCone);
		SpotAngles = FVector2D(CosOuterCone, InvCosConeDifference);
	}
	{
		FMatrix LightToWorld = SpotLightComponent->GetComponentTransform().ToMatrixNoScale();
		Tangent = FVector(LightToWorld.M[2][0], LightToWorld.M[2][1], LightToWorld.M[2][2]);
	}
	AttenuationRadius = SpotLightComponent->AttenuationRadius;
	SourceRadius = SpotLightComponent->SourceRadius;
	SourceSoftRadius = SpotLightComponent->SoftSourceRadius;
	SourceLength = SpotLightComponent->SourceLength;
	ShadowMapChannel = SpotLightComponent->PreviewShadowMapChannel;
	FalloffExponent = SpotLightComponent->LightFalloffExponent;
	IsInverseSquared = SpotLightComponent->bUseInverseSquaredFalloff;
}

FRectLightBuildInfo::FRectLightBuildInfo(URectLightComponent* RectLightComponent)
	: FLocalLightBuildInfo(RectLightComponent)
{
	ComponentUObject = RectLightComponent;
	Position = RectLightComponent->GetLightPosition();
	AttenuationRadius = RectLightComponent->AttenuationRadius;
}

bool FRectLightBuildInfo::AffectsBounds(const FBoxSphereBounds& InBounds) const
{
	return (InBounds.Origin - Position).SizeSquared() <= FMath::Square(AttenuationRadius + InBounds.SphereRadius);
}

FRectLightRenderState::FRectLightRenderState(URectLightComponent* RectLightComponent)
	: FLocalLightRenderState(RectLightComponent)
{
	Color = RectLightComponent->GetColoredLightBrightness();
	Position = RectLightComponent->GetLightPosition();
	Direction = RectLightComponent->GetDirection();
	{
		FMatrix LightToWorld = RectLightComponent->GetComponentTransform().ToMatrixNoScale();
		Tangent = FVector(LightToWorld.M[2][0], LightToWorld.M[2][1], LightToWorld.M[2][2]);
	}
	SourceWidth = RectLightComponent->SourceWidth;
	SourceHeight = RectLightComponent->SourceHeight;
	BarnDoorAngle = FMath::Clamp(RectLightComponent->BarnDoorAngle, 0.f, GetRectLightBarnDoorMaxAngle());
	BarnDoorLength = FMath::Max(0.1f, RectLightComponent->BarnDoorLength);
	AttenuationRadius = RectLightComponent->AttenuationRadius;
	ShadowMapChannel = RectLightComponent->PreviewShadowMapChannel;
	SourceTexture = RectLightComponent->SourceTexture;
}

void FRectLightRenderState::RenderThreadInit()
{
	FLocalLightRenderState::RenderThreadInit();
	RectAtlasId = RectLightAtlas::AddTexture(SourceTexture);

	const RectLightAtlas::FAtlasSlotDesc Slot = RectLightAtlas::GetAtlasSlot(RectAtlasId);
	RectLightAtlasUVOffset = Slot.UVOffset;
	RectLightAtlasUVScale = Slot.UVScale;
	RectLightAtlasMaxLevel = Slot.MaxMipLevel;
}

void FRectLightRenderState::RenderThreadFinalize()
{
	FLocalLightRenderState::RenderThreadFinalize();
	RectLightAtlas::RemoveTexture(RectAtlasId);
	RectAtlasId = INDEX_NONE;
}

FLightRenderParameters FDirectionalLightRenderState::GetLightShaderParameters() const
{
	FLightRenderParameters LightParameters;

	LightParameters.WorldPosition = FVector::ZeroVector;
	LightParameters.InvRadius = 0.0f;
	// TODO: support SkyAtmosphere
	// LightParameters.Color = FVector(GetColor() * SkyAtmosphereTransmittanceToLight);
	LightParameters.Color = Color;
	LightParameters.FalloffExponent = 0.0f;

	LightParameters.Direction = (FVector3f)-Direction;					// LWC_TODO: Precision loss?
	LightParameters.Tangent = (FVector3f)-Direction;					// LWC_TODO: Precision loss?

	LightParameters.SpotAngles = FVector2f(0, 0);
	LightParameters.SpecularScale = 0; // Irrelevant when tracing shadow rays
	LightParameters.DiffuseScale = 0;  // Irrelevant when tracing shadow rays
	LightParameters.SourceRadius = FMath::Sin(0.5f * FMath::DegreesToRadians(LightSourceAngle));
	LightParameters.SoftSourceRadius = 0; // Irrelevant when tracing shadow rays. FMath::Sin(0.5f * FMath::DegreesToRadians(LightSourceSoftAngle));
	LightParameters.SourceLength = 0.0f;
	LightParameters.RectLightAtlasUVOffset = FVector2f::ZeroVector;
	LightParameters.RectLightAtlasUVScale = FVector2f::ZeroVector;
	LightParameters.RectLightAtlasMaxLevel = FLightRenderParameters::GetRectLightAtlasInvalidMIPLevel();

	return LightParameters;
}

FLightRenderParameters FPointLightRenderState::GetLightShaderParameters() const
{
	FLightRenderParameters LightParameters;
	
	LightParameters.WorldPosition = Position;
	LightParameters.Direction = (FVector3f)-Direction;					// LWC_TODO: Precision loss?
	LightParameters.Tangent = (FVector3f)Tangent;						// LWC_TODO: Precision loss?
	LightParameters.InvRadius = 1.0f / AttenuationRadius;
	LightParameters.Color = Color;
	LightParameters.SourceRadius = SourceRadius;
	LightParameters.RectLightAtlasUVOffset = FVector2f::ZeroVector;
	LightParameters.RectLightAtlasUVScale = FVector2f::ZeroVector;
	LightParameters.RectLightAtlasMaxLevel = FLightRenderParameters::GetRectLightAtlasInvalidMIPLevel();
	return LightParameters;
}

FLightRenderParameters FSpotLightRenderState::GetLightShaderParameters() const
{
	FLightRenderParameters LightParameters;

	LightParameters.WorldPosition = Position;
	LightParameters.Direction = (FVector3f)-Direction;					// LWC_TODO: Precision loss?
	LightParameters.Tangent = (FVector3f)Tangent;						// LWC_TODO: Precision loss?
	LightParameters.SpotAngles = FVector2f(SpotAngles);					// LWC_TODO: Precision loss?
	LightParameters.InvRadius = 1.0f / AttenuationRadius;
	LightParameters.Color = Color;
	LightParameters.SourceRadius = SourceRadius;
	LightParameters.RectLightAtlasUVOffset = FVector2f::ZeroVector;
	LightParameters.RectLightAtlasUVScale = FVector2f::ZeroVector;
	LightParameters.RectLightAtlasMaxLevel = FLightRenderParameters::GetRectLightAtlasInvalidMIPLevel();

	return LightParameters;
}

FLightRenderParameters FRectLightRenderState::GetLightShaderParameters() const
{
	FLightRenderParameters LightParameters;

	LightParameters.WorldPosition = Position;
	LightParameters.Direction = (FVector3f)-Direction;					// LWC_TODO: Precision loss?
	LightParameters.Tangent = (FVector3f)Tangent;						// LWC_TODO: Precision loss?
	LightParameters.InvRadius = 1.0f / AttenuationRadius;

	FLinearColor LightColor = Color;
	LightColor /= 0.5f * SourceWidth * SourceHeight;
	LightParameters.Color = LightColor;

	LightParameters.SourceRadius = SourceWidth * 0.5f;
	LightParameters.SourceLength = SourceHeight * 0.5f;
	LightParameters.RectLightBarnCosAngle = FMath::Cos(FMath::DegreesToRadians(BarnDoorAngle));
	LightParameters.RectLightBarnLength = BarnDoorLength;
	
	return LightParameters;
}

void FSkyLightRenderState::PrepareSkyTexture(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type FeatureLevel)
{
	FRDGBuilder GraphBuilder(RHICmdList);

	FRDGTextureRef SkylightTexture;
	FRDGTextureRef SkylightPdf;

	FReflectionUniformParameters Parameters;
	Parameters.SkyLightCubemap = RegisterExternalTexture(GraphBuilder, ProcessedTexture, TEXT("SkyLightTexture"));
	Parameters.SkyLightCubemapSampler = ProcessedTextureSampler;
	Parameters.SkyLightBlendDestinationCubemap = ProcessedTexture;
	Parameters.SkyLightBlendDestinationCubemapSampler = ProcessedTextureSampler;
	Parameters.SkyLightParameters = FVector4f(1, 1, 0, 0);

	FLinearColor SkyColor = Color;
	// since we are resampled into an octahedral layout, we multiply the cubemap resolution by 2 to get roughly the same number of texels
	uint32 Size = FMath::RoundUpToPowerOfTwo(2 * TextureDimensions.X);

	const bool UseMISCompensation = true;

	PrepareSkyTexture_Internal(
		GraphBuilder,
		FeatureLevel,
		Parameters,
		Size,
		SkyColor,
		UseMISCompensation,
		// Out
		SkylightTexture,
		SkylightPdf,
		SkylightInvResolution,
		SkylightMipCount
	);

	GraphBuilder.QueueTextureExtraction(SkylightTexture, &PathTracingSkylightTexture);
	GraphBuilder.QueueTextureExtraction(SkylightPdf, &PathTracingSkylightPdf);

	GraphBuilder.Execute();
}

void FDirectionalLightRenderState::RenderStaticShadowDepthMap(FRHICommandListImmediate& RHICmdList, FSceneRenderState& Scene)
{
	FRDGBuilder GraphBuilder(RHICmdList);
	FSceneUniformBuffer SceneUniforms;

	if (!Scene.SetupRayTracingScene(GraphBuilder, SceneUniforms))
	{
		return;
	}
	
	FVector XAxis, YAxis;
	Direction.FindBestAxisVectors(XAxis, YAxis);
	// Create a coordinate system for the dominant directional light, with the z axis corresponding to the light's direction
	FMatrix WorldToLight = FBasisVectorMatrix(XAxis, YAxis, Direction, FVector4(0, 0, 0));

	const FBox LightSpaceImportanceBounds = Scene.CombinedImportanceVolume.TransformBy(WorldToLight);

	const float ClampedResolutionScale = 1.0f;
	const float StaticShadowDepthMapTransitionSampleDistanceX = 10;
	const float StaticShadowDepthMapTransitionSampleDistanceY = 10;
	
	int32 ShadowMapSizeX = FMath::TruncToInt(FMath::Max(LightSpaceImportanceBounds.GetExtent().X * 2.0f * ClampedResolutionScale / StaticShadowDepthMapTransitionSampleDistanceX, 4.0f));
	int32 ShadowMapSizeY = FMath::TruncToInt(FMath::Max(LightSpaceImportanceBounds.GetExtent().Y * 2.0f * ClampedResolutionScale / StaticShadowDepthMapTransitionSampleDistanceY, 4.0f));

	const uint64 StaticShadowDepthMapMaxSamples = 16777216;
	
	// Clamp the number of dominant shadow samples generated if necessary while maintaining aspect ratio
	if ((uint64)ShadowMapSizeX * (uint64)ShadowMapSizeY > StaticShadowDepthMapMaxSamples)
	{
		const float AspectRatio = ShadowMapSizeX / (float)ShadowMapSizeY;
		ShadowMapSizeY = FMath::TruncToInt(FMath::Sqrt(StaticShadowDepthMapMaxSamples / AspectRatio));
		ShadowMapSizeX = FMath::TruncToInt(StaticShadowDepthMapMaxSamples / (float)ShadowMapSizeY);
	}
	
	FRDGTextureRef DepthMapTexture = GraphBuilder.CreateTexture(
		FRDGTextureDesc::Create2D(
			FIntPoint{ShadowMapSizeX, ShadowMapSizeY},
			PF_R16F,
			FClearValueBinding::DepthFar,
			ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV
		), TEXT("GPULMStaticShadowDepthMap"));

	FStaticShadowDepthMapTracingRGS::FParameters* PassParameters = GraphBuilder.AllocParameters<FStaticShadowDepthMapTracingRGS::FParameters>();
	PassParameters->ViewUniformBuffer = Scene.ReferenceView->ViewUniformBuffer;
	PassParameters->TLAS = Scene.RayTracingSceneSRV;
	PassParameters->ShadowMapSize = FIntPoint{ShadowMapSizeX, ShadowMapSizeY};
	PassParameters->StaticShadowDepthMapSuperSampleFactor = 1;
	PassParameters->LightSpaceImportanceBoundsMin = FVector3f(LightSpaceImportanceBounds.Min);
	PassParameters->LightSpaceImportanceBoundsMax = FVector3f(LightSpaceImportanceBounds.Max);
	PassParameters->LightToWorld = FMatrix44f(WorldToLight.InverseFast());
	PassParameters->MaxPossibleDistance = LightSpaceImportanceBounds.Max.Z - LightSpaceImportanceBounds.Min.Z;
	PassParameters->DepthMapTexture = GraphBuilder.CreateUAV(DepthMapTexture);

	FStaticShadowDepthMapTracingRGS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FStaticShadowDepthMapTracingRGS::FLightType>(0);
	auto RayGenShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FStaticShadowDepthMapTracingRGS>(PermutationVector);
	ClearUnusedGraphResources(RayGenShader, PassParameters);
	
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("StaticShadowDepthMapTracing"),
		PassParameters,
		ERDGPassFlags::Compute,
		[PassParameters, RayGenShader, RayTracingPipelineState = Scene.RayTracingPipelineState, SBT = Scene.SBT](FRHICommandList& RHICmdList)
	{
		FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
		SetShaderParameters(GlobalResources, RayGenShader, *PassParameters);

		RHICmdList.RayTraceDispatch(RayTracingPipelineState, RayGenShader.GetRayTracingShader(), SBT, GlobalResources, PassParameters->ShadowMapSize.X, PassParameters->ShadowMapSize.Y);
	}
	);

	FRHIGPUTextureReadback DepthMapTextureReadback(TEXT("GPULMStaticShadowDepthMapReadback"));
	
	AddEnqueueCopyPass(GraphBuilder, &DepthMapTextureReadback, DepthMapTexture);
	
	GraphBuilder.Execute();

	RHICmdList.SubmitAndBlockUntilGPUIdle();

	check(DepthMapTextureReadback.IsReady());

	int32 RowPitchInPixels;
	FFloat16* LockedData = (FFloat16*)DepthMapTextureReadback.Lock(RowPitchInPixels);
	LightComponentMapBuildData->DepthMap.Empty();
	LightComponentMapBuildData->DepthMap.DepthSamples.AddZeroed(ShadowMapSizeX * ShadowMapSizeY);
	for (int32 Y = 0; Y < ShadowMapSizeY; Y++)
	{
		for (int32 X = 0; X < ShadowMapSizeX; X++)
		{
			LightComponentMapBuildData->DepthMap.DepthSamples[Y * ShadowMapSizeX + X] = LockedData[Y * RowPitchInPixels + X];
		}
	}
	LightComponentMapBuildData->DepthMap.ShadowMapSizeX = ShadowMapSizeX;
	LightComponentMapBuildData->DepthMap.ShadowMapSizeY = ShadowMapSizeY;
	WorldToLight *= FTranslationMatrix(-LightSpaceImportanceBounds.Min) * FScaleMatrix(FVector(1.0) / (LightSpaceImportanceBounds.Max - LightSpaceImportanceBounds.Min));
	LightComponentMapBuildData->DepthMap.WorldToLight = WorldToLight;
	DepthMapTextureReadback.Unlock();
	
	Scene.DestroyRayTracingScene();
}

void FSpotLightRenderState::RenderStaticShadowDepthMap(FRHICommandListImmediate& RHICmdList, FSceneRenderState& Scene)
{
	FRDGBuilder GraphBuilder(RHICmdList);
	FSceneUniformBuffer SceneUniforms;

	if (!Scene.SetupRayTracingScene(GraphBuilder, SceneUniforms))
	{
		return;
	}
	
	FVector XAxis, YAxis;
	Direction.FindBestAxisVectors(XAxis, YAxis);
	// Create a coordinate system for the dominant directional light, with the z axis corresponding to the light's direction
	FMatrix WorldToLight = FTranslationMatrix(-Position) * FBasisVectorMatrix(XAxis, YAxis, Direction, FVector4(0, 0, 0));

	// Distance from the light's direction axis to the edge of the cone at the radius of the light
	const float HalfCrossSectionLength = AttenuationRadius * FMath::Tan(FMath::Acos(SpotAngles.X));
	
	const FBox LightSpaceImportanceBounds
	{
		FVector{-HalfCrossSectionLength, -HalfCrossSectionLength, 0},
		FVector{HalfCrossSectionLength, HalfCrossSectionLength, AttenuationRadius},
	};
	
	const float ClampedResolutionScale = 1.0f;
	const float StaticShadowDepthMapTransitionSampleDistanceX = 10;
	const float StaticShadowDepthMapTransitionSampleDistanceY = 10;
	
	int32 ShadowMapSizeX = FMath::TruncToInt(FMath::Max(HalfCrossSectionLength * ClampedResolutionScale / StaticShadowDepthMapTransitionSampleDistanceX, 4.0f));
	int32 ShadowMapSizeY = ShadowMapSizeX;

	const uint64 StaticShadowDepthMapMaxSamples = 16777216;
	
	// Clamp the number of dominant shadow samples generated if necessary while maintaining aspect ratio
	if ((uint64)ShadowMapSizeX * (uint64)ShadowMapSizeY > StaticShadowDepthMapMaxSamples)
	{
		const float AspectRatio = ShadowMapSizeX / (float)ShadowMapSizeY;
		ShadowMapSizeY = FMath::TruncToInt(FMath::Sqrt(StaticShadowDepthMapMaxSamples / AspectRatio));
		ShadowMapSizeX = FMath::TruncToInt(StaticShadowDepthMapMaxSamples / (float)ShadowMapSizeY);
	}
	
	FRDGTextureRef DepthMapTexture = GraphBuilder.CreateTexture(
		FRDGTextureDesc::Create2D(
			FIntPoint{ShadowMapSizeX, ShadowMapSizeY},
			PF_R16F,
			FClearValueBinding::DepthFar,
			ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV
		), TEXT("GPULMStaticShadowDepthMap"));

	const float MaxPossibleDistance = LightSpaceImportanceBounds.Max.Z - LightSpaceImportanceBounds.Min.Z;
	const FMatrix LightToWorld = WorldToLight.InverseFast();

	FStaticShadowDepthMapTracingRGS::FParameters* PassParameters = GraphBuilder.AllocParameters<FStaticShadowDepthMapTracingRGS::FParameters>();
	PassParameters->ViewUniformBuffer = Scene.ReferenceView->ViewUniformBuffer;
	PassParameters->TLAS = Scene.RayTracingSceneSRV;
	PassParameters->ShadowMapSize = FIntPoint{ShadowMapSizeX, ShadowMapSizeY};
	PassParameters->StaticShadowDepthMapSuperSampleFactor = 1;
	PassParameters->LightSpaceImportanceBoundsMin = FVector3f(LightSpaceImportanceBounds.Min);
	PassParameters->LightSpaceImportanceBoundsMax = FVector3f(LightSpaceImportanceBounds.Max);
	PassParameters->LightToWorld = FMatrix44f(LightToWorld);
	PassParameters->WorldToLight = FMatrix44f(WorldToLight);
	PassParameters->MaxPossibleDistance = MaxPossibleDistance;
	PassParameters->DepthMapTexture = GraphBuilder.CreateUAV(DepthMapTexture);

	FStaticShadowDepthMapTracingRGS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FStaticShadowDepthMapTracingRGS::FLightType>(1);
	auto RayGenShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FStaticShadowDepthMapTracingRGS>(PermutationVector);
	ClearUnusedGraphResources(RayGenShader, PassParameters);
	
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("StaticShadowDepthMapTracing"),
		PassParameters,
		ERDGPassFlags::Compute,
		[PassParameters, RayGenShader, RayTracingPipelineState = Scene.RayTracingPipelineState, SBT = Scene.SBT](FRHICommandList& RHICmdList)
	{
		FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
		SetShaderParameters(GlobalResources, RayGenShader, *PassParameters);

		RHICmdList.RayTraceDispatch(RayTracingPipelineState, RayGenShader.GetRayTracingShader(), SBT, GlobalResources, PassParameters->ShadowMapSize.X, PassParameters->ShadowMapSize.Y);
	}
	);

	FRHIGPUTextureReadback DepthMapTextureReadback(TEXT("GPULMStaticShadowDepthMapReadback"));
	
	AddEnqueueCopyPass(GraphBuilder, &DepthMapTextureReadback, DepthMapTexture);
	
	GraphBuilder.Execute();

	RHICmdList.SubmitAndBlockUntilGPUIdle();

	check(DepthMapTextureReadback.IsReady());

	int32 RowPitchInPixels;
	FFloat16* LockedData = (FFloat16*)DepthMapTextureReadback.Lock(RowPitchInPixels);
	LightComponentMapBuildData->DepthMap.Empty();
	LightComponentMapBuildData->DepthMap.DepthSamples.AddZeroed(ShadowMapSizeX * ShadowMapSizeY);
	for (int32 Y = 0; Y < ShadowMapSizeY; Y++)
	{
		for (int32 X = 0; X < ShadowMapSizeX; X++)
		{
			LightComponentMapBuildData->DepthMap.DepthSamples[Y * ShadowMapSizeX + X] = LockedData[Y * RowPitchInPixels + X];
		}
	}
	LightComponentMapBuildData->DepthMap.ShadowMapSizeX = ShadowMapSizeX;
	LightComponentMapBuildData->DepthMap.ShadowMapSizeY = ShadowMapSizeY;
	WorldToLight *=
		// Perspective projection sized to the spotlight cone
		FPerspectiveMatrix(FMath::Acos(SpotAngles.X), 1, 1, 0, AttenuationRadius)
		// Convert from NDC to texture space, normalize Z
		* FMatrix(
			FPlane(.5,	0,		0,											0),
			FPlane(0,	.5,	0,											0),
			FPlane(0,	0,		1.0 / LightSpaceImportanceBounds.Max.Z,	0),
			FPlane(.5,	.5,	0,											1));
	LightComponentMapBuildData->DepthMap.WorldToLight = WorldToLight;
	DepthMapTextureReadback.Unlock();
	
	Scene.DestroyRayTracingScene();
}

void FPointLightRenderState::RenderStaticShadowDepthMap(FRHICommandListImmediate& RHICmdList, FSceneRenderState& Scene)
{
	FRDGBuilder GraphBuilder(RHICmdList);
	FSceneUniformBuffer SceneUniforms;

	if (!Scene.SetupRayTracingScene(GraphBuilder, SceneUniforms))
	{
		return;
	}

	const float ClampedResolutionScale = 1.0f;
	const float StaticShadowDepthMapTransitionSampleDistanceX = 10;
	const float StaticShadowDepthMapTransitionSampleDistanceY = 10;
	
	int32 ShadowMapSizeX = FMath::TruncToInt(FMath::Max(AttenuationRadius * 4 * ClampedResolutionScale / StaticShadowDepthMapTransitionSampleDistanceX, 4.0f));
	int32 ShadowMapSizeY = ShadowMapSizeX;

	const uint64 StaticShadowDepthMapMaxSamples = 16777216;
	
	// Clamp the number of dominant shadow samples generated if necessary while maintaining aspect ratio
	if ((uint64)ShadowMapSizeX * (uint64)ShadowMapSizeY > StaticShadowDepthMapMaxSamples)
	{
		const float AspectRatio = ShadowMapSizeX / (float)ShadowMapSizeY;
		ShadowMapSizeY = FMath::TruncToInt(FMath::Sqrt(StaticShadowDepthMapMaxSamples / AspectRatio));
		ShadowMapSizeX = FMath::TruncToInt(StaticShadowDepthMapMaxSamples / (float)ShadowMapSizeY);
	}
	
	FRDGTextureRef DepthMapTexture = GraphBuilder.CreateTexture(
		FRDGTextureDesc::Create2D(
			FIntPoint{ShadowMapSizeX, ShadowMapSizeY},
			PF_R16F,
			FClearValueBinding::DepthFar,
			ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV
		), TEXT("GPULMStaticShadowDepthMap"));

	FStaticShadowDepthMapTracingRGS::FParameters* PassParameters = GraphBuilder.AllocParameters<FStaticShadowDepthMapTracingRGS::FParameters>();
	PassParameters->ViewUniformBuffer = Scene.ReferenceView->ViewUniformBuffer;
	PassParameters->TLAS = Scene.RayTracingSceneSRV;
	PassParameters->ShadowMapSize = FIntPoint{ShadowMapSizeX, ShadowMapSizeY};
	PassParameters->StaticShadowDepthMapSuperSampleFactor = 1;
	PassParameters->LightSpaceImportanceBoundsMin = FVector3f{EForceInit::ForceInitToZero};
	PassParameters->LightSpaceImportanceBoundsMax = FVector3f{EForceInit::ForceInitToZero};
	PassParameters->LightToWorld = FTranslationMatrix44f(FVector3f(Position));
	PassParameters->WorldToLight = FTranslationMatrix44f(-FVector3f(Position));
	PassParameters->MaxPossibleDistance = AttenuationRadius;
	PassParameters->DepthMapTexture = GraphBuilder.CreateUAV(DepthMapTexture);

	FStaticShadowDepthMapTracingRGS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FStaticShadowDepthMapTracingRGS::FLightType>(2);
	auto RayGenShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FStaticShadowDepthMapTracingRGS>(PermutationVector);
	ClearUnusedGraphResources(RayGenShader, PassParameters);
	
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("StaticShadowDepthMapTracing"),
		PassParameters,
		ERDGPassFlags::Compute,
		[PassParameters, RayGenShader, RayTracingPipelineState = Scene.RayTracingPipelineState, SBT = Scene.SBT](FRHICommandList& RHICmdList)
	{
		FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
		SetShaderParameters(GlobalResources, RayGenShader, *PassParameters);

		RHICmdList.RayTraceDispatch(RayTracingPipelineState, RayGenShader.GetRayTracingShader(), SBT, GlobalResources, PassParameters->ShadowMapSize.X, PassParameters->ShadowMapSize.Y);
	}
	);

	FRHIGPUTextureReadback DepthMapTextureReadback(TEXT("GPULMStaticShadowDepthMapReadback"));
	
	AddEnqueueCopyPass(GraphBuilder, &DepthMapTextureReadback, DepthMapTexture);
	
	GraphBuilder.Execute();

	RHICmdList.SubmitAndBlockUntilGPUIdle();

	check(DepthMapTextureReadback.IsReady());

	int32 RowPitchInPixels;
	FFloat16* LockedData = (FFloat16*)DepthMapTextureReadback.Lock(RowPitchInPixels);
	LightComponentMapBuildData->DepthMap.Empty();
	LightComponentMapBuildData->DepthMap.DepthSamples.AddZeroed(ShadowMapSizeX * ShadowMapSizeY);
	for (int32 Y = 0; Y < ShadowMapSizeY; Y++)
	{
		for (int32 X = 0; X < ShadowMapSizeX; X++)
		{
			LightComponentMapBuildData->DepthMap.DepthSamples[Y * ShadowMapSizeX + X] = LockedData[Y * RowPitchInPixels + X];
		}
	}
	LightComponentMapBuildData->DepthMap.ShadowMapSizeX = ShadowMapSizeX;
	LightComponentMapBuildData->DepthMap.ShadowMapSizeY = ShadowMapSizeY;
	LightComponentMapBuildData->DepthMap.WorldToLight = FMatrix::Identity;
	DepthMapTextureReadback.Unlock();
	
	Scene.DestroyRayTracingScene();
}

void FRectLightRenderState::RenderStaticShadowDepthMap(FRHICommandListImmediate& RHICmdList, FSceneRenderState& Scene)
{
	FRDGBuilder GraphBuilder(RHICmdList);
	FSceneUniformBuffer SceneUniforms;

	if (!Scene.SetupRayTracingScene(GraphBuilder, SceneUniforms))
	{
		return;
	}

	const float ClampedResolutionScale = 1.0f;
	const float StaticShadowDepthMapTransitionSampleDistanceX = 10;
	const float StaticShadowDepthMapTransitionSampleDistanceY = 10;
	
	int32 ShadowMapSizeX = FMath::TruncToInt(FMath::Max(AttenuationRadius * 4 * ClampedResolutionScale / StaticShadowDepthMapTransitionSampleDistanceX, 4.0f));
	int32 ShadowMapSizeY = ShadowMapSizeX;

	const uint64 StaticShadowDepthMapMaxSamples = 16777216;
	
	// Clamp the number of dominant shadow samples generated if necessary while maintaining aspect ratio
	if ((uint64)ShadowMapSizeX * (uint64)ShadowMapSizeY > StaticShadowDepthMapMaxSamples)
	{
		const float AspectRatio = ShadowMapSizeX / (float)ShadowMapSizeY;
		ShadowMapSizeY = FMath::TruncToInt(FMath::Sqrt(StaticShadowDepthMapMaxSamples / AspectRatio));
		ShadowMapSizeX = FMath::TruncToInt(StaticShadowDepthMapMaxSamples / (float)ShadowMapSizeY);
	}
	
	FRDGTextureRef DepthMapTexture = GraphBuilder.CreateTexture(
		FRDGTextureDesc::Create2D(
			FIntPoint{ShadowMapSizeX, ShadowMapSizeY},
			PF_R16F,
			FClearValueBinding::DepthFar,
			ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV
		), TEXT("GPULMStaticShadowDepthMap"));

	FStaticShadowDepthMapTracingRGS::FParameters* PassParameters = GraphBuilder.AllocParameters<FStaticShadowDepthMapTracingRGS::FParameters>();
	PassParameters->ViewUniformBuffer = Scene.ReferenceView->ViewUniformBuffer;
	PassParameters->TLAS = Scene.RayTracingSceneSRV;
	PassParameters->ShadowMapSize = FIntPoint{ShadowMapSizeX, ShadowMapSizeY};
	PassParameters->StaticShadowDepthMapSuperSampleFactor = 1;
	PassParameters->LightSpaceImportanceBoundsMin = FVector3f{EForceInit::ForceInitToZero};
	PassParameters->LightSpaceImportanceBoundsMax = FVector3f{EForceInit::ForceInitToZero};
	PassParameters->LightToWorld = FTranslationMatrix44f(FVector3f(Position));
	PassParameters->WorldToLight = FTranslationMatrix44f(-FVector3f(Position));
	PassParameters->MaxPossibleDistance = AttenuationRadius;
	PassParameters->DepthMapTexture = GraphBuilder.CreateUAV(DepthMapTexture);

	FStaticShadowDepthMapTracingRGS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FStaticShadowDepthMapTracingRGS::FLightType>(2);
	auto RayGenShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FStaticShadowDepthMapTracingRGS>(PermutationVector);
	ClearUnusedGraphResources(RayGenShader, PassParameters);
	
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("StaticShadowDepthMapTracing"),
		PassParameters,
		ERDGPassFlags::Compute,
		[PassParameters, RayGenShader, RayTracingPipelineState = Scene.RayTracingPipelineState, SBT = Scene.SBT](FRHICommandList& RHICmdList)
	{
		FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
		SetShaderParameters(GlobalResources, RayGenShader, *PassParameters);

		RHICmdList.RayTraceDispatch(RayTracingPipelineState, RayGenShader.GetRayTracingShader(), SBT, GlobalResources, PassParameters->ShadowMapSize.X, PassParameters->ShadowMapSize.Y);
	}
	);

	FRHIGPUTextureReadback DepthMapTextureReadback(TEXT("GPULMStaticShadowDepthMapReadback"));
	
	AddEnqueueCopyPass(GraphBuilder, &DepthMapTextureReadback, DepthMapTexture);
	
	GraphBuilder.Execute();

	RHICmdList.SubmitAndBlockUntilGPUIdle();

	check(DepthMapTextureReadback.IsReady());

	int32 RowPitchInPixels;
	FFloat16* LockedData = (FFloat16*)DepthMapTextureReadback.Lock(RowPitchInPixels);
	LightComponentMapBuildData->DepthMap.Empty();
	LightComponentMapBuildData->DepthMap.DepthSamples.AddZeroed(ShadowMapSizeX * ShadowMapSizeY);
	for (int32 Y = 0; Y < ShadowMapSizeY; Y++)
	{
		for (int32 X = 0; X < ShadowMapSizeX; X++)
		{
			LightComponentMapBuildData->DepthMap.DepthSamples[Y * ShadowMapSizeX + X] = LockedData[Y * RowPitchInPixels + X];
		}
	}
	LightComponentMapBuildData->DepthMap.ShadowMapSizeX = ShadowMapSizeX;
	LightComponentMapBuildData->DepthMap.ShadowMapSizeY = ShadowMapSizeY;
	LightComponentMapBuildData->DepthMap.WorldToLight = FMatrix::Identity;
	DepthMapTextureReadback.Unlock();
	
	Scene.DestroyRayTracingScene();
}

}
