// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntityArray.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SkyLightComponent.h"
#include "GeometryInterface.h"
#include "Engine/MapBuildDataRegistry.h"

class FSkyLightImportanceSamplingData;

struct FLightShaderConstants
{
	FVector3f	PositionHigh;
	float		InvRadius;
	FVector3f	Color;
	float		FalloffExponent;
	FVector3f	Direction;
	float		SpecularScale;
	FVector3f	Tangent;
	float		SourceRadius;
	FVector3f	PositionLow;
	float		SourceLength;
	FVector2f	SpotAngles;
	float		SoftSourceRadius;
	float		RectLightBarnCosAngle;
	float		RectLightBarnLength;
	float		DiffuseScale;

	FLightShaderConstants() = default;

	FLightShaderConstants(const FLightRenderParameters& LightShaderParameters)
	{
		const FDFVector3 AbsoluteWorldPosition(LightShaderParameters.WorldPosition);

		PositionHigh = AbsoluteWorldPosition.High;
		InvRadius = LightShaderParameters.InvRadius;
		Color = FVector3f(LightShaderParameters.Color);
		FalloffExponent = LightShaderParameters.FalloffExponent;
		Direction = LightShaderParameters.Direction;
		SpecularScale = LightShaderParameters.SpecularScale;
		DiffuseScale = LightShaderParameters.DiffuseScale;
		Tangent = LightShaderParameters.Tangent;
		SourceRadius = LightShaderParameters.SourceRadius;
		PositionLow = AbsoluteWorldPosition.Low;
		SourceLength = LightShaderParameters.SourceLength;
		SpotAngles = LightShaderParameters.SpotAngles;
		SoftSourceRadius = LightShaderParameters.SoftSourceRadius;
		RectLightBarnCosAngle = LightShaderParameters.RectLightBarnCosAngle;
		RectLightBarnLength = LightShaderParameters.RectLightBarnLength;
	}
};

namespace GPULightmass
{

/**
 * BuildInfo store extra game thread data for GPULightmass's internal usage beyond the component
 * RenderState is GPULightmass's equivalent for scene proxies
 */

struct FLightSceneRenderState;

struct FLocalLightBuildInfo
{
	FLocalLightBuildInfo(ULightComponent* LightComponent);
	virtual ~FLocalLightBuildInfo() {}
	FLocalLightBuildInfo(FLocalLightBuildInfo&& In) = default; // Default move constructor is killed by the custom destructor

	bool bStationary = false;
	bool bCastShadow = true;
	int ShadowMapChannel = INDEX_NONE;

	// This will also be held by FLocalLightRenderState to extend its lifetime until RenderThread has finished with it
	TSharedPtr<FLightComponentMapBuildData, ESPMode::ThreadSafe> LightComponentMapBuildData;

	virtual bool AffectsBounds(const FBoxSphereBounds& InBounds) const = 0;
	virtual ULightComponent* GetComponentUObject() const = 0;
	bool CastsStationaryShadow() { return bStationary && bCastShadow; }

	void AllocateMapBuildData(ULevel* StorageLevel);
};

struct FLocalLightRenderState
{
	FLocalLightRenderState(ULightComponent* LightComponent);
	virtual void RenderThreadInit();
	virtual void RenderThreadFinalize();
	virtual ~FLocalLightRenderState() {}

	bool bStationary = false;
	bool bCastShadow = true;
	int ShadowMapChannel = INDEX_NONE;
	uint32 IESAtlasId = INDEX_NONE;
	UTextureLightProfile* IESTexture = nullptr;
	
	float GetIESAtlasIndex() const;

	TSharedPtr<FLightComponentMapBuildData, ESPMode::ThreadSafe> LightComponentMapBuildData;

	virtual FLightRenderParameters GetLightShaderParameters() const = 0;
	virtual void RenderStaticShadowDepthMap(FRHICommandListImmediate& RHICmdList, class FSceneRenderState& RenderState) {}
};

class FLightArrayBase;

class FLightBuildInfoRef : public FGenericEntityRef
{
public:
	FLightBuildInfoRef(
		FLightArrayBase& LightArray,
		TArray<TSet<RefAddr>>& Refs,
		TSparseArray<int32>& RefAllocator,
		int32 ElementId)
		: FGenericEntityRef(ElementId, Refs, RefAllocator)
		, LightArray(LightArray)
	{}

	void RemoveFromAray();
	FLocalLightBuildInfo& Resolve();

private:
	FLightArrayBase& LightArray;

	template<typename T>
	friend class TLightArray;
};

class FLightRenderStateArrayBase;

class FLightRenderStateRef : public FGenericEntityRef
{
public:
	FLightRenderStateRef(
		FLightRenderStateArrayBase& LightArray,
		TArray<TSet<RefAddr>>& Refs,
		TSparseArray<int32>& RefAllocator,
		int32 ElementId)
		: FGenericEntityRef(ElementId, Refs, RefAllocator)
		, LightRenderStateArray(LightArray)
	{}

	FLightRenderStateRef(FLightRenderStateArrayBase& LightArray, const FGenericEntityRef& InRef)
		: FGenericEntityRef(InRef)
		, LightRenderStateArray(LightArray)
	{}

	FLocalLightRenderState& Resolve();

private:
	FLightRenderStateArrayBase& LightRenderStateArray;

	template<typename T>
	friend class TLightRenderStateArray;
};

struct FDirectionalLightBuildInfo : public FLocalLightBuildInfo
{
	FDirectionalLightBuildInfo(UDirectionalLightComponent* DirectionalLightComponent);

	UDirectionalLightComponent* ComponentUObject = nullptr;

	virtual ULightComponent* GetComponentUObject() const override { return ComponentUObject; }
	virtual bool AffectsBounds(const FBoxSphereBounds& InBounds) const override { return true; }
};

using FDirectionalLightRef = TEntityArray<FDirectionalLightBuildInfo>::EntityRefType;

struct FPointLightBuildInfo : public FLocalLightBuildInfo
{
	FPointLightBuildInfo(UPointLightComponent* ComponentUObject);

	UPointLightComponent* ComponentUObject = nullptr;

	FVector Position;
	float AttenuationRadius;

	virtual ULightComponent* GetComponentUObject() const override { return ComponentUObject; }
	virtual bool AffectsBounds(const FBoxSphereBounds& InBounds) const override;
};

using FPointLightRef = TEntityArray<FPointLightBuildInfo>::EntityRefType;

struct FSpotLightBuildInfo : public FLocalLightBuildInfo
{
	FSpotLightBuildInfo(USpotLightComponent* ComponentUObject);

	USpotLightComponent* ComponentUObject = nullptr;

	FVector Position;
	FVector Direction;
	float InnerConeAngle;
	float OuterConeAngle;
	float AttenuationRadius;

	virtual ULightComponent* GetComponentUObject() const override { return ComponentUObject; }
	virtual bool AffectsBounds(const FBoxSphereBounds& InBounds) const override;
};

using FSpotLightRef = TEntityArray<FSpotLightBuildInfo>::EntityRefType;

struct FRectLightBuildInfo : public FLocalLightBuildInfo
{
	FRectLightBuildInfo(URectLightComponent* ComponentUObject);

	URectLightComponent* ComponentUObject = nullptr;

	FVector Position;
	float AttenuationRadius;

	virtual ULightComponent* GetComponentUObject() const override { return ComponentUObject; }
	virtual bool AffectsBounds(const FBoxSphereBounds& InBounds) const override;
};

using FRectLightRef = TEntityArray<FRectLightBuildInfo>::EntityRefType;

struct FDirectionalLightRenderState : public FLocalLightRenderState
{
	FDirectionalLightRenderState(UDirectionalLightComponent* DirectionalLightComponent);

	FVector Direction;
	FLinearColor Color;
	float LightSourceAngle;
	float LightSourceSoftAngle;

	virtual FLightRenderParameters GetLightShaderParameters() const override;
	virtual void RenderStaticShadowDepthMap(FRHICommandListImmediate& RHICmdList, FSceneRenderState& RenderState) override;
};

using FDirectionalLightRenderStateRef = TEntityArray<FDirectionalLightRenderState>::EntityRefType;

struct FPointLightRenderState : public FLocalLightRenderState
{
	FPointLightRenderState(UPointLightComponent* PointLightComponent);

	FVector Position;
	FVector Direction;
	FVector Tangent;
	FLinearColor Color;
	float AttenuationRadius;
	float SourceRadius;
	float SourceSoftRadius;
	float SourceLength;
	float FalloffExponent;
	bool IsInverseSquared;

	virtual FLightRenderParameters GetLightShaderParameters() const override;
	virtual void RenderStaticShadowDepthMap(FRHICommandListImmediate& RHICmdList, FSceneRenderState& RenderState) override;
};

using FPointLightRenderStateRef = TEntityArray<FPointLightRenderState>::EntityRefType;

struct FSpotLightRenderState : public FLocalLightRenderState
{
	FSpotLightRenderState(USpotLightComponent* ComponentUObject);

	FVector Position;
	FVector Direction;
	FVector Tangent;
	FVector2D SpotAngles;
	FLinearColor Color;
	float AttenuationRadius;
	float SourceRadius;
	float SourceSoftRadius;
	float SourceLength;
	float FalloffExponent;
	bool IsInverseSquared;

	virtual FLightRenderParameters GetLightShaderParameters() const override;
	virtual void RenderStaticShadowDepthMap(FRHICommandListImmediate& RHICmdList, FSceneRenderState& RenderState) override;
};

using FSpotLightRenderStateRef = TEntityArray<FSpotLightRenderState>::EntityRefType;

struct FRectLightRenderState : public FLocalLightRenderState
{
	FRectLightRenderState(URectLightComponent* ComponentUObject);
	virtual void RenderThreadInit() override;
	virtual void RenderThreadFinalize() override;
	
	FLinearColor Color;
	float AttenuationRadius;
	FVector Position;
	FVector Direction;
	FVector Tangent;
	float SourceWidth;
	float SourceHeight;
	float BarnDoorAngle;
	float BarnDoorLength;
	UTexture* SourceTexture;
	uint32 RectAtlasId;
	FVector2f RectLightAtlasUVOffset;
	FVector2f RectLightAtlasUVScale;
	float RectLightAtlasMaxLevel;

	virtual FLightRenderParameters GetLightShaderParameters() const override;
	virtual void RenderStaticShadowDepthMap(FRHICommandListImmediate& RHICmdList, FSceneRenderState& RenderState) override;
};

using FRectLightRenderStateRef = TEntityArray<FRectLightRenderState>::EntityRefType;

struct FSkyLightRenderState
{
	bool bStationary = false;
	bool bCastShadow = true;	
	bool CastsStationaryShadow() { return bStationary && bCastShadow; }
	
	FLinearColor Color;
	FTextureRHIRef ProcessedTexture;
	FSamplerStateRHIRef ProcessedTextureSampler;
	FIntPoint TextureDimensions;
	FSHVectorRGB3 IrradianceEnvironmentMap;
	FRWBufferStructured SkyIrradianceEnvironmentMap;

	// New sky dome
	void PrepareSkyTexture(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type FeatureLevel);
	float SkylightInvResolution;
	int32 SkylightMipCount;
	TRefCountPtr<IPooledRenderTarget> PathTracingSkylightTexture;
	TRefCountPtr<IPooledRenderTarget> PathTracingSkylightPdf;
};

struct FSkyLightBuildInfo
{
	USkyLightComponent* ComponentUObject = nullptr;

	bool bStationary = false;
	bool bCastShadow = true;	
	bool CastsStationaryShadow() { return bStationary && bCastShadow; }
};

class FLightArrayBase
{
public:
	virtual ~FLightArrayBase() {}
	virtual void Remove(const FLightBuildInfoRef& Light) = 0;
	virtual FLocalLightBuildInfo& ResolveAsLocalLightBuildInfo(const FLightBuildInfoRef& Light) = 0;
};

template<typename T>
class TLightArray : public FLightArrayBase, public TEntityArray<T>
{
public:
	virtual void Remove(const FLightBuildInfoRef& Light) override
	{
		check(&Light.LightArray == (FLightArrayBase*)this);

		this->RemoveAt(Light.GetElementIdChecked());
	}

	virtual FLocalLightBuildInfo& ResolveAsLocalLightBuildInfo(const FLightBuildInfoRef& Light) override
	{
		check(&Light.LightArray == (FLightArrayBase*)this);

		return this->Elements[Light.GetElementIdChecked()];
	}
};

class FLightRenderStateArrayBase
{
public:
	virtual ~FLightRenderStateArrayBase() {}
	virtual FLocalLightRenderState& ResolveAsLocalLightRenderState(const FLightRenderStateRef& Light) = 0;
};

template<typename T>
class TLightRenderStateArray : public FLightRenderStateArrayBase, public TEntityArray<T>
{
public:
	virtual FLocalLightRenderState& ResolveAsLocalLightRenderState(const FLightRenderStateRef& Light) override
	{
		check(&Light.LightRenderStateArray == (FLightRenderStateArrayBase*)this);

		return this->Elements[Light.GetElementIdChecked()];
	}
};

struct FLightScene
{
	TOptional<FSkyLightBuildInfo> SkyLight;
	TLightArray<FDirectionalLightBuildInfo> DirectionalLights;
	TLightArray<FPointLightBuildInfo> PointLights;
	TLightArray<FSpotLightBuildInfo> SpotLights;
	TLightArray<FRectLightBuildInfo> RectLights;

	TMap<UDirectionalLightComponent*, FDirectionalLightRef> RegisteredDirectionalLightComponentUObjects;
	TMap<UPointLightComponent*, FPointLightRef> RegisteredPointLightComponentUObjects;
	TMap<USpotLightComponent*, FSpotLightRef> RegisteredSpotLightComponentUObjects;
	TMap<URectLightComponent*, FRectLightRef> RegisteredRectLightComponentUObjects;
};

struct FLightSceneRenderState
{
	TOptional<FSkyLightRenderState> SkyLight;
	TLightRenderStateArray<FDirectionalLightRenderState> DirectionalLights;
	TLightRenderStateArray<FPointLightRenderState> PointLights;
	TLightRenderStateArray<FSpotLightRenderState> SpotLights;
	TLightRenderStateArray<FRectLightRenderState> RectLights;
};

}

static uint32 GetTypeHash(const GPULightmass::FDirectionalLightRenderState& O)
{
	return HashCombine(GetTypeHash(O.ShadowMapChannel), HashCombine(GetTypeHash(O.LightSourceAngle), HashCombine(GetTypeHash(O.Color), HashCombine(GetTypeHash(O.Direction), GetTypeHash(O.bStationary)))));
}

static uint32 GetTypeHash(const GPULightmass::FDirectionalLightRenderStateRef& Ref)
{
	return GetTypeHash(Ref.GetReference_Unsafe());
}

static uint32 GetTypeHash(const GPULightmass::FPointLightRenderState& O)
{
	return HashCombine(GetTypeHash(O.AttenuationRadius), HashCombine(GetTypeHash(O.ShadowMapChannel), HashCombine(GetTypeHash(O.SourceRadius), HashCombine(GetTypeHash(O.Color), HashCombine(GetTypeHash(O.Position), GetTypeHash(O.bStationary))))));
}

static uint32 GetTypeHash(const GPULightmass::FPointLightRenderStateRef& Ref)
{
	return GetTypeHash(Ref.GetReference_Unsafe());
}

static uint32 GetTypeHash(const GPULightmass::FSpotLightRenderState& O)
{
	return HashCombine(GetTypeHash(O.Tangent), HashCombine(GetTypeHash(O.SpotAngles), HashCombine(GetTypeHash(O.Direction), HashCombine(GetTypeHash(O.AttenuationRadius), HashCombine(GetTypeHash(O.ShadowMapChannel), HashCombine(GetTypeHash(O.SourceRadius), HashCombine(GetTypeHash(O.Color), HashCombine(GetTypeHash(O.Position), GetTypeHash(O.bStationary)))))))));
}

static uint32 GetTypeHash(const GPULightmass::FSpotLightRenderStateRef& Ref)
{
	return GetTypeHash(Ref.GetReference_Unsafe());
}

static uint32 GetTypeHash(const GPULightmass::FRectLightRenderState& O)
{
	return HashCombine(GetTypeHash(O.Tangent), 
		HashCombine(GetTypeHash(O.SourceWidth), 
		HashCombine(GetTypeHash(O.SourceHeight), 
		HashCombine(GetTypeHash(O.BarnDoorAngle),
		HashCombine(GetTypeHash(O.BarnDoorLength),
		HashCombine(GetTypeHash(O.Direction), 
		HashCombine(GetTypeHash(O.AttenuationRadius), 
		HashCombine(GetTypeHash(O.ShadowMapChannel),
		HashCombine(GetTypeHash(O.Color), 
		HashCombine(GetTypeHash(O.Position), 
		GetTypeHash(O.bStationary)))))))))));
}

static uint32 GetTypeHash(const GPULightmass::FRectLightRenderStateRef& Ref)
{
	return GetTypeHash(Ref.GetReference_Unsafe());
}
