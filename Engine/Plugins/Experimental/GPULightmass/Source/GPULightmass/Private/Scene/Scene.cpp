// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene.h"
#include "Async/ParallelFor.h"
#include "GPULightmass.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Async/Async.h"
#include "LightmapEncoding.h"
#include "GPULightmassCommon.h"
#include "Components/StaticMeshComponent.h"
#include "GameFramework/WorldSettings.h"
#include "Framework/Notifications/NotificationManager.h"
#include "StaticMeshComponentLODInfo.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Misc/ScopedSlowTask.h"
#include "LightmapPreviewVirtualTexture.h"
#include "EngineModule.h"
#include "LightmapRenderer.h"
#include "VolumetricLightmap.h"
#include "GPUScene.h"
#include "Lightmass/LightmassImportanceVolume.h"
#include "Logging/MessageLog.h"
#include "Misc/ConfigCacheIni.h"
#include "LevelEditorViewport.h"
#include "Editor.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "LightmapDenoising.h"
#include "ShaderCompiler.h"
#include "Misc/FileHelper.h"
#include "Components/ReflectionCaptureComponent.h"
#include "ReflectionEnvironment.h"
#include "RHIStaticStates.h"
#include "InstancedStaticMeshSceneProxyDesc.h"
#include "StaticLightingBuildContext.h"

#define LOCTEXT_NAMESPACE "StaticLightingSystem"

extern ENGINE_API bool GCompressLightmaps;

extern float GetTerrainExpandPatchCount(float LightMapRes, int32& X, int32& Y, int32 ComponentSize, int32 LightmapSize, int32& DesiredSize, uint32 LightingLOD);

namespace GPULightmass
{

FScene::FScene(FGPULightmass* InGPULightmass)
	: GPULightmass(InGPULightmass)
	, Settings(InGPULightmass->Settings)
	, Geometries(*this)
	, FeatureLevel(InGPULightmass->World->GetFeatureLevel())
{
	StaticMeshInstances.LinkRenderStateArray(RenderState.StaticMeshInstanceRenderStates);
	InstanceGroups.LinkRenderStateArray(RenderState.InstanceGroupRenderStates);
	Landscapes.LinkRenderStateArray(RenderState.LandscapeRenderStates);

	RenderState.Settings = Settings;
	RenderState.FeatureLevel = FeatureLevel;

	ENQUEUE_RENDER_COMMAND(RenderThreadInit)(
		[&RenderState = RenderState](FRHICommandListImmediate& RHICmdList) mutable
	{
		RenderState.RenderThreadInit(RHICmdList);
	});
}

void FSceneRenderState::RenderThreadInit(FRHICommandList& RHICmdList)
{
	check(IsInRenderingThread());

	LightmapRenderer = MakeUnique<FLightmapRenderer>(RHICmdList, this);
	VolumetricLightmapRenderer = MakeUnique<FVolumetricLightmapRenderer>(this);
	IrradianceCache = MakeUnique<FIrradianceCache>(RHICmdList, Settings->IrradianceCacheQuality, Settings->IrradianceCacheSpacing, Settings->IrradianceCacheCornerRejection);
	IrradianceCache->CurrentRevision = LightmapRenderer->GetCurrentRevision();
}

const FMeshMapBuildData* FScene::GetComponentLightmapData(const UPrimitiveComponent* InComponent, int32 LODIndex)
{
	if (const ULandscapeComponent* LandscapeComponent = Cast<const ULandscapeComponent>(InComponent))
	{
		if (RegisteredLandscapeComponentUObjects.Contains(LandscapeComponent))
		{
			FLandscapeRef Instance = RegisteredLandscapeComponentUObjects[LandscapeComponent];

			return Instance->GetMeshMapBuildDataForLODIndex(LODIndex);
		}
	}
	else if (const UInstancedStaticMeshComponent* InstancedStaticMeshComponent = Cast<const UInstancedStaticMeshComponent>(InComponent))
	{
		if (RegisteredInstancedStaticMeshComponentUObjects.Contains(InstancedStaticMeshComponent))
		{
			FInstanceGroupRef Instance = RegisteredInstancedStaticMeshComponentUObjects[InstancedStaticMeshComponent];

			return Instance->GetMeshMapBuildDataForLODIndex(LODIndex);
		}
	}
	else if (const UStaticMeshComponent* StaticMeshComponent = Cast<const UStaticMeshComponent>(InComponent))
	{
		if (RegisteredStaticMeshComponentUObjects.Contains(StaticMeshComponent))
		{
			FStaticMeshInstanceRef Instance = RegisteredStaticMeshComponentUObjects[StaticMeshComponent];

			return Instance->GetMeshMapBuildDataForLODIndex(LODIndex);
		}
	}

	return nullptr;
}

const FLightComponentMapBuildData* FScene::GetComponentLightmapData(const ULightComponent* InComponent)
{
	if (const UDirectionalLightComponent* DirectionalLight = Cast<UDirectionalLightComponent>(InComponent))
	{
		if (LightScene.RegisteredDirectionalLightComponentUObjects.Contains(DirectionalLight))
		{
			return LightScene.RegisteredDirectionalLightComponentUObjects[DirectionalLight]->LightComponentMapBuildData.Get();
		}
	}
	else if (const URectLightComponent* RectLight = Cast<URectLightComponent>(InComponent))
	{
		if (LightScene.RegisteredRectLightComponentUObjects.Contains(RectLight))
		{
			return LightScene.RegisteredRectLightComponentUObjects[RectLight]->LightComponentMapBuildData.Get();
		}
	}
	else if (const USpotLightComponent* SpotLight = Cast<USpotLightComponent>(InComponent))
	{
		if (LightScene.RegisteredSpotLightComponentUObjects.Contains(SpotLight))
		{
			return LightScene.RegisteredSpotLightComponentUObjects[SpotLight]->LightComponentMapBuildData.Get();
		}
	}
	else if (const UPointLightComponent* PointLight = Cast<UPointLightComponent>(InComponent))
	{
		if (LightScene.RegisteredPointLightComponentUObjects.Contains(PointLight))
		{
			return LightScene.RegisteredPointLightComponentUObjects[PointLight]->LightComponentMapBuildData.Get();
		}
	}

	return nullptr;
}

void FScene::GatherImportanceVolumes()
{
	FBox CombinedImportanceVolume(ForceInit);
	TArray<FBox> ImportanceVolumes;

	for (TObjectIterator<ALightmassImportanceVolume> It; It; ++It)
	{
		ALightmassImportanceVolume* LMIVolume = *It;
		if (GPULightmass->World->ContainsActor(LMIVolume) && IsValid(LMIVolume))
		{
			CombinedImportanceVolume += LMIVolume->GetComponentsBoundingBox(true);
			ImportanceVolumes.Add(LMIVolume->GetComponentsBoundingBox(true));
		}
	}

	if (CombinedImportanceVolume.GetExtent().SizeSquared() == 0)
	{
		float MinimumImportanceVolumeExtentWithoutWarning = 0.0f;
		verify(GConfig->GetFloat(TEXT("DevOptions.StaticLightingSceneConstants"), TEXT("MinimumImportanceVolumeExtentWithoutWarning"), MinimumImportanceVolumeExtentWithoutWarning, GLightmassIni));

		FBox AutomaticImportanceVolumeBounds(ForceInit);

		for (FGeometryAndItsArray GeomIt : Geometries)
		{
			FGeometry& Geometry = GeomIt.GetGeometry();

			if (Geometry.bCastShadow)
			{
				AutomaticImportanceVolumeBounds += Geometry.WorldBounds.GetBox();
			}
		}

		FBox ReasonableSceneBounds = AutomaticImportanceVolumeBounds;
		if (ReasonableSceneBounds.GetExtent().SizeSquared() > (MinimumImportanceVolumeExtentWithoutWarning * MinimumImportanceVolumeExtentWithoutWarning))
		{
			// Emit a serious warning to the user about performance.
			FMessageLog("LightingResults").PerformanceWarning(LOCTEXT("LightmassError_MissingImportanceVolume", "No importance volume found and the scene is so large that the automatically synthesized volume will not yield good results.  Please add a tightly bounding lightmass importance volume to optimize your scene's quality and lighting build times."));

			// Clamp the size of the importance volume we create to a reasonable size
			ReasonableSceneBounds = FBox(ReasonableSceneBounds.GetCenter() - MinimumImportanceVolumeExtentWithoutWarning, ReasonableSceneBounds.GetCenter() + MinimumImportanceVolumeExtentWithoutWarning);
		}
		else
		{
			// The scene isn't too big, so we'll use the scene's bounds as a synthetic importance volume
			// NOTE: We don't want to pop up a message log for this common case when creating a new level, so we just spray a log message.  It's not very important to a user.
			UE_LOGF(LogGPULightmass, Warning, "No importance volume found, so the scene bounding box was used.  You can optimize your scene's quality and lighting build times by adding importance volumes.");

			float AutomaticImportanceVolumeExpandBy = 0.0f;
			verify(GConfig->GetFloat(TEXT("DevOptions.StaticLightingSceneConstants"), TEXT("AutomaticImportanceVolumeExpandBy"), AutomaticImportanceVolumeExpandBy, GLightmassIni));

			// Expand the scene's bounds a bit to make sure volume lighting samples placed on surfaces are inside
			ReasonableSceneBounds = ReasonableSceneBounds.ExpandBy(AutomaticImportanceVolumeExpandBy);
		}

		CombinedImportanceVolume = ReasonableSceneBounds;
		ImportanceVolumes.Add(ReasonableSceneBounds);
	}
	
	ENQUEUE_RENDER_COMMAND(UpdateLIVs)([&RenderState = RenderState, CombinedImportanceVolume, ImportanceVolumes](FRHICommandList&) mutable {
		RenderState.CombinedImportanceVolume = CombinedImportanceVolume;
		RenderState.ImportanceVolumes = ImportanceVolumes;
	});
}

FGeometryIterator FGeometryRange::begin()
{
	TArray<FGeometryArrayBase*> Arrays { &Scene.StaticMeshInstances, &Scene.InstanceGroups, &Scene.Landscapes };
	int StartIndex = 0;
	while (StartIndex < Arrays.Num() && Arrays[StartIndex]->Num() == 0)
	{
		StartIndex++;
	}
	return FGeometryIterator { 0, Arrays, StartIndex };
}

FGeometryIterator FGeometryRange::end()
{
	return FGeometryIterator { Scene.Landscapes.Num(), {&Scene.StaticMeshInstances, &Scene.InstanceGroups, &Scene.Landscapes}, 3 };
}

void AddLightToLightmap(
	FLightmap& Lightmap,
	FLocalLightBuildInfo& Light)
{
	// For both static and stationary lights
	Lightmap.LightmapObject->LightGuids.Add(Light.GetComponentUObject()->LightGuid);

	if (Light.CastsStationaryShadow())
	{
		Lightmap.NumStationaryLightsPerShadowChannel[Light.ShadowMapChannel]++;
		Lightmap.LightmapObject->bShadowChannelValid[Light.ShadowMapChannel] = true;
		// TODO: implement SDF. For area lights and invalid channels this will be fixed to 1
		Lightmap.LightmapObject->InvUniformPenumbraSize[Light.ShadowMapChannel] = 1.0f / Light.GetComponentUObject()->GetUniformPenumbraSize();

		// TODO: needs GPUScene update to reflect penumbra size changes
	}
}

void RemoveLightFromLightmap(
	FLightmap& Lightmap,
	FLocalLightBuildInfo& Light)
{
	Lightmap.LightmapObject->LightGuids.Remove(Light.GetComponentUObject()->LightGuid);

	if (Light.CastsStationaryShadow())
	{
		Lightmap.NumStationaryLightsPerShadowChannel[Light.ShadowMapChannel]--;

		if (Lightmap.NumStationaryLightsPerShadowChannel[Light.ShadowMapChannel] == 0)
		{
			Lightmap.LightmapObject->bShadowChannelValid[Light.ShadowMapChannel] = false;
			Lightmap.LightmapObject->InvUniformPenumbraSize[Light.ShadowMapChannel] = 1.0f;
		}
	}
}
template<typename LightComponentType>
struct LightTypeInfo
{
};

template<>
struct LightTypeInfo<UDirectionalLightComponent>
{
	using BuildInfoType = FDirectionalLightBuildInfo;
	using LightRefType = FDirectionalLightRef;
	using RenderStateType = FDirectionalLightRenderState;
	using RenderStateRefType = FDirectionalLightRenderStateRef;

	using LightComponentRegistrationType = TMap<UDirectionalLightComponent*, LightRefType>;
	inline static LightComponentRegistrationType& GetLightComponentRegistration(FLightScene& LightScene)
	{
		return LightScene.RegisteredDirectionalLightComponentUObjects;
	}

	using LightArrayType = TLightArray<BuildInfoType>;
	inline static LightArrayType& GetLightArray(FLightScene& LightScene)
	{
		return LightScene.DirectionalLights;
	}

	using LightRenderStateArrayType = TLightRenderStateArray<RenderStateType>;
	inline static LightRenderStateArrayType& GetLightRenderStateArray(FLightSceneRenderState& LightSceneRenderState)
	{
		return LightSceneRenderState.DirectionalLights;
	}
};

template<>
struct LightTypeInfo<UPointLightComponent>
{
	using BuildInfoType = FPointLightBuildInfo;
	using LightRefType = FPointLightRef;
	using RenderStateType = FPointLightRenderState;
	using RenderStateRefType = FPointLightRenderStateRef;

	using LightComponentRegistrationType = TMap<UPointLightComponent*, LightRefType>;
	inline static LightComponentRegistrationType& GetLightComponentRegistration(FLightScene& LightScene)
	{
		return LightScene.RegisteredPointLightComponentUObjects;
	}

	using LightArrayType = TLightArray<BuildInfoType>;
	inline static LightArrayType& GetLightArray(FLightScene& LightScene)
	{
		return LightScene.PointLights;
	}

	using LightRenderStateArrayType = TLightRenderStateArray<RenderStateType>;
	inline static LightRenderStateArrayType& GetLightRenderStateArray(FLightSceneRenderState& LightSceneRenderState)
	{
		return LightSceneRenderState.PointLights;
	}
};

template<>
struct LightTypeInfo<USpotLightComponent>
{
	using BuildInfoType = FSpotLightBuildInfo;
	using LightRefType = FSpotLightRef;
	using RenderStateType = FSpotLightRenderState;
	using RenderStateRefType = FSpotLightRenderStateRef;

	using LightComponentRegistrationType = TMap<USpotLightComponent*, LightRefType>;
	inline static LightComponentRegistrationType& GetLightComponentRegistration(FLightScene& LightScene)
	{
		return LightScene.RegisteredSpotLightComponentUObjects;
	}

	using LightArrayType = TLightArray<BuildInfoType>;
	inline static LightArrayType& GetLightArray(FLightScene& LightScene)
	{
		return LightScene.SpotLights;
	}

	using LightRenderStateArrayType = TLightRenderStateArray<RenderStateType>;
	inline static LightRenderStateArrayType& GetLightRenderStateArray(FLightSceneRenderState& LightSceneRenderState)
	{
		return LightSceneRenderState.SpotLights;
	}
};

template<>
struct LightTypeInfo<URectLightComponent>
{
	using BuildInfoType = FRectLightBuildInfo;
	using LightRefType = FRectLightRef;
	using RenderStateType = FRectLightRenderState;
	using RenderStateRefType = FRectLightRenderStateRef;

	using LightComponentRegistrationType = TMap<URectLightComponent*, LightRefType>;
	inline static LightComponentRegistrationType& GetLightComponentRegistration(FLightScene& LightScene)
	{
		return LightScene.RegisteredRectLightComponentUObjects;
	}

	using LightArrayType = TLightArray<BuildInfoType>;
	inline static LightArrayType& GetLightArray(FLightScene& LightScene)
	{
		return LightScene.RectLights;
	}

	using LightRenderStateArrayType = TLightRenderStateArray<RenderStateType>;
	inline static LightRenderStateArrayType& GetLightRenderStateArray(FLightSceneRenderState& LightSceneRenderState)
	{
		return LightSceneRenderState.RectLights;
	}
};

template<typename LightComponentType>
void FScene::AddLight(LightComponentType* LightComponent)
{
	if (LightTypeInfo<LightComponentType>::GetLightComponentRegistration(LightScene).Contains(LightComponent))
	{
		return;
	}

	const bool bCastStationaryShadows = LightComponent->CastShadows && LightComponent->CastStaticShadows && LightComponent->Mobility == EComponentMobility::Stationary;

	if (bCastStationaryShadows)
	{
		if (LightComponent->PreviewShadowMapChannel == INDEX_NONE)
		{
			UE_LOGF(LogGPULightmass, Log, "Ignoring light with ShadowMapChannel == -1 (probably in the middle of SpawnActor)");
			return;
		}
	}

	typename LightTypeInfo<LightComponentType>::BuildInfoType Light(LightComponent);

	typename LightTypeInfo<LightComponentType>::LightRefType LightRef = LightTypeInfo<LightComponentType>::GetLightArray(LightScene).Emplace(MoveTemp(Light));
	LightTypeInfo<LightComponentType>::GetLightComponentRegistration(LightScene).Add(LightComponent, LightRef);

	typename LightTypeInfo<LightComponentType>::RenderStateType LightRenderState(LightComponent);
	LightRenderState.LightComponentMapBuildData = LightRef->LightComponentMapBuildData;

	TArray<FPrimitiveSceneProxy*> SceneProxiesToUpdateOnRenderThread;
	TArray<FGeometryRenderStateToken> RelevantGeometriesToUpdateOnRenderThread;

	for (FGeometryAndItsArray GeomIt : Geometries)
	{
		FGeometry& Geometry = GeomIt.GetGeometry();

		if (LightRef->AffectsBounds(Geometry.WorldBounds))
		{
			if (LightRef->CastsStationaryShadow())
			{
				RelevantGeometriesToUpdateOnRenderThread.Add({ GeomIt.Index, GeomIt.Array.GetRenderStateArray() });
			}

			for (FLightmapRef& Lightmap : Geometry.LODLightmaps)
			{
				if (Lightmap.IsValid())
				{
					AddLightToLightmap(Lightmap.GetReference_Unsafe(), LightRef.GetReference_Unsafe());
				}
			}

			if (Geometry.GetComponentUObject()->SceneProxy)
			{
				SceneProxiesToUpdateOnRenderThread.Add(Geometry.GetComponentUObject()->SceneProxy);
			}
		}
	}

	if (LightRef->CastsStationaryShadow())
	{
		GatherImportanceVolumes();
	}

	ENQUEUE_RENDER_COMMAND(UpdateStaticLightingBufferCmd)(
		[SceneProxiesToUpdateOnRenderThread = MoveTemp(SceneProxiesToUpdateOnRenderThread)](FRHICommandListImmediate& RHICmdList)
	{
		for (FPrimitiveSceneProxy* SceneProxy : SceneProxiesToUpdateOnRenderThread)
		{
			FPrimitiveSceneInfo* PrimitiveSceneInfo = SceneProxy->GetPrimitiveSceneInfo();
			if (PrimitiveSceneInfo && PrimitiveSceneInfo->IsIndexValid())
			{
				PrimitiveSceneInfo->RequestGPUSceneUpdate(EPrimitiveDirtyState::ChangedStaticLighting);
			}
		}
	});

	ENQUEUE_RENDER_COMMAND(RenderThreadUpdate)(
		[
			&RenderState = RenderState,
			LightRenderState = MoveTemp(LightRenderState),
			RelevantGeometriesToUpdateOnRenderThread,
			bShouldRenderStaticShadowDepthMap = LightRef->CastsStationaryShadow(),
			&StaticShadowDepthMap = LightComponent->StaticShadowDepthMap,
			DepthMapPtr = &LightRef->LightComponentMapBuildData->DepthMap
		](FRHICommandListImmediate& RHICmdList) mutable
	{
		typename LightTypeInfo<LightComponentType>::RenderStateRefType LightRenderStateRef = LightTypeInfo<LightComponentType>::GetLightRenderStateArray(RenderState.LightSceneRenderState).Emplace(MoveTemp(LightRenderState));

		LightRenderStateRef->RenderThreadInit();

		if (bShouldRenderStaticShadowDepthMap)
		{
			LightRenderStateRef->RenderStaticShadowDepthMap(RHICmdList, RenderState);
			StaticShadowDepthMap.Data = DepthMapPtr;
			StaticShadowDepthMap.InitRHI(RHICmdList);
		}

		for (FGeometryRenderStateToken Token : RelevantGeometriesToUpdateOnRenderThread)
		{
			for (FLightmapRenderStateRef& Lightmap : Token.RenderStateArray->Get(Token.ElementId).LODLightmapRenderStates)
			{
				if (Lightmap.IsValid())
				{
					Lightmap->AddRelevantLight(LightRenderStateRef);
				}
			}
		}
	});

	ENQUEUE_RENDER_COMMAND(InvalidateRevision)([&RenderState = RenderState](FRHICommandListImmediate& RHICmdList) { 
		RenderState.LightmapRenderer->BumpRevision();
		RenderState.VolumetricLightmapRenderer->FrameNumber = 0;
		RenderState.VolumetricLightmapRenderer->SamplesTaken = 0;
	});
}

template void FScene::AddLight(UDirectionalLightComponent* LightComponent);
template void FScene::AddLight(UPointLightComponent* LightComponent);
template void FScene::AddLight(USpotLightComponent* LightComponent);
template void FScene::AddLight(URectLightComponent* LightComponent);

template<typename LightComponentType>
void FScene::RemoveLight(LightComponentType* PointLightComponent)
{
	if (!LightTypeInfo<LightComponentType>::GetLightComponentRegistration(LightScene).Contains(PointLightComponent))
	{
		return;
	}

	typename LightTypeInfo<LightComponentType>::LightRefType Light = LightTypeInfo<LightComponentType>::GetLightComponentRegistration(LightScene)[PointLightComponent];

	TArray<FPrimitiveSceneProxy*> SceneProxiesToUpdateOnRenderThread;
	TArray<FGeometryRenderStateToken> RelevantGeometriesToUpdateOnRenderThread;

	for (FGeometryAndItsArray GeomIt : Geometries)
	{
		FGeometry& Geometry = GeomIt.GetGeometry();

		if (Light->AffectsBounds(Geometry.WorldBounds))
		{
			if (Light->bStationary)
			{
				RelevantGeometriesToUpdateOnRenderThread.Add({ GeomIt.Index, GeomIt.Array.GetRenderStateArray() });
			}

			for (FLightmapRef& Lightmap : Geometry.LODLightmaps)
			{
				if (Lightmap.IsValid())
				{
					RemoveLightFromLightmap(Lightmap.GetReference_Unsafe(), Light);
				}
			}

			if (Geometry.GetComponentUObject()->SceneProxy)
			{
				SceneProxiesToUpdateOnRenderThread.Add(Geometry.GetComponentUObject()->SceneProxy);
			}
		}
	}

	ENQUEUE_RENDER_COMMAND(UpdateStaticLightingBufferCmd)(
		[SceneProxiesToUpdateOnRenderThread = MoveTemp(SceneProxiesToUpdateOnRenderThread)](FRHICommandListImmediate& RHICmdList)
	{
		for (FPrimitiveSceneProxy* SceneProxy : SceneProxiesToUpdateOnRenderThread)
		{
			FPrimitiveSceneInfo* PrimitiveSceneInfo = SceneProxy->GetPrimitiveSceneInfo();
			if (PrimitiveSceneInfo && PrimitiveSceneInfo->IsIndexValid())
			{
				PrimitiveSceneInfo->RequestGPUSceneUpdate(EPrimitiveDirtyState::ChangedStaticLighting);
			}
		}
	});

	int32 ElementId = Light.GetElementId();
	LightTypeInfo<LightComponentType>::GetLightArray(LightScene).RemoveAt(ElementId);
	LightTypeInfo<LightComponentType>::GetLightComponentRegistration(LightScene).Remove(PointLightComponent);

	ENQUEUE_RENDER_COMMAND(RenderThreadUpdate)(
		[
			&RenderState = RenderState,
			RelevantGeometriesToUpdateOnRenderThread,
			ElementId
		](FRHICommandListImmediate& RHICmdList) mutable
	{
		typename LightTypeInfo<LightComponentType>::RenderStateRefType LightRenderStateRef(LightTypeInfo<LightComponentType>::GetLightRenderStateArray(RenderState.LightSceneRenderState).Elements[ElementId], LightTypeInfo<LightComponentType>::GetLightRenderStateArray(RenderState.LightSceneRenderState));

		LightRenderStateRef->RenderThreadFinalize();
		
		for (FGeometryRenderStateToken Token : RelevantGeometriesToUpdateOnRenderThread)
		{
			for (FLightmapRenderStateRef& Lightmap : Token.RenderStateArray->Get(Token.ElementId).LODLightmapRenderStates)
			{
				if (Lightmap.IsValid())
				{
					Lightmap->RemoveRelevantLight(LightRenderStateRef);
				}
			}
		}

		LightTypeInfo<LightComponentType>::GetLightRenderStateArray(RenderState.LightSceneRenderState).RemoveAt(ElementId);
	});

	ENQUEUE_RENDER_COMMAND(InvalidateRevision)([&RenderState = RenderState](FRHICommandListImmediate& RHICmdList) {
		RenderState.LightmapRenderer->BumpRevision();
		RenderState.VolumetricLightmapRenderer->FrameNumber = 0;
		RenderState.VolumetricLightmapRenderer->SamplesTaken = 0;
	});
}

template void FScene::RemoveLight(UDirectionalLightComponent* LightComponent);
template void FScene::RemoveLight(UPointLightComponent* LightComponent);
template void FScene::RemoveLight(USpotLightComponent* LightComponent);
template void FScene::RemoveLight(URectLightComponent* LightComponent);

template<typename LightComponentType>
bool FScene::HasLight(LightComponentType* PointLightComponent)
{
	return LightTypeInfo<LightComponentType>::GetLightComponentRegistration(LightScene).Contains(PointLightComponent);
}

template bool FScene::HasLight(UDirectionalLightComponent* LightComponent);
template bool FScene::HasLight(UPointLightComponent* LightComponent);
template bool FScene::HasLight(USpotLightComponent* LightComponent);
template bool FScene::HasLight(URectLightComponent* LightComponent);

void FScene::AddLight(USkyLightComponent* SkyLight)
{
	if (LightScene.SkyLight.IsSet() && LightScene.SkyLight->ComponentUObject == SkyLight)
	{
		return;
	}

	if (!SkyLight->GetProcessedSkyTexture())
	{
		UE_LOGF(LogGPULightmass, Log, "Skipping skylight with empty cubemap");
		return;
	}

	if (LightScene.SkyLight.IsSet())
	{
		UE_LOGF(LogGPULightmass, Log, "Warning: trying to add more than one skylight - removing the old one");
		RemoveLight(LightScene.SkyLight->ComponentUObject);
	}

	int32 LightId = INDEX_NONE;

	FSkyLightBuildInfo NewSkyLight;
	NewSkyLight.ComponentUObject = SkyLight;
	NewSkyLight.bStationary = SkyLight->Mobility == EComponentMobility::Stationary;
	NewSkyLight.bCastShadow = SkyLight->CastShadows && SkyLight->CastStaticShadows;

	LightScene.SkyLight = MoveTemp(NewSkyLight);

	FSkyLightRenderState NewSkyLightRenderState;
	NewSkyLightRenderState.bStationary = SkyLight->Mobility == EComponentMobility::Stationary;
	NewSkyLightRenderState.bCastShadow = SkyLight->CastShadows && SkyLight->CastStaticShadows;
	NewSkyLightRenderState.Color = SkyLight->GetLightColor() * SkyLight->Intensity;
	NewSkyLightRenderState.TextureDimensions = FIntPoint(SkyLight->GetProcessedSkyTexture()->GetSizeX(), SkyLight->GetProcessedSkyTexture()->GetSizeY());
	NewSkyLightRenderState.IrradianceEnvironmentMap = SkyLight->GetIrradianceEnvironmentMap();

	ENQUEUE_RENDER_COMMAND(AddLightRenderState)(
		[&RenderState = RenderState, NewSkyLightRenderState = MoveTemp(NewSkyLightRenderState), ProcessedSkyTexture = SkyLight->GetProcessedSkyTexture(), FeatureLevel = FeatureLevel](FRHICommandListImmediate& RHICmdList) mutable
	{
		// Dereferencing ProcessedSkyTexture must be deferred onto render thread
		NewSkyLightRenderState.ProcessedTexture = ProcessedSkyTexture->TextureRHI;
		NewSkyLightRenderState.ProcessedTextureSampler = ProcessedSkyTexture->SamplerStateRHI;

		NewSkyLightRenderState.SkyIrradianceEnvironmentMap.Initialize(RHICmdList, TEXT("SkyIrradianceEnvironmentMap"), sizeof(FVector4f), 7);

		NewSkyLightRenderState.PrepareSkyTexture(RHICmdList, FeatureLevel);

		// Set the captured environment map data
		void* DataPtr = RHICmdList.LockBuffer(NewSkyLightRenderState.SkyIrradianceEnvironmentMap.Buffer, 0, NewSkyLightRenderState.SkyIrradianceEnvironmentMap.NumBytes, RLM_WriteOnly);
		SetupSkyIrradianceEnvironmentMapConstantsFromSkyIrradiance((FVector4f*)DataPtr, NewSkyLightRenderState.IrradianceEnvironmentMap);
		RHICmdList.UnlockBuffer(NewSkyLightRenderState.SkyIrradianceEnvironmentMap.Buffer);

		RenderState.LightSceneRenderState.SkyLight = MoveTemp(NewSkyLightRenderState);

		RenderState.LightmapRenderer->BumpRevision();
	});
}

void FScene::RemoveLight(USkyLightComponent* SkyLight)
{
	if (!LightScene.SkyLight.IsSet() || LightScene.SkyLight->ComponentUObject != SkyLight)
	{
		return;
	}

	check(LightScene.SkyLight.IsSet());

	LightScene.SkyLight.Reset();

	ENQUEUE_RENDER_COMMAND(RemoveLightRenderState)(
		[&RenderState = RenderState](FRHICommandListImmediate& RHICmdList) mutable
	{
		RenderState.LightSceneRenderState.SkyLight.Reset();

		RenderState.LightmapRenderer->BumpRevision();
	});
}

void FScene::OnSkyAtmosphereModified()
{
	ConditionalTriggerSkyLightRecapture();
}

void FScene::ConditionalTriggerSkyLightRecapture()
{
	if (LightScene.SkyLight.IsSet())
	{
		USkyLightComponent* SkyLight = LightScene.SkyLight->ComponentUObject;
		if (SkyLight->SourceType == SLS_CapturedScene || SkyLight->bRealTimeCapture)
		{
			SkyLight->SetCaptureIsDirty();
			SkyLight->UpdateSkyCaptureContents(SkyLight->GetWorld());
		}
	}
}

template<typename LightType, typename GeometryRefType>
TArray<int32> AddAllPossiblyRelevantLightsToGeometry(
	TEntityArray<LightType>& LightArray,
	GeometryRefType Instance
)
{
	TArray<int32> RelevantLightsToAddOnRenderThread;

	for (LightType& Light : LightArray.Elements)
	{
		if (Light.AffectsBounds(Instance->WorldBounds))
		{
			if (Light.CastsStationaryShadow())
			{
				RelevantLightsToAddOnRenderThread.Add(&Light - LightArray.Elements.GetData());
			}

			for (FLightmapRef& Lightmap : Instance->LODLightmaps)
			{
				if (Lightmap.IsValid())
				{
					AddLightToLightmap(Lightmap.GetReference_Unsafe(), Light);
				}
			}
		}
	}

	return RelevantLightsToAddOnRenderThread;
}

void FScene::AddGeometryInstanceFromComponent(UStaticMeshComponent* InComponent)
{
	if (RegisteredStaticMeshComponentUObjects.Contains(InComponent))
	{
		return;
	}
	
	FStaticMeshInstanceRef Instance = StaticMeshInstances.Emplace(InComponent);
	Instance->WorldBounds = InComponent->Bounds;
	Instance->bCastShadow = InComponent->CastShadow && InComponent->bCastStaticShadow;
	Instance->bLODsShareStaticLighting = InComponent->GetStaticMesh()->CanLODsShareStaticLighting();

	RegisteredStaticMeshComponentUObjects.Add(InComponent, Instance);

	const int32 SMCurrentMinLOD = InComponent->GetStaticMesh()->GetDefaultMinLOD();
	int32 EffectiveMinLOD = InComponent->bOverrideMinLOD ? InComponent->MinLOD : SMCurrentMinLOD;

	// Find the first LOD with any vertices (ie that haven't been stripped)
	int FirstAvailableLOD = 0;
	for (; FirstAvailableLOD < InComponent->GetStaticMesh()->GetRenderData()->LODResources.Num(); FirstAvailableLOD++)
	{
		if (InComponent->GetStaticMesh()->GetRenderData()->LODResources[FirstAvailableLOD].GetNumVertices() > 0)
		{
			break;
		}
	}

	Instance->ClampedMinLOD = FMath::Clamp(EffectiveMinLOD, FirstAvailableLOD, InComponent->GetStaticMesh()->GetRenderData()->LODResources.Num() - 1);

	Instance->AllocateLightmaps(Lightmaps);

	FStaticMeshInstanceRenderState InstanceRenderState;
	InstanceRenderState.ComponentUObject = Instance->ComponentUObject;
	InstanceRenderState.RenderData = Instance->ComponentUObject->GetStaticMesh()->GetRenderData();
	InstanceRenderState.LocalToWorld = InComponent->GetRenderMatrix();
	InstanceRenderState.WorldBounds = InComponent->Bounds;
	InstanceRenderState.ActorPosition = InComponent->GetActorPositionForRenderer();
	InstanceRenderState.LocalBounds = InComponent->CalcBounds(FTransform::Identity);
	InstanceRenderState.bCastShadow = InComponent->CastShadow && InComponent->bCastStaticShadow;
	InstanceRenderState.LODOverrideColorVertexBuffers.AddZeroed(InComponent->GetStaticMesh()->GetRenderData()->LODResources.Num());
	InstanceRenderState.LODOverrideColorVFUniformBuffers.AddDefaulted(InComponent->GetStaticMesh()->GetRenderData()->LODResources.Num());
	InstanceRenderState.ClampedMinLOD = Instance->ClampedMinLOD;

	for (int32 LODIndex = Instance->ClampedMinLOD; LODIndex < FMath::Min(InComponent->LODData.Num(), InComponent->GetStaticMesh()->GetRenderData()->LODResources.Num()); LODIndex++)
	{
		const FStaticMeshComponentLODInfo& ComponentLODInfo = InComponent->LODData[LODIndex];

		// Initialize this LOD's overridden vertex colors, if it has any
		if (ComponentLODInfo.OverrideVertexColors)
		{
			bool bBroken = false;
			for (int32 SectionIndex = 0; SectionIndex < InComponent->GetStaticMesh()->GetRenderData()->LODResources[LODIndex].Sections.Num(); SectionIndex++)
			{
				const FStaticMeshSection& Section = InComponent->GetStaticMesh()->GetRenderData()->LODResources[LODIndex].Sections[SectionIndex];
				if (Section.MaxVertexIndex >= ComponentLODInfo.OverrideVertexColors->GetNumVertices())
				{
					bBroken = true;
					break;
				}
			}
			if (!bBroken)
			{
				InstanceRenderState.LODOverrideColorVertexBuffers[LODIndex] = ComponentLODInfo.OverrideVertexColors;
			}
		}
	}

	TArray<FLightmapRenderState::Initializer> InstanceLightmapRenderStateInitializers;

	for (FLightmapRef& Lightmap : Instance->LODLightmaps)
	{
		if (Lightmap.IsValid())
		{
			Lightmap->CreateGameThreadResources();

			for (FDirectionalLightBuildInfo& DirectionalLight : LightScene.DirectionalLights.Elements)
			{
				AddLightToLightmap(Lightmap.GetReference_Unsafe(), DirectionalLight);
			}

			// Ownership will be transferred to render thread, can be nullptr if not created
			FLightmapResourceCluster* ResourceCluster = Lightmap->ResourceCluster.Release();

			FLightmapRenderState::Initializer Initializer {
				Lightmap->Name,
				Lightmap->Size,
				FMath::Min((int32)FMath::CeilLogTwo((uint32)FMath::Min(Lightmap->GetPaddedSizeInTiles().X, Lightmap->GetPaddedSizeInTiles().Y)), GPreviewLightmapMipmapMaxLevel),
				ResourceCluster,
				FVector4f(FVector2f(Lightmap->LightmapObject->CoordinateScale), FVector2f(Lightmap->LightmapObject->CoordinateBias))
			};

			InstanceLightmapRenderStateInitializers.Add(MoveTemp(Initializer));
		}
		else
		{
			InstanceLightmapRenderStateInitializers.Add(FLightmapRenderState::Initializer {});
		}
	}
	
	TArray<int32> RelevantPointLightsToAddOnRenderThread = AddAllPossiblyRelevantLightsToGeometry(LightScene.PointLights, Instance);
	TArray<int32> RelevantSpotLightsToAddOnRenderThread = AddAllPossiblyRelevantLightsToGeometry(LightScene.SpotLights, Instance);
	TArray<int32> RelevantRectLightsToAddOnRenderThread = AddAllPossiblyRelevantLightsToGeometry(LightScene.RectLights, Instance);

	ENQUEUE_RENDER_COMMAND(RenderThreadInit)(
		[
			FeatureLevel = FeatureLevel,
			InstanceRenderState = MoveTemp(InstanceRenderState), 
			InstanceLightmapRenderStateInitializers = MoveTemp(InstanceLightmapRenderStateInitializers),
			&RenderState = RenderState,
			RelevantPointLightsToAddOnRenderThread,
			RelevantSpotLightsToAddOnRenderThread,
			RelevantRectLightsToAddOnRenderThread
		](FRHICommandListImmediate&) mutable
	{
		FStaticMeshInstanceRenderStateRef InstanceRenderStateRef = RenderState.StaticMeshInstanceRenderStates.Emplace(MoveTemp(InstanceRenderState));

		for (int32 LODIndex = 0; LODIndex < InstanceLightmapRenderStateInitializers.Num(); LODIndex++)
		{
			FLightmapRenderState::Initializer& Initializer = InstanceLightmapRenderStateInitializers[LODIndex];
			if (Initializer.IsValid())
			{
				FLightmapRenderStateRef LightmapRenderState = RenderState.LightmapRenderStates.Emplace(Initializer, RenderState.StaticMeshInstanceRenderStates.CreateGeometryInstanceRef(InstanceRenderStateRef, LODIndex));
				CreateLightmapPreviewVirtualTexture(LightmapRenderState, FeatureLevel, RenderState.LightmapRenderer.Get());

				InstanceRenderStateRef->LODLightmapRenderStates.Emplace(LightmapRenderState);

				for (int32 ElementId : RelevantPointLightsToAddOnRenderThread)
				{
					LightmapRenderState->AddRelevantLight(FPointLightRenderStateRef(RenderState.LightSceneRenderState.PointLights.Elements[ElementId], RenderState.LightSceneRenderState.PointLights));
				}

				for (int32 ElementId : RelevantSpotLightsToAddOnRenderThread)
				{
					LightmapRenderState->AddRelevantLight(FSpotLightRenderStateRef(RenderState.LightSceneRenderState.SpotLights.Elements[ElementId], RenderState.LightSceneRenderState.SpotLights));
				}

				for (int32 ElementId : RelevantRectLightsToAddOnRenderThread)
				{
					LightmapRenderState->AddRelevantLight(FRectLightRenderStateRef(RenderState.LightSceneRenderState.RectLights.Elements[ElementId], RenderState.LightSceneRenderState.RectLights));
				}
			}
			else
			{
				InstanceRenderStateRef->LODLightmapRenderStates.Emplace(RenderState.LightmapRenderStates.CreateNullRef());
			}
		}

		for (int32 LODIndex = InstanceRenderStateRef->ClampedMinLOD; LODIndex < InstanceLightmapRenderStateInitializers.Num(); LODIndex++)
		{
			if (InstanceRenderStateRef->LODOverrideColorVertexBuffers[LODIndex] != nullptr)
			{
				const FLocalVertexFactory* LocalVF = &InstanceRenderStateRef->ComponentUObject->GetStaticMesh()->GetRenderData()->LODVertexFactories[LODIndex].VertexFactoryOverrideColorVertexBuffer;
				InstanceRenderStateRef->LODOverrideColorVFUniformBuffers[LODIndex] = CreateLocalVFUniformBuffer(LocalVF, LODIndex, InstanceRenderStateRef->LODOverrideColorVertexBuffers[LODIndex], 0, 0);
			}
		}

		RenderState.LightmapRenderer->BumpRevision();

		RenderState.CachedRayTracingScene.Reset();
	});

	bNeedsVoxelization = true;

	if (InComponent->GetWorld())
	{
		InComponent->GetWorld()->SendAllEndOfFrameUpdates();
	}
}

void FScene::RemoveGeometryInstanceFromComponent(UStaticMeshComponent* InComponent)
{
	if (!RegisteredStaticMeshComponentUObjects.Contains(InComponent))
	{
		return;
	}

	FStaticMeshInstanceRef Instance = RegisteredStaticMeshComponentUObjects[InComponent];
	for (FLightmapRef& Lightmap : Instance->LODLightmaps)
	{
		if (Lightmap.IsValid())
		{
			Lightmaps.Remove(Lightmap);
		}
	}

	int32 ElementId = Instance.GetElementId();
	StaticMeshInstances.RemoveAt(ElementId);
	RegisteredStaticMeshComponentUObjects.Remove(InComponent);

	ENQUEUE_RENDER_COMMAND(RenderThreadRemove)(
		[ElementId, &RenderState = RenderState](FRHICommandListImmediate&) mutable
	{
		for (FLightmapRenderStateRef& Lightmap : RenderState.StaticMeshInstanceRenderStates.Elements[ElementId].LODLightmapRenderStates)
		{
			if (Lightmap.IsValid())
			{
				Lightmap->ReleasePreviewVirtualTexture();
				RenderState.LightmapRenderStates.Remove(Lightmap);
			}
		}

		RenderState.StaticMeshInstanceRenderStates.RemoveAt(ElementId);

		RenderState.LightmapRenderer->BumpRevision();

		RenderState.CachedRayTracingScene.Reset();
	});

	bNeedsVoxelization = true;
}

void FScene::AddGeometryInstanceFromComponent(UInstancedStaticMeshComponent* InComponent)
{
	if (InComponent->PerInstanceSMData.Num() == 0)
	{
		return;
	}

	if (RegisteredInstancedStaticMeshComponentUObjects.Contains(InComponent))
	{
		return;
	}

	FInstanceGroupRef Instance = InstanceGroups.Emplace(InComponent);
	Instance->WorldBounds = InComponent->Bounds;
	Instance->bCastShadow = InComponent->CastShadow && InComponent->bCastStaticShadow;

	RegisteredInstancedStaticMeshComponentUObjects.Add(InComponent, Instance);

	if (InComponent->GetWorld())
	{
		if (UHierarchicalInstancedStaticMeshComponent* HISMC = Cast<UHierarchicalInstancedStaticMeshComponent>(InComponent))
		{
			HISMC->BuildTreeIfOutdated(false, true);
		}

		InComponent->GetWorld()->SendAllEndOfFrameUpdates();
	}

	Instance->AllocateLightmaps(Lightmaps);

	TArray<FLightmapRenderState::Initializer> InstanceLightmapRenderStateInitializers;

	for (int32 LODIndex = 0; LODIndex < Instance->LODLightmaps.Num(); LODIndex++)
	{
		FLightmapRef& Lightmap = Instance->LODLightmaps[LODIndex];

		if (Lightmap.IsValid())
		{
			Lightmap->CreateGameThreadResources();

			{
				int32 BaseLightMapWidth = Instance->LODPerInstanceLightmapSize[LODIndex].X;
				int32 BaseLightMapHeight = Instance->LODPerInstanceLightmapSize[LODIndex].Y;

				FVector2D Scale = FVector2D(BaseLightMapWidth - 2, BaseLightMapHeight - 2) / Lightmap->GetPaddedSize();
				Lightmap->LightmapObject->CoordinateScale = Scale;
				Lightmap->LightmapObject->CoordinateBias = FVector2D(0, 0);

				int32 InstancesPerRow = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(InComponent->PerInstanceSMData.Num())));
				Lightmap->MeshMapBuildData->PerInstanceLightmapData.AddDefaulted(InComponent->PerInstanceSMData.Num());
				for (int32 GameThreadInstanceIndex = 0; GameThreadInstanceIndex < InComponent->PerInstanceSMData.Num(); GameThreadInstanceIndex++)
				{
					int32 RenderIndex = InComponent->GetRenderIndex(GameThreadInstanceIndex);
					if (RenderIndex != INDEX_NONE)
					{
						int32 X = RenderIndex % InstancesPerRow;
						int32 Y = RenderIndex / InstancesPerRow;
						FVector2f Bias = (FVector2f(X, Y) * FVector2f(BaseLightMapWidth, BaseLightMapHeight) + FVector2f(1, 1)) / Lightmap->GetPaddedSize();
						Lightmap->MeshMapBuildData->PerInstanceLightmapData[GameThreadInstanceIndex].LightmapUVBias = Bias;
						Lightmap->MeshMapBuildData->PerInstanceLightmapData[GameThreadInstanceIndex].ShadowmapUVBias = Bias;
					}
				}
			}

			for (FDirectionalLightBuildInfo& DirectionalLight : LightScene.DirectionalLights.Elements)
			{
				AddLightToLightmap(Lightmap.GetReference_Unsafe(), DirectionalLight);
			}

			// Ownership will be transferred to render thread, can be nullptr if not created
			FLightmapResourceCluster* ResourceCluster = Lightmap->ResourceCluster.Release();

			FLightmapRenderState::Initializer Initializer {
				Lightmap->Name,
				Lightmap->Size,
				FMath::Min((int32)FMath::CeilLogTwo((uint32)FMath::Min(Lightmap->GetPaddedSizeInTiles().X, Lightmap->GetPaddedSizeInTiles().Y)), GPreviewLightmapMipmapMaxLevel),
				ResourceCluster,
				FVector4f(FVector4(Lightmap->LightmapObject->CoordinateScale, Lightmap->LightmapObject->CoordinateBias))
			};

			InstanceLightmapRenderStateInitializers.Add(Initializer);
		}
		else
		{
			InstanceLightmapRenderStateInitializers.Add(FLightmapRenderState::Initializer{});
		}
	}

	FInstanceGroupRenderState InstanceRenderState;
	InstanceRenderState.ComponentUObject = Instance->ComponentUObject;
	InstanceRenderState.RenderData = Instance->ComponentUObject->GetStaticMesh()->GetRenderData();
	FInstancedStaticMeshSceneProxyDesc ProxyDesc(Instance->ComponentUObject);
	if (DoesPlatformUseInstancedStaticMeshVertexFactory(GMaxRHIShaderPlatform))
	{
		InstanceRenderState.InstancedRenderData = MakeUnique<FInstancedStaticMeshRenderData>(&ProxyDesc, FeatureLevel);
	}
	InstanceRenderState.InstanceDataSceneProxy = ProxyDesc.InstanceDataSceneProxy;
	InstanceRenderState.LocalToWorld = InComponent->GetRenderMatrix();
	InstanceRenderState.WorldBounds = InComponent->Bounds;
	InstanceRenderState.ActorPosition = InComponent->GetActorPositionForRenderer();
	InstanceRenderState.LocalBounds = InComponent->CalcBounds(FTransform::Identity);
	InstanceRenderState.bCastShadow = InComponent->CastShadow && InComponent->bCastStaticShadow;

	InstanceRenderState.ComponentUObject->SetBakedLightingDataChangedAll();

	for (int32 LODIndex = 0; LODIndex < Instance->LODLightmaps.Num(); LODIndex++)
	{
		InstanceRenderState.LODPerInstanceLightmapSize.Add(Instance->LODPerInstanceLightmapSize[LODIndex]);
	}

	TArray<int32> RelevantPointLightsToAddOnRenderThread = AddAllPossiblyRelevantLightsToGeometry(LightScene.PointLights, Instance);
	TArray<int32> RelevantSpotLightsToAddOnRenderThread = AddAllPossiblyRelevantLightsToGeometry(LightScene.SpotLights, Instance);
	TArray<int32> RelevantRectLightsToAddOnRenderThread = AddAllPossiblyRelevantLightsToGeometry(LightScene.RectLights, Instance);

	ENQUEUE_RENDER_COMMAND(RenderThreadInit)(
		[
			FeatureLevel = FeatureLevel,
			InstanceRenderState = MoveTemp(InstanceRenderState),
			InstanceLightmapRenderStateInitializers = MoveTemp(InstanceLightmapRenderStateInitializers),
			&RenderState = RenderState,
			RelevantPointLightsToAddOnRenderThread,
			RelevantSpotLightsToAddOnRenderThread,
			RelevantRectLightsToAddOnRenderThread
		](FRHICommandListImmediate& RHICmdList) mutable
	{
		if (InstanceRenderState.InstancedRenderData)
		{
			InstanceRenderState.InstancedRenderData->BindBuffersToVertexFactories(RHICmdList, nullptr);
		}

		if (FInstanceDataUpdateTaskInfo *TaskInfo = InstanceRenderState.InstanceDataSceneProxy->GetUpdateTaskInfo())
		{
			TaskInfo->WaitForUpdateCompletion();
		}
		InstanceRenderState.InstanceSceneDataBuffers = InstanceRenderState.InstanceDataSceneProxy->GeInstanceSceneDataBuffers();
		InstanceRenderState.NumInstances = InstanceRenderState.InstanceSceneDataBuffers->GetNumInstances();

		FInstanceGroupRenderStateRef InstanceRenderStateRef = RenderState.InstanceGroupRenderStates.Emplace(MoveTemp(InstanceRenderState));

		for (int32 LODIndex = 0; LODIndex < InstanceLightmapRenderStateInitializers.Num(); LODIndex++)
		{
			FLightmapRenderState::Initializer& Initializer = InstanceLightmapRenderStateInitializers[LODIndex];
			if (Initializer.IsValid())
			{
				FLightmapRenderStateRef LightmapRenderState = RenderState.LightmapRenderStates.Emplace(Initializer, RenderState.InstanceGroupRenderStates.CreateGeometryInstanceRef(InstanceRenderStateRef, LODIndex));
				CreateLightmapPreviewVirtualTexture(LightmapRenderState, FeatureLevel, RenderState.LightmapRenderer.Get());


				InstanceRenderStateRef->LODLightmapRenderStates.Emplace(MoveTemp(LightmapRenderState));

				for (int32 ElementId : RelevantPointLightsToAddOnRenderThread)
				{
					LightmapRenderState->AddRelevantLight(FPointLightRenderStateRef(RenderState.LightSceneRenderState.PointLights.Elements[ElementId], RenderState.LightSceneRenderState.PointLights));
				}

				for (int32 ElementId : RelevantSpotLightsToAddOnRenderThread)
				{
					LightmapRenderState->AddRelevantLight(FSpotLightRenderStateRef(RenderState.LightSceneRenderState.SpotLights.Elements[ElementId], RenderState.LightSceneRenderState.SpotLights));
				}

				for (int32 ElementId : RelevantRectLightsToAddOnRenderThread)
				{
					LightmapRenderState->AddRelevantLight(FRectLightRenderStateRef(RenderState.LightSceneRenderState.RectLights.Elements[ElementId], RenderState.LightSceneRenderState.RectLights));
				}
			}
			else
			{
				InstanceRenderStateRef->LODLightmapRenderStates.Emplace(RenderState.LightmapRenderStates.CreateNullRef());
			}
		}

		RenderState.LightmapRenderer->BumpRevision();

		RenderState.CachedRayTracingScene.Reset();
	});

	bNeedsVoxelization = true;
}

void FScene::RemoveGeometryInstanceFromComponent(UInstancedStaticMeshComponent* InComponent)
{
	if (!RegisteredInstancedStaticMeshComponentUObjects.Contains(InComponent))
	{
		return;
	}

	FInstanceGroupRef Instance = RegisteredInstancedStaticMeshComponentUObjects[InComponent];
	for (FLightmapRef& Lightmap : Instance->LODLightmaps)
	{
		if (Lightmap.IsValid())
		{
			Lightmaps.Remove(Lightmap);
		}
	}

	int32 ElementId = Instance.GetElementId();
	InstanceGroups.RemoveAt(ElementId);
	RegisteredInstancedStaticMeshComponentUObjects.Remove(InComponent);

	if (UHierarchicalInstancedStaticMeshComponent* HISMC = Cast<UHierarchicalInstancedStaticMeshComponent>(InComponent))
	{
		HISMC->BuildTreeIfOutdated(false, true);
	}

	InComponent->SetBakedLightingDataChangedAll();

	ENQUEUE_RENDER_COMMAND(RenderThreadRemove)(
		[ElementId, &RenderState = RenderState](FRHICommandListImmediate&) mutable
	{
		if (RenderState.InstanceGroupRenderStates.Elements[ElementId].InstancedRenderData)
		{
			RenderState.InstanceGroupRenderStates.Elements[ElementId].InstancedRenderData->ReleaseResources(nullptr, nullptr);
		}
		for (FLightmapRenderStateRef& Lightmap : RenderState.InstanceGroupRenderStates.Elements[ElementId].LODLightmapRenderStates)
		{
			if (Lightmap.IsValid())
			{
				Lightmap->ReleasePreviewVirtualTexture();
				RenderState.LightmapRenderStates.Remove(Lightmap);
			}
		}

		RenderState.InstanceGroupRenderStates.RemoveAt(ElementId);

		RenderState.LightmapRenderer->BumpRevision();

		RenderState.CachedRayTracingScene.Reset();
	});

	bNeedsVoxelization = true;
}

void FScene::AddGeometryInstanceFromComponent(ULandscapeComponent* InComponent)
{
	if (InComponent->GetLandscapeInfo() == nullptr)
	{
		return;
	}

	if (RegisteredLandscapeComponentUObjects.Contains(InComponent))
	{
		return;
	}

	FLandscapeRef Instance = Landscapes.Emplace(InComponent);
	Instance->WorldBounds = InComponent->Bounds;
	Instance->bCastShadow = InComponent->CastShadow && InComponent->bCastStaticShadow;

	RegisteredLandscapeComponentUObjects.Add(InComponent, Instance);

	Instance->AllocateLightmaps(Lightmaps);

	TArray<FLightmapRenderState::Initializer> InstanceLightmapRenderStateInitializers;

	for (int32 LODIndex = 0; LODIndex < Instance->LODLightmaps.Num(); LODIndex++)
	{
		FLightmapRef& Lightmap = Instance->LODLightmaps[LODIndex];

		if (Lightmap.IsValid())
		{
			Lightmap->CreateGameThreadResources();

			Lightmap->LightmapObject->CoordinateScale = FVector2D(Lightmap->Size) / Lightmap->GetPaddedSize();
			Lightmap->LightmapObject->CoordinateBias = FVector2D(0, 0);

			for (FDirectionalLightBuildInfo& DirectionalLight : LightScene.DirectionalLights.Elements)
			{
				AddLightToLightmap(Lightmap.GetReference_Unsafe(), DirectionalLight);
			}

			// Ownership will be transferred to render thread, can be nullptr if not created
			FLightmapResourceCluster* ResourceCluster = Lightmap->ResourceCluster.Release();

			FLightmapRenderState::Initializer Initializer{
				Lightmap->Name,
				Lightmap->Size,
				FMath::Min((int32)FMath::CeilLogTwo((uint32)FMath::Min(Lightmap->GetPaddedSizeInTiles().X, Lightmap->GetPaddedSizeInTiles().Y)), GPreviewLightmapMipmapMaxLevel),
				ResourceCluster,
				FVector4f(FVector4(Lightmap->LightmapObject->CoordinateScale, Lightmap->LightmapObject->CoordinateBias))
			};

			InstanceLightmapRenderStateInitializers.Add(Initializer);
		}
		else
		{
			InstanceLightmapRenderStateInitializers.Add(FLightmapRenderState::Initializer{});
		}
	}

	FLandscapeRenderState InstanceRenderState;
	InstanceRenderState.ComponentUObject = Instance->ComponentUObject;
	InstanceRenderState.LocalToWorld = InComponent->GetRenderMatrix();
	InstanceRenderState.WorldBounds = InComponent->Bounds;
	InstanceRenderState.ActorPosition = InComponent->GetActorPositionForRenderer();
	InstanceRenderState.LocalBounds = InComponent->CalcBounds(FTransform::Identity);
	InstanceRenderState.bCastShadow = InComponent->CastShadow && InComponent->bCastStaticShadow;

	const int8 SubsectionSizeLog2 = FMath::CeilLogTwo(InComponent->SubsectionSizeQuads + 1);
	InstanceRenderState.SharedBuffersKey = (SubsectionSizeLog2 & 0xf) | ((InComponent->NumSubsections & 0xf) << 4);
	InstanceRenderState.SharedBuffersKey |= 1 << 29; // Use this bit to indicate it is GPULightmass specific buffer (which only has FixedGridVertexFactory created)

	TArray<UMaterialInterface*> AvailableMaterials;

	if (InComponent->GetLandscapeProxy()->bUseDynamicMaterialInstance)
	{
		AvailableMaterials.Append(InComponent->MaterialInstancesDynamic);
	}
	else
	{
		AvailableMaterials.Append(InComponent->MaterialInstances);
	}

	int32 LODIndex = 0;
	InstanceRenderState.MaterialInterface = AvailableMaterials[InComponent->LODIndexToMaterialIndex[LODIndex]];

	InstanceRenderState.LocalToWorldNoScaling = InstanceRenderState.LocalToWorld;
	InstanceRenderState.LocalToWorldNoScaling.RemoveScaling();

	FLandscapeRenderState::Initializer Initializer;
	Initializer.SubsectionSizeQuads        = InComponent->SubsectionSizeQuads;
	Initializer.SubsectionSizeVerts        = InComponent->SubsectionSizeQuads + 1;
	Initializer.NumSubsections             = InComponent->NumSubsections;
	Initializer.ComponentSizeQuads         = InComponent->ComponentSizeQuads;
	Initializer.ComponentSizeVerts         = InComponent->ComponentSizeQuads + 1;
	Initializer.StaticLightingResolution   = InComponent->StaticLightingResolution > 0.f ? InComponent->StaticLightingResolution : InComponent->GetLandscapeProxy()->StaticLightingResolution;
	Initializer.StaticLightingLOD          = InComponent->GetLandscapeProxy()->StaticLightingLOD;
	Initializer.ComponentBase              = InComponent->GetSectionBase() / InComponent->ComponentSizeQuads;
	Initializer.SectionBase                = InComponent->GetSectionBase();
	Initializer.HeightmapTexture           = InComponent->GetHeightmap();
	Initializer.HeightmapSubsectionOffsetU = ((float)(InComponent->SubsectionSizeQuads + 1) / (float)InComponent->GetHeightmap()->GetSizeX());
	Initializer.HeightmapSubsectionOffsetV = ((float)(InComponent->SubsectionSizeQuads + 1) / (float)InComponent->GetHeightmap()->GetSizeY());
	Initializer.HeightmapScaleBias         = (FVector4f)InComponent->HeightmapScaleBias;
	Initializer.WeightmapScaleBias         = (FVector4f)InComponent->WeightmapScaleBias;
	Initializer.WeightmapSubsectionOffset  = InComponent->WeightmapSubsectionOffset;
	Initializer.InvLODBlendRange           = 1.0f / FMath::Max(InComponent->GetLandscapeProxy()->LODBlendRange, 0.01f);

	TArray<int32> RelevantPointLightsToAddOnRenderThread = AddAllPossiblyRelevantLightsToGeometry(LightScene.PointLights, Instance);
	TArray<int32> RelevantSpotLightsToAddOnRenderThread = AddAllPossiblyRelevantLightsToGeometry(LightScene.SpotLights, Instance);
	TArray<int32> RelevantRectLightsToAddOnRenderThread = AddAllPossiblyRelevantLightsToGeometry(LightScene.RectLights, Instance);

	ENQUEUE_RENDER_COMMAND(RenderThreadInit)(
		[
			InstanceRenderState = MoveTemp(InstanceRenderState),
			LocalFeatureLevel = FeatureLevel,
			Initializer,
			InstanceLightmapRenderStateInitializers = MoveTemp(InstanceLightmapRenderStateInitializers),
			&RenderState = RenderState,
			RelevantPointLightsToAddOnRenderThread,
			RelevantSpotLightsToAddOnRenderThread,
			RelevantRectLightsToAddOnRenderThread
		](FRHICommandListImmediate& RHICmdList) mutable
	{
		InstanceRenderState.SharedBuffers = FLandscapeComponentSceneProxy::SharedBuffersMap.FindRef(InstanceRenderState.SharedBuffersKey);
		if (InstanceRenderState.SharedBuffers == nullptr)
		{
			int NumRTSections = 1;
			InstanceRenderState.SharedBuffers = new FLandscapeSharedBuffers(
				RHICmdList,
				InstanceRenderState.SharedBuffersKey, Initializer.SubsectionSizeQuads, Initializer.NumSubsections, NumRTSections,
				LocalFeatureLevel);

			FLandscapeComponentSceneProxy::SharedBuffersMap.Add(InstanceRenderState.SharedBuffersKey, InstanceRenderState.SharedBuffers);

			FLandscapeFixedGridVertexFactory* LandscapeVertexFactory = new FLandscapeFixedGridVertexFactory(LocalFeatureLevel);
			LandscapeVertexFactory->Data.PositionComponent = FVertexStreamComponent(InstanceRenderState.SharedBuffers->VertexBuffer, 0, sizeof(FLandscapeVertex), VET_UByte4);
			LandscapeVertexFactory->InitResource(RHICmdList);
			InstanceRenderState.SharedBuffers->FixedGridVertexFactory = LandscapeVertexFactory;
		}
		check(InstanceRenderState.SharedBuffers);
		InstanceRenderState.SharedBuffers->AddRef();

		InstanceRenderState.SubsectionSizeVerts = Initializer.SubsectionSizeVerts;
		InstanceRenderState.NumSubsections = Initializer.NumSubsections;

		FLandscapeRenderStateRef InstanceRenderStateRef = RenderState.LandscapeRenderStates.Emplace(MoveTemp(InstanceRenderState));

		int32 MaxLOD = 0;
		InstanceRenderStateRef->LandscapeFixedGridUniformShaderParameters.AddDefaulted(MaxLOD + 1);
		for (int32 LodIndex = 0; LodIndex <= MaxLOD; ++LodIndex)
		{
			InstanceRenderStateRef->LandscapeFixedGridUniformShaderParameters[LodIndex].InitResource(RHICmdList);
			FLandscapeFixedGridUniformShaderParameters Parameters;
			Parameters.LodValues = FVector4f(
				LodIndex,
				0.f,
				(float)((InstanceRenderStateRef->SubsectionSizeVerts >> LodIndex) - 1),
				1.f / (float)((InstanceRenderStateRef->SubsectionSizeVerts >> LodIndex) - 1));
			InstanceRenderStateRef->LandscapeFixedGridUniformShaderParameters[LodIndex].SetContents(RHICmdList, Parameters);
		}

		{
			// Set Lightmap ScaleBias
			int32 PatchExpandCountX = 0;
			int32 PatchExpandCountY = 0;
			int32 DesiredSize = 1; // output by GetTerrainExpandPatchCount but not used below
			const float LightMapRatio = GetTerrainExpandPatchCount(Initializer.StaticLightingResolution, PatchExpandCountX, PatchExpandCountY, Initializer.ComponentSizeQuads, (Initializer.NumSubsections * (Initializer.SubsectionSizeQuads + 1)), DesiredSize, Initializer.StaticLightingLOD);
			const float LightmapLODScaleX = LightMapRatio / ((Initializer.ComponentSizeVerts >> Initializer.StaticLightingLOD) + 2 * PatchExpandCountX);
			const float LightmapLODScaleY = LightMapRatio / ((Initializer.ComponentSizeVerts >> Initializer.StaticLightingLOD) + 2 * PatchExpandCountY);
			const float LightmapBiasX = PatchExpandCountX * LightmapLODScaleX;
			const float LightmapBiasY = PatchExpandCountY * LightmapLODScaleY;
			const float LightmapScaleX = LightmapLODScaleX * (float)((Initializer.ComponentSizeVerts >> Initializer.StaticLightingLOD) - 1) / Initializer.ComponentSizeQuads;
			const float LightmapScaleY = LightmapLODScaleY * (float)((Initializer.ComponentSizeVerts >> Initializer.StaticLightingLOD) - 1) / Initializer.ComponentSizeQuads;
			const float LightmapExtendFactorX = (float)Initializer.SubsectionSizeQuads * LightmapScaleX;
			const float LightmapExtendFactorY = (float)Initializer.SubsectionSizeQuads * LightmapScaleY;

			// Set FLandscapeUniformVSParameters for this subsection
			FLandscapeUniformShaderParameters LandscapeParams;
			LandscapeParams.ComponentBaseX = Initializer.ComponentBase.X;
			LandscapeParams.ComponentBaseY = Initializer.ComponentBase.Y;
			LandscapeParams.SubsectionSizeVerts = Initializer.SubsectionSizeVerts;
			LandscapeParams.NumSubsections = Initializer.NumSubsections;
			LandscapeParams.LastLOD = FMath::CeilLogTwo(Initializer.SubsectionSizeQuads + 1) - 1;
			LandscapeParams.HeightmapUVScaleBias = Initializer.HeightmapScaleBias;
			LandscapeParams.WeightmapUVScaleBias = Initializer.WeightmapScaleBias;
			LandscapeParams.LocalToWorldNoScaling = FMatrix44f(InstanceRenderState.LocalToWorldNoScaling);				// LWC_TODO: Precision loss
			LandscapeParams.InvLODBlendRange = Initializer.InvLODBlendRange;

			LandscapeParams.LandscapeLightmapScaleBias = FVector4f(
				LightmapScaleX,
				LightmapScaleY,
				LightmapBiasY,
				LightmapBiasX);
			LandscapeParams.SubsectionSizeVertsLayerUVPan = FVector4f(
				Initializer.SubsectionSizeQuads + 1,
				1.f / (float)Initializer.SubsectionSizeQuads,
				Initializer.SectionBase.X,
				Initializer.SectionBase.Y
			);
			LandscapeParams.SubsectionOffsetParams = FVector4f(
				Initializer.HeightmapSubsectionOffsetU,
				Initializer.HeightmapSubsectionOffsetV,
				Initializer.WeightmapSubsectionOffset,
				Initializer.SubsectionSizeQuads
			);
			LandscapeParams.LightmapSubsectionOffsetParams = FVector4f(
				LightmapExtendFactorX,
				LightmapExtendFactorY,
				0,
				0
			);

			LandscapeParams.HeightmapTexture = Initializer.HeightmapTexture->TextureReference.TextureReferenceRHI;
			LandscapeParams.HeightmapTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();

			LandscapeParams.NormalmapTexture = Initializer.HeightmapTexture->TextureReference.TextureReferenceRHI;
			LandscapeParams.NormalmapTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();

			InstanceRenderStateRef->LandscapeUniformShaderParameters = MakeUnique<TUniformBuffer<FLandscapeUniformShaderParameters>>();
			InstanceRenderStateRef->LandscapeUniformShaderParameters->SetContents(RHICmdList, LandscapeParams);
			InstanceRenderStateRef->LandscapeUniformShaderParameters->InitResource(RHICmdList);
		}

		for (int32 LODIndex = 0; LODIndex < InstanceLightmapRenderStateInitializers.Num(); LODIndex++)
		{
			FLightmapRenderState::Initializer& LightmapInitializer = InstanceLightmapRenderStateInitializers[LODIndex];
			if (LightmapInitializer.IsValid())
			{
				FLightmapRenderStateRef LightmapRenderState = RenderState.LightmapRenderStates.Emplace(LightmapInitializer, RenderState.LandscapeRenderStates.CreateGeometryInstanceRef(InstanceRenderStateRef, LODIndex));
				CreateLightmapPreviewVirtualTexture(LightmapRenderState, LocalFeatureLevel, RenderState.LightmapRenderer.Get());

				InstanceRenderStateRef->LODLightmapRenderStates.Emplace(MoveTemp(LightmapRenderState));

				for (int32 ElementId : RelevantPointLightsToAddOnRenderThread)
				{
					LightmapRenderState->AddRelevantLight(FPointLightRenderStateRef(RenderState.LightSceneRenderState.PointLights.Elements[ElementId], RenderState.LightSceneRenderState.PointLights));
				}

				for (int32 ElementId : RelevantSpotLightsToAddOnRenderThread)
				{
					LightmapRenderState->AddRelevantLight(FSpotLightRenderStateRef(RenderState.LightSceneRenderState.SpotLights.Elements[ElementId], RenderState.LightSceneRenderState.SpotLights));
				}

				for (int32 ElementId : RelevantRectLightsToAddOnRenderThread)
				{
					LightmapRenderState->AddRelevantLight(FRectLightRenderStateRef(RenderState.LightSceneRenderState.RectLights.Elements[ElementId], RenderState.LightSceneRenderState.RectLights));
				}
			}
			else
			{
				InstanceRenderStateRef->LODLightmapRenderStates.Emplace(RenderState.LightmapRenderStates.CreateNullRef());
			}
		}

		RenderState.LightmapRenderer->BumpRevision();

		RenderState.CachedRayTracingScene.Reset();
	});

	bNeedsVoxelization = true;

	if (InComponent->GetWorld())
	{
		InComponent->GetWorld()->SendAllEndOfFrameUpdates();
	}
}

void FScene::RemoveGeometryInstanceFromComponent(ULandscapeComponent* InComponent)
{
	if (!RegisteredLandscapeComponentUObjects.Contains(InComponent))
	{
		return;
	}

	FLandscapeRef Instance = RegisteredLandscapeComponentUObjects[InComponent];

	for (FLightmapRef& Lightmap : Instance->LODLightmaps)
	{
		if (Lightmap.IsValid())
		{
			Lightmaps.Remove(Lightmap);
		}
	}

	int32 ElementId = Instance.GetElementId();
	Landscapes.RemoveAt(ElementId);
	RegisteredLandscapeComponentUObjects.Remove(InComponent);

	if (InComponent->GetLandscapeProxy())
	{
		TSet<ULandscapeComponent*> Components;
		Components.Add(InComponent);
		InComponent->GetLandscapeProxy()->FlushGrassComponents(&Components, false);
	}

	ENQUEUE_RENDER_COMMAND(RenderThreadRemove)(
		[ElementId, &RenderState = RenderState](FRHICommandListImmediate&) mutable
	{
		FLandscapeRenderState& LandscapeRenderState = RenderState.LandscapeRenderStates.Elements[ElementId];
		LandscapeRenderState.SharedBuffers->Release();
		LandscapeRenderState.SharedBuffers = nullptr;

		LandscapeRenderState.LandscapeUniformShaderParameters->ReleaseResource();

		for (auto& UniformBuffer : LandscapeRenderState.LandscapeFixedGridUniformShaderParameters)
		{
			UniformBuffer.ReleaseResource();
		}

		if (IsRayTracingEnabled())
		{
			for (int32 SubY = 0; SubY < LandscapeRenderState.NumSubsections; SubY++)
			{
				for (int32 SubX = 0; SubX < LandscapeRenderState.NumSubsections; SubX++)
				{
					const int8 SubSectionIdx = SubX + SubY * LandscapeRenderState.NumSubsections;

					if (LandscapeRenderState.SectionRayTracingStates[SubSectionIdx].IsValid())
					{
						LandscapeRenderState.SectionRayTracingStates[SubSectionIdx]->Geometry.ReleaseResource();
					}
				}
			}
		}

		for (FLightmapRenderStateRef& Lightmap : RenderState.LandscapeRenderStates.Elements[ElementId].LODLightmapRenderStates)
		{
			if (Lightmap.IsValid())
			{
				Lightmap->ReleasePreviewVirtualTexture();
				RenderState.LightmapRenderStates.Remove(Lightmap);
			}
		}

		RenderState.LandscapeRenderStates.RemoveAt(ElementId);

		RenderState.LightmapRenderer->BumpRevision();

		RenderState.CachedRayTracingScene.Reset();
	});

	bNeedsVoxelization = true;

	if (InComponent->GetWorld())
	{
		InComponent->GetWorld()->SendAllEndOfFrameUpdates();
	}
}

void FScene::BackgroundTick()
{
	int32 Percentage = FPlatformAtomics::AtomicRead(&RenderState.Percentage);

	if (Percentage != 100)
	{
		GPULightmass->ShowLightBuildNotification();
	}
	else
	{
		GPULightmass->RemoveLightBuildNotification();
	}
	
	if (GPULightmass->LightBuildNotification.IsValid())
	{
		const bool bIsViewportNonRealtime = !FGPULightmassModule::IsRealtimeOn();
		if (bIsViewportNonRealtime)
		{
			if (GPULightmass->Settings->Mode == EGPULightmassMode::FullBake)
			{
				FText Text = FText::Format(LOCTEXT("LightBuildProgressMessage", "Building lighting{0}:  {1}%"), FText(), FText::AsNumber(Percentage));
				GPULightmass->LightBuildNotification->SetText(Text);
			}
			else
			{
				FText Text = FText::Format(LOCTEXT("LightBuildProgressForCurrentViewMessage", "Building lighting for current view{0}:  {1}%"), FText(), FText::AsNumber(Percentage));
				GPULightmass->LightBuildNotification->SetText(Text);
			}
		}
		else
		{
			if (GPULightmass->Settings->Mode == EGPULightmassMode::FullBake)
			{
				FText Text = FText::Format(LOCTEXT("LightBuildProgressSlowModeMessage", "Building lighting{0}:  {1}% (slow mode)"), FText(), FText::AsNumber(Percentage));
				GPULightmass->LightBuildNotification->SetText(Text);
			}
			else
			{
				FText Text = FText::Format(LOCTEXT("LightBuildProgressForCurrentViewSlowModeMessage", "Building lighting for current view{0}:  {1}% (slow mode)"), FText(), FText::AsNumber(Percentage));
				GPULightmass->LightBuildNotification->SetText(Text);
			}
		}
	}
	GPULightmass->LightBuildPercentage = Percentage;

	if (Percentage < 100 || GPULightmass->Settings->Mode == EGPULightmassMode::BakeWhatYouSee)
	{
		bool bIsCompilingShaders = GShaderCompilingManager && GShaderCompilingManager->IsCompiling();

		if (!bIsCompilingShaders)
		{
			if (bNeedsVoxelization)
			{
				GatherImportanceVolumes();

				float TargetDetailCellSize = Settings->VolumetricLightmapDetailCellSize;
				
				ENQUEUE_RENDER_COMMAND(BackgroundTickRenderThread)(
					[&RenderState = RenderState, TargetDetailCellSize](FRHICommandListImmediate& RHICmdList) mutable
				{
					RenderState.VolumetricLightmapRenderer->TargetDetailCellSize = TargetDetailCellSize;
					RenderState.VolumetricLightmapRenderer->VoxelizeScene(RHICmdList);
					RenderState.VolumetricLightmapRenderer->FrameNumber = 0;
					RenderState.VolumetricLightmapRenderer->SamplesTaken = 0;
				});

				bNeedsVoxelization = false;
			}

			ENQUEUE_RENDER_COMMAND(BackgroundTickRenderThread)(
				[&RenderState = RenderState](FRHICommandListImmediate& RHICmdList) mutable
			{
				RenderState.BackgroundTick(RHICmdList);
			});
		}
	}
	else
	{
		if (!bNeedsVoxelization)
		{
			ApplyFinishedLightmapsToWorld();
			GPULightmass->World->GetSubsystem<UGPULightmassSubsystem>()->OnLightBuildEnded().Broadcast();
			// We can't destroy GPULM in its member func
			// FGPULightmassModule::EditorTick() will handle the destruction
		}
	}
}

void FSceneRenderState::BackgroundTick(FRHICommandListImmediate& RHICmdList)
{
	if (IrradianceCache->CurrentRevision != LightmapRenderer->GetCurrentRevision())
	{
		IrradianceCache = MakeUnique<FIrradianceCache>(RHICmdList, Settings->IrradianceCacheQuality, Settings->IrradianceCacheSpacing, Settings->IrradianceCacheCornerRejection);
		IrradianceCache->CurrentRevision = LightmapRenderer->GetCurrentRevision();
	}

	bool bHaveFinishedSurfaceLightmaps = false;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GPULightmassCountProgress);

		uint64 SamplesTaken = 0;
		uint64 TotalSamples = 0;

		// Count surface lightmap work
		if (!LightmapRenderer->bOnlyBakeWhatYouSee)
		{
			// Count work has been done
			for (FLightmapRenderState& Lightmap : LightmapRenderStates.Elements)
			{
				for (int32 Y = 0; Y < Lightmap.GetPaddedSizeInTiles().Y; Y++)
				{
					for (int32 X = 0; X < Lightmap.GetPaddedSizeInTiles().X; X++)
					{
						FTileVirtualCoordinates VirtualCoordinates(FIntPoint(X, Y), 0);

						TotalSamples += LightmapRenderer->NumTotalPassesToRender * GPreviewLightmapPhysicalTileSize * GPreviewLightmapPhysicalTileSize;
						SamplesTaken += (Lightmap.DoesTileHaveValidCPUData(VirtualCoordinates, LightmapRenderer->GetCurrentRevision()) ?
							LightmapRenderer->NumTotalPassesToRender  :
							FMath::Min(Lightmap.RetrieveTileState(VirtualCoordinates).RenderPassIndex, LightmapRenderer->NumTotalPassesToRender - 1)) * GPreviewLightmapPhysicalTileSize * GPreviewLightmapPhysicalTileSize;
					}
				}
			}
		}
		else // LightmapRenderer->bOnlyBakeWhatYouSee == true
		{
			if (LightmapRenderer->RecordedTileRequests.Num() > 0)
			{
				for (FLightmapTileRequest& Tile : LightmapRenderer->RecordedTileRequests)
				{
					TotalSamples += LightmapRenderer->NumTotalPassesToRender * GPreviewLightmapPhysicalTileSize * GPreviewLightmapPhysicalTileSize;

					SamplesTaken += (Tile.RenderState->DoesTileHaveValidCPUData(Tile.VirtualCoordinates, LightmapRenderer->GetCurrentRevision()) ?
						LightmapRenderer->NumTotalPassesToRender :
						FMath::Min(Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).RenderPassIndex, LightmapRenderer->NumTotalPassesToRender - 1)) * GPreviewLightmapPhysicalTileSize * GPreviewLightmapPhysicalTileSize;
				}
			}
			else
			{
				for (TArray<FLightmapTileRequest>& FrameRequests : LightmapRenderer->TilesVisibleLastFewFrames)
				{
					for (FLightmapTileRequest& Tile : FrameRequests)
					{
						TotalSamples += LightmapRenderer->NumTotalPassesToRender  * GPreviewLightmapPhysicalTileSize * GPreviewLightmapPhysicalTileSize;

						SamplesTaken += (Tile.RenderState->DoesTileHaveValidCPUData(Tile.VirtualCoordinates, LightmapRenderer->GetCurrentRevision()) ?
							LightmapRenderer->NumTotalPassesToRender :
							FMath::Min(Tile.RenderState->RetrieveTileState(Tile.VirtualCoordinates).RenderPassIndex, LightmapRenderer->NumTotalPassesToRender - 1)) * GPreviewLightmapPhysicalTileSize * GPreviewLightmapPhysicalTileSize;
					}
				}
			}
		}

		bHaveFinishedSurfaceLightmaps = SamplesTaken == TotalSamples;

		{
			int32 NumCellsPerBrick = 5 * 5 * 5;
			// VLM traces 2 paths samples per pass
			// Multiply by 2 to get a more linear progress
			SamplesTaken += 2 * VolumetricLightmapRenderer->SamplesTaken * NumCellsPerBrick;
			TotalSamples += 2 * (uint64)VolumetricLightmapRenderer->NumTotalBricks * NumCellsPerBrick * VolumetricLightmapRenderer->NumTotalPassesToRender;
		}

		int32 IntPercentage = FMath::FloorToInt(SamplesTaken * 100.0 / TotalSamples);
		IntPercentage = FMath::Max(IntPercentage, 0);
		// With high number of samples (like 8192) double precision isn't enough to prevent fake 100%s
		if (SamplesTaken < TotalSamples)
		{
			IntPercentage = FMath::Min(IntPercentage, 99);
		}
		else
		{
			IntPercentage = 100;
		}

		FPlatformAtomics::InterlockedExchange(&Percentage, IntPercentage);
	}

	LightmapRenderer->BackgroundTick(RHICmdList);

	// If we're in background baking mode, schedule VLM work to be after surface lightmaps
	const bool bIsViewportNonRealtime = !FGPULightmassModule::IsRealtimeOn();
	if (!bIsViewportNonRealtime || (bIsViewportNonRealtime && bHaveFinishedSurfaceLightmaps))
	{
		VolumetricLightmapRenderer->BackgroundTick(RHICmdList);
	}
}

template<typename CopyFunc>
void CopyRectTiled(
	FIntPoint SrcMin,
	FIntRect DstRect,
	int32 SrcRowPitchInPixels,
	int32 DstRowPitchInPixels,
	CopyFunc Func,
	int32 VirtualTileSize = GPreviewLightmapVirtualTileSize,
	int32 PhysicalTileSize = GPreviewLightmapVirtualTileSize,
	int32 TileBorderSize = 0
)
{
	ParallelFor(DstRect.Max.Y - DstRect.Min.Y, [&](int32 dy)
	{
		int32 Y = DstRect.Min.Y + dy;
		for (int32 X = DstRect.Min.X; X < DstRect.Max.X; X++)
		{
			FIntPoint SrcPosition = FIntPoint(X, Y) - DstRect.Min + SrcMin;
			FIntPoint SrcTilePosition(SrcPosition.X / VirtualTileSize, SrcPosition.Y / VirtualTileSize);
			FIntPoint PositionInTile(SrcPosition.X % VirtualTileSize, SrcPosition.Y % VirtualTileSize);

			FIntPoint SrcPixelPosition = PositionInTile + FIntPoint(TileBorderSize, TileBorderSize);
			FIntPoint DstPixelPosition = FIntPoint(X, Y);

			int32 SrcLinearIndex = SrcPixelPosition.Y * SrcRowPitchInPixels + SrcPixelPosition.X;
			int32 DstLinearIndex = DstPixelPosition.Y * DstRowPitchInPixels + DstPixelPosition.X;

			Func(DstLinearIndex, SrcTilePosition, SrcLinearIndex);
		}
	});
}

void ReadbackVolumetricLightmapDataLayerFromGPU(FRHICommandListImmediate& RHICmdList, FVolumetricLightmapDataLayer& Layer, FIntVector Dimensions)
{
	const FRHITextureCreateDesc StagingDesc =
		FRHITextureCreateDesc::Create2D(TEXT("VolumetricLightmapDataLayerReadback"), Layer.Texture->GetSizeX(), Layer.Texture->GetSizeY(), Layer.Texture->GetFormat())
		.SetFlags(ETextureCreateFlags::CPUReadback | ETextureCreateFlags::HideInVisualizeTexture);

	FTextureRHIRef StagingTexture2DSlice = RHICmdList.CreateTexture(StagingDesc);
	FGPUFenceRHIRef Fence = RHICreateGPUFence(TEXT("VolumetricLightmapDataLayerReadback"));

	check(Dimensions.Z == Layer.Texture->GetSizeZ());

	Layer.Resize(Dimensions.X * Dimensions.Y * Dimensions.Z * GPixelFormats[Layer.Format].BlockBytes);

	for (int32 SliceIndex = 0; SliceIndex < (int32)Layer.Texture->GetSizeZ(); SliceIndex++)
	{
		Fence->Clear();

		FRHICopyTextureInfo CopyInfo;
		CopyInfo.Size = FIntVector(Layer.Texture->GetSizeX(), Layer.Texture->GetSizeY(), 1);
		CopyInfo.SourcePosition = FIntVector(0, 0, SliceIndex);
		RHICmdList.CopyTexture(Layer.Texture, StagingTexture2DSlice, CopyInfo);
		RHICmdList.WriteGPUFence(Fence);

		uint8* Buffer;
		int32 RowPitchInPixels;
		int32 Height;
		RHICmdList.MapStagingSurface(StagingTexture2DSlice, Fence, (void*&)Buffer, RowPitchInPixels, Height);
		check(RowPitchInPixels >= Dimensions.X);
		check(Height == Dimensions.Y);

		const int32 SrcPitch = RowPitchInPixels * GPixelFormats[Layer.Format].BlockBytes;
		const int32 DstPitch = Dimensions.X * GPixelFormats[Layer.Format].BlockBytes;
		const int32 DepthPitch = Dimensions.Y * Dimensions.X * GPixelFormats[Layer.Format].BlockBytes;

		const int32 DestZIndex = SliceIndex * DepthPitch;

		for (int32 YIndex = 0; YIndex < Dimensions.Y; YIndex++)
		{
			const int32 DestIndex = DestZIndex + YIndex * DstPitch;
			const int32 SourceIndex = YIndex * SrcPitch;
			FMemory::Memcpy((uint8*)&Layer.Data[DestIndex], (const uint8*)&Buffer[SourceIndex], DstPitch);
		}
		
		RHICmdList.UnmapStagingSurface(StagingTexture2DSlice);
	}

}

void GatherBuildDataResourcesToKeep(const ULevel* InLevel, ULevel* LightingScenario, TSet<FGuid>& BuildDataResourcesToKeep)
{
	// This is only required is using a lighting scenario, otherwise the build data is saved within the level itself and follows it's inclusion in the lighting build.
	if (InLevel && LightingScenario)
	{
		BuildDataResourcesToKeep.Add(InLevel->LevelBuildDataId);

		for (const AActor* Actor : InLevel->Actors)
		{
			if (!Actor) // Skip null actors
			{
				continue;
			}

			for (const UActorComponent* Component : Actor->GetComponents())
			{
				if (!Component) // Skip null components
				{
					continue;
				}

				const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component);
				if (PrimitiveComponent)
				{
					PrimitiveComponent->AddMapBuildDataGUIDs(BuildDataResourcesToKeep);
					continue;
				}

				const ULightComponent* LightComponent = Cast<ULightComponent>(Component);
				if (LightComponent)
				{
					BuildDataResourcesToKeep.Add(LightComponent->LightGuid);
					continue;
				}

				const UReflectionCaptureComponent* ReflectionCaptureComponent = Cast<UReflectionCaptureComponent>(Component);
				if (ReflectionCaptureComponent)
				{
					BuildDataResourcesToKeep.Add(ReflectionCaptureComponent->MapBuildDataId);
					continue;
				}
			}
		}
	}
}

void FLocalLightBuildInfo::AllocateMapBuildData(ULevel* StorageLevel)
{
	check(!CastsStationaryShadow() || GetComponentUObject()->PreviewShadowMapChannel != INDEX_NONE);

	UMapBuildDataRegistry* Registry = StorageLevel->GetOrCreateMapBuildData();
	FLightComponentMapBuildData& LightBuildData = Registry->FindOrAllocateLightBuildData(GetComponentUObject()->LightGuid, true);
	LightBuildData.ShadowMapChannel = CastsStationaryShadow() ? GetComponentUObject()->PreviewShadowMapChannel : INDEX_NONE;
	// Copy to storage
	LightBuildData.DepthMap = LightComponentMapBuildData->DepthMap;
}

void FScene::AddRelevantStaticLightGUIDs(FQuantizedLightmapData* QuantizedLightmapData, const FBoxSphereBounds& WorldBounds)
{
	// Add static lights to lightmap data
	for (FDirectionalLightBuildInfo& DirectionalLight : LightScene.DirectionalLights.Elements)
	{
		if (!DirectionalLight.bStationary)
		{
			UDirectionalLightComponent* Light = DirectionalLight.ComponentUObject;
			QuantizedLightmapData->LightGuids.Add(Light->LightGuid);
		}
	}

	for (FPointLightBuildInfo& PointLight : LightScene.PointLights.Elements)
	{
		if (!PointLight.bStationary)
		{
			UPointLightComponent* Light = PointLight.ComponentUObject;
			if (PointLight.AffectsBounds(WorldBounds))
			{
				QuantizedLightmapData->LightGuids.Add(Light->LightGuid);
			}
		}
	}

	for (FSpotLightBuildInfo& SpotLight : LightScene.SpotLights.Elements)
	{
		if (!SpotLight.bStationary)
		{
			USpotLightComponent* Light = SpotLight.ComponentUObject;
			if (SpotLight.AffectsBounds(WorldBounds))
			{
				QuantizedLightmapData->LightGuids.Add(Light->LightGuid);
			}
		}
	}

	for (FRectLightBuildInfo& RectLight : LightScene.RectLights.Elements)
	{
		if (!RectLight.bStationary)
		{
			URectLightComponent* Light = RectLight.ComponentUObject;
			if (RectLight.AffectsBounds(WorldBounds))
			{
				QuantizedLightmapData->LightGuids.Add(Light->LightGuid);
			}
		}
	}
}

void FScene::ApplyFinishedLightmapsToWorld()
{
	UWorld* World = GPULightmass->World;

	const bool bUseVirtualTextures = UseVirtualTextureLightmap(GetFeatureLevelShaderPlatform(World->GetFeatureLevel()));

	bool bHasSkyShadowing = LightScene.SkyLight.IsSet() && LightScene.SkyLight->CastsStationaryShadow();
	
	{
		FScopedSlowTask SlowTask(3);
		SlowTask.MakeDialog();

		SlowTask.EnterProgressFrame(1, LOCTEXT("InvalidatingPreviousLightingStatus", "Invalidating previous lighting"));

		FGlobalComponentRecreateRenderStateContext RecreateRenderStateContext; // Implicit FlushRenderingCommands();

		ULevel* LightingScenario = World->GetActiveLightingScenario();

		// Now we can access RT scene & preview lightmap textures directly

		TSet<FGuid> BuildDataResourcesToKeep;

		for (int32 LevelIndex = 0; LevelIndex < World->GetNumLevels(); LevelIndex++)
		{
			ULevel* Level = World->GetLevel(LevelIndex);

			if (Level)
			{
				if (!Level->bIsVisible && !Level->bIsLightingScenario)
				{
					// Do not touch invisible, normal levels
					GatherBuildDataResourcesToKeep(Level, LightingScenario, BuildDataResourcesToKeep);
				}
			}
		}

		for (int32 LevelIndex = 0; LevelIndex < World->GetNumLevels(); LevelIndex++)
		{
			ULevel* Level = World->GetLevel(LevelIndex);

			if (Level)
			{
				// Invalidate static lighting for normal visible levels, and the current lighting scenario
				// Since the current lighting scenario can contain build data for invisible normal levels, use BuildDataResourcesToKeep
				if (Level->bIsVisible && (!Level->bIsLightingScenario || Level == LightingScenario))
				{
					Level->ReleaseRenderingResources();

					if (Level->MapBuildData)
					{
						Level->MapBuildData->InvalidateStaticLighting(World, false, &BuildDataResourcesToKeep);
					}
				}
			}
		}

		for (FDirectionalLightBuildInfo& DirectionalLight : LightScene.DirectionalLights.Elements)
		{
			DirectionalLight.AllocateMapBuildData(LightingScenario ? LightingScenario : DirectionalLight.ComponentUObject->GetOwner()->GetLevel());
		}

		for (FPointLightBuildInfo& PointLight : LightScene.PointLights.Elements)
		{
			PointLight.AllocateMapBuildData(LightingScenario ? LightingScenario : PointLight.ComponentUObject->GetOwner()->GetLevel());
		}

		for (FSpotLightBuildInfo& SpotLight : LightScene.SpotLights.Elements)
		{
			SpotLight.AllocateMapBuildData(LightingScenario ? LightingScenario : SpotLight.ComponentUObject->GetOwner()->GetLevel());
		}

		for (FRectLightBuildInfo& RectLight : LightScene.RectLights.Elements)
		{
			RectLight.AllocateMapBuildData(LightingScenario ? LightingScenario : RectLight.ComponentUObject->GetOwner()->GetLevel());
		}

		if (RenderState.VolumetricLightmapRenderer->NumTotalBricks > 0)
		{
			ULevel* SubLevelStorageLevel = LightingScenario ? LightingScenario : ToRawPtr(World->PersistentLevel);
			UMapBuildDataRegistry* SubLevelRegistry = SubLevelStorageLevel->GetOrCreateMapBuildData();
			FPrecomputedVolumetricLightmapData& SubLevelData = SubLevelRegistry->AllocateLevelPrecomputedVolumetricLightmapBuildData(World->PersistentLevel->LevelBuildDataId);

			SubLevelData = *RenderState.VolumetricLightmapRenderer->GetPrecomputedVolumetricLightmapForPreview()->Data;

			ENQUEUE_RENDER_COMMAND(ReadbackVLMDataCmd)([&SubLevelData](FRHICommandListImmediate& RHICmdList) {
				SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::GPU0());
				ReadbackVolumetricLightmapDataLayerFromGPU(RHICmdList, SubLevelData.IndirectionTexture, SubLevelData.IndirectionTextureDimensions);
				ReadbackVolumetricLightmapDataLayerFromGPU(RHICmdList, SubLevelData.BrickData.AmbientVector, SubLevelData.BrickDataDimensions);
				for (int32 i = 0; i < UE_ARRAY_COUNT(SubLevelData.BrickData.SHCoefficients); i++)
				{
					ReadbackVolumetricLightmapDataLayerFromGPU(RHICmdList, SubLevelData.BrickData.SHCoefficients[i], SubLevelData.BrickDataDimensions);
				}
				ReadbackVolumetricLightmapDataLayerFromGPU(RHICmdList, SubLevelData.BrickData.DirectionalLightShadowing, SubLevelData.BrickDataDimensions);
				ReadbackVolumetricLightmapDataLayerFromGPU(RHICmdList, SubLevelData.BrickData.SkyBentNormal, SubLevelData.BrickDataDimensions);
			});
		}

		// Fill non-existing mip 0 tiles by upsampling from higher mips, if available
		if (RenderState.LightmapRenderer->bOnlyBakeWhatYouSee)
		{
			for (FLightmapRenderState& Lightmap : RenderState.LightmapRenderStates.Elements)
			{
				for (int32 TileX = 0; TileX < Lightmap.GetPaddedSizeInTiles().X; TileX++)
				{
					FTileDataLayer::Evict();

					for (int32 TileY = 0; TileY < Lightmap.GetPaddedSizeInTiles().Y; TileY++)
					{
						FTileVirtualCoordinates Coords(FIntPoint(TileX, TileY), 0);
						if (!Lightmap.DoesTileHaveValidCPUData(Coords, RenderState.LightmapRenderer->GetCurrentRevision()))
						{
							if (!Lightmap.TileStorage.Contains(Coords))
							{
								Lightmap.TileStorage.Add(Coords, FTileStorage{});
							}

							for (int32 MipLevel = 0; MipLevel <= Lightmap.GetMaxLevel(); MipLevel++)
							{
								FTileVirtualCoordinates ParentCoords(FIntPoint(TileX / (1 << MipLevel), TileY / (1 << MipLevel)), MipLevel);
								if (Lightmap.DoesTileHaveValidCPUData(ParentCoords, RenderState.LightmapRenderer->GetCurrentRevision()))
								{
									Lightmap.TileStorage[Coords].CPUTextureData[0]->Decompress();
									Lightmap.TileStorage[Coords].CPUTextureData[1]->Decompress();
									Lightmap.TileStorage[Coords].CPUTextureData[2]->Decompress();
									Lightmap.TileStorage[Coords].CPUTextureData[3]->Decompress();

									Lightmap.TileStorage[ParentCoords].CPUTextureData[0]->Decompress();
									Lightmap.TileStorage[ParentCoords].CPUTextureData[1]->Decompress();
									Lightmap.TileStorage[ParentCoords].CPUTextureData[2]->Decompress();
									Lightmap.TileStorage[ParentCoords].CPUTextureData[3]->Decompress();

									for (int32 X = 0; X < GPreviewLightmapVirtualTileSize; X++)
									{
										for (int32 Y = 0; Y < GPreviewLightmapVirtualTileSize; Y++)
										{
											FIntPoint DstPixelPosition = FIntPoint(X, Y);
											FIntPoint SrcPixelPosition = (FIntPoint(TileX, TileY) * GPreviewLightmapVirtualTileSize + FIntPoint(X, Y)) / (1 << MipLevel);
											SrcPixelPosition.X %= GPreviewLightmapVirtualTileSize;
											SrcPixelPosition.Y %= GPreviewLightmapVirtualTileSize;

											int32 DstRowPitchInPixels = GPreviewLightmapVirtualTileSize;
											int32 SrcRowPitchInPixels = GPreviewLightmapVirtualTileSize;

											int32 SrcLinearIndex = SrcPixelPosition.Y * SrcRowPitchInPixels + SrcPixelPosition.X;
											int32 DstLinearIndex = DstPixelPosition.Y * DstRowPitchInPixels + DstPixelPosition.X;

											Lightmap.TileStorage[Coords].CPUTextureData[0]->Data[DstLinearIndex] = Lightmap.TileStorage[ParentCoords].CPUTextureData[0]->Data[SrcLinearIndex];
											Lightmap.TileStorage[Coords].CPUTextureData[1]->Data[DstLinearIndex] = Lightmap.TileStorage[ParentCoords].CPUTextureData[1]->Data[SrcLinearIndex];
											Lightmap.TileStorage[Coords].CPUTextureData[2]->Data[DstLinearIndex] = Lightmap.TileStorage[ParentCoords].CPUTextureData[2]->Data[SrcLinearIndex];
											Lightmap.TileStorage[Coords].CPUTextureData[3]->Data[DstLinearIndex] = Lightmap.TileStorage[ParentCoords].CPUTextureData[3]->Data[SrcLinearIndex];
										}
									}

									break;
								}
							}
						}
					}
				}
			}
		}

		SlowTask.EnterProgressFrame(1, LOCTEXT("EncodingTexturesStaticLightingStatis", "Encoding textures"));

		{
			int32 NumLightmapsToTranscode = 0;

			for (int32 InstanceIndex = 0; InstanceIndex < StaticMeshInstances.Elements.Num(); InstanceIndex++)
			{
				for (int32 LODIndex = 0; LODIndex < StaticMeshInstances.Elements[InstanceIndex].LODLightmaps.Num(); LODIndex++)
				{
					if (StaticMeshInstances.Elements[InstanceIndex].LODLightmaps[LODIndex].IsValid())
					{
						NumLightmapsToTranscode++;
					}
				}
			}

			for (int32 InstanceGroupIndex = 0; InstanceGroupIndex < InstanceGroups.Elements.Num(); InstanceGroupIndex++)
			{
				for (int32 LODIndex = 0; LODIndex < InstanceGroups.Elements[InstanceGroupIndex].LODLightmaps.Num(); LODIndex++)
				{
					if (InstanceGroups.Elements[InstanceGroupIndex].LODLightmaps[LODIndex].IsValid())
					{
						NumLightmapsToTranscode++;
					}
				}
			}

			for (int32 LandscapeIndex = 0; LandscapeIndex < Landscapes.Elements.Num(); LandscapeIndex++)
			{
				for (int32 LODIndex = 0; LODIndex < Landscapes.Elements[LandscapeIndex].LODLightmaps.Num(); LODIndex++)
				{
					if (Landscapes.Elements[LandscapeIndex].LODLightmaps[LODIndex].IsValid())
					{
						NumLightmapsToTranscode++;
					}
				}
			}

			FDenoiserContext DenoiserContext;

			FScopedSlowTask SubSlowTask(NumLightmapsToTranscode, LOCTEXT("TranscodingLightmaps", "Transcoding lightmaps"));
			SubSlowTask.MakeDialog();

			for (int32 InstanceIndex = 0; InstanceIndex < StaticMeshInstances.Elements.Num(); InstanceIndex++)
			{
				for (int32 LODIndex = 0; LODIndex < StaticMeshInstances.Elements[InstanceIndex].LODLightmaps.Num(); LODIndex++)
				{
					if (StaticMeshInstances.Elements[InstanceIndex].LODLightmaps[LODIndex].IsValid())
					{
						if (Settings->DenoisingOptions == EGPULightmassDenoisingOptions::OnCompletion)
						{
							SubSlowTask.EnterProgressFrame(1, LOCTEXT("DenoisingAndTranscodingLightmaps", "Denoising & transcoding lightmaps"));
						}
						else
						{
							SubSlowTask.EnterProgressFrame(1, LOCTEXT("TranscodingLightmaps", "Transcoding lightmaps"));
						}

						FLightmapRenderState& Lightmap = RenderState.LightmapRenderStates.Elements[StaticMeshInstances.Elements[InstanceIndex].LODLightmaps[LODIndex].GetElementId()];

						for (auto& Tile : Lightmap.TileStorage)
						{
							Tile.Value.CPUTextureData[0]->Decompress();
							Tile.Value.CPUTextureData[1]->Decompress();
							Tile.Value.CPUTextureData[2]->Decompress();
							Tile.Value.CPUTextureData[3]->Decompress();
						}

						// Transencode GI layers
						TArray<FLightSampleData> LightSampleData;
						LightSampleData.AddZeroed(Lightmap.GetSize().X * Lightmap.GetSize().Y); // LightSampleData will have different row pitch as VT is padded to tiles

						TArray<FLinearColor> IncidentLighting;
						TArray<FLinearColor> LuminanceSH;
						TArray<FVector3f> SkyBentNormal;
						
						IncidentLighting.AddZeroed(Lightmap.GetSize().X * Lightmap.GetSize().Y);
						LuminanceSH.AddZeroed(Lightmap.GetSize().X * Lightmap.GetSize().Y);
						
						{
							int32 SrcRowPitchInPixels = GPreviewLightmapVirtualTileSize;
							int32 DstRowPitchInPixels = Lightmap.GetSize().X;

							CopyRectTiled(
								FIntPoint(0, 0),
								FIntRect(FIntPoint(0, 0), Lightmap.GetSize()),
								SrcRowPitchInPixels,
								DstRowPitchInPixels,
								[&Lightmap, &IncidentLighting, &LuminanceSH](int32 DstLinearIndex, FIntPoint SrcTilePosition, int32 SrcLinearIndex) mutable
							{
								IncidentLighting[DstLinearIndex] = Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[0]->Data[SrcLinearIndex];
								LuminanceSH[DstLinearIndex] = Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[1]->Data[SrcLinearIndex];
							});

							if (bHasSkyShadowing)
							{
								SkyBentNormal.AddZeroed(Lightmap.GetSize().X * Lightmap.GetSize().Y);

								CopyRectTiled(
									FIntPoint(0, 0),
									FIntRect(FIntPoint(0, 0), Lightmap.GetSize()),
									SrcRowPitchInPixels,
									DstRowPitchInPixels,
									[&Lightmap, &SkyBentNormal](int32 DstLinearIndex, FIntPoint SrcTilePosition, int32 SrcLinearIndex) mutable
									{
										FLinearColor SkyOcclusion = Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[3]->Data[SrcLinearIndex];
										
										// Revert sqrt in LightmapEncoding.ush which was done for preview
										float Length = SkyOcclusion.A * SkyOcclusion.A;
										FVector3f UnpackedBentNormalVector = FVector3f(SkyOcclusion) * 2.0f - 1.0f;
										SkyBentNormal[DstLinearIndex] = UnpackedBentNormalVector * Length;
									});
							}
						}

						if (Settings->DenoisingOptions == EGPULightmassDenoisingOptions::OnCompletion)
						{
							if (Settings->Denoiser == EGPULightmassDenoiser::SimpleFireflyRemover)
							{
								TLightSampleDataProvider<FLinearColor> GISampleData(Lightmap.GetSize(), IncidentLighting, LuminanceSH);
								SimpleFireflyFilter(GISampleData);

								if (bHasSkyShadowing)
								{
									TLightSampleDataProvider<FVector3f> SkyBentNormalSampleData(Lightmap.GetSize(), IncidentLighting, SkyBentNormal);
									SimpleFireflyFilter(SkyBentNormalSampleData);
								}
							}
							else
							{
								if (bHasSkyShadowing)
								{
									DenoiseSkyBentNormal(Lightmap.GetSize(), IncidentLighting, SkyBentNormal, DenoiserContext);
								}
								DenoiseRawData(Lightmap.GetSize(), IncidentLighting, LuminanceSH, DenoiserContext);
							}
						}
							
						for (int32 Y = 0 ; Y < Lightmap.GetSize().Y; Y++)
						{
							for (int32 X = 0 ; X < Lightmap.GetSize().X; X++)
							{
								int32 LinearIndex = Y * Lightmap.GetSize().X + X;
								LightSampleData[LinearIndex] = ConvertToLightSample(IncidentLighting[LinearIndex], LuminanceSH[LinearIndex]);
							}
						}
								
						if (bHasSkyShadowing)
						{
							for (int32 Y = 0 ; Y < Lightmap.GetSize().Y; Y++)
							{
								for (int32 X = 0 ; X < Lightmap.GetSize().X; X++)
								{
									int32 LinearIndex = Y * Lightmap.GetSize().X + X;
									LightSampleData[LinearIndex].SkyOcclusion[0] = SkyBentNormal[LinearIndex].X;
									LightSampleData[LinearIndex].SkyOcclusion[1] = SkyBentNormal[LinearIndex].Y;
									LightSampleData[LinearIndex].SkyOcclusion[2] = SkyBentNormal[LinearIndex].Z;
								}
							}
						}

#if 0
						{
							// Debug: dump color and SH as pfm
							TArray<FVector> Color;
							TArray<FVector> SH;
							Color.AddZeroed(Lightmap.GetSize().X * Lightmap.GetSize().Y);
							SH.AddZeroed(Lightmap.GetSize().X * Lightmap.GetSize().Y);

							for (int32 Y = 0; Y < Lightmap.GetSize().Y; Y++)
							{
								for (int32 X = 0; X < Lightmap.GetSize().X; X++)
								{
									Color[Y * Lightmap.GetSize().X + X][0] = LightSampleData[Y * Lightmap.GetSize().X + X].Coefficients[0][0];
									Color[Y * Lightmap.GetSize().X + X][1] = LightSampleData[Y * Lightmap.GetSize().X + X].Coefficients[0][1];
									Color[Y * Lightmap.GetSize().X + X][2] = LightSampleData[Y * Lightmap.GetSize().X + X].Coefficients[0][2];

									SH[Y * Lightmap.GetSize().X + X][0] = LightSampleData[Y * Lightmap.GetSize().X + X].Coefficients[1][0];
									SH[Y * Lightmap.GetSize().X + X][1] = LightSampleData[Y * Lightmap.GetSize().X + X].Coefficients[1][1];
									SH[Y * Lightmap.GetSize().X + X][2] = LightSampleData[Y * Lightmap.GetSize().X + X].Coefficients[1][2];
								}
							}

							{
								FFileHelper::SaveStringToFile(
									FString::Printf(TEXT("PF\n%d %d\n-1.0\n"), Lightmap.GetSize().X, Lightmap.GetSize().Y),
									*FString::Printf(TEXT("%s_Irradiance_%dspp.pfm"), *Lightmap.Name, GGPULightmassSamplesPerTexel),
									FFileHelper::EEncodingOptions::ForceAnsi
								);

								TArray<uint8> Bytes((uint8*)Color.GetData(), Color.GetTypeSize() * Color.Num());

								FFileHelper::SaveArrayToFile(Bytes, *FString::Printf(TEXT("%s_Irradiance_%dspp.pfm"), *Lightmap.Name, GGPULightmassSamplesPerTexel), &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
							}

							{
								FFileHelper::SaveStringToFile(
									FString::Printf(TEXT("PF\n%d %d\n-1.0\n"), Lightmap.GetSize().X, Lightmap.GetSize().Y),
									*FString::Printf(TEXT("%s_SH_%dspp.pfm"), *Lightmap.Name, GGPULightmassSamplesPerTexel),
									FFileHelper::EEncodingOptions::ForceAnsi
								);

								TArray<uint8> Bytes((uint8*)SH.GetData(), SH.GetTypeSize() * SH.Num());

								FFileHelper::SaveArrayToFile(Bytes, *FString::Printf(TEXT("%s_SH_%dspp.pfm"), *Lightmap.Name, GGPULightmassSamplesPerTexel), &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
							}
						}
#endif

						FQuantizedLightmapData* QuantizedLightmapData = new FQuantizedLightmapData();
						QuantizedLightmapData->SizeX = Lightmap.GetSize().X;
						QuantizedLightmapData->SizeY = Lightmap.GetSize().Y;
						QuantizedLightmapData->bHasSkyShadowing = bHasSkyShadowing;

						QuantizeLightSamples(LightSampleData, QuantizedLightmapData->Data, QuantizedLightmapData->Scale, QuantizedLightmapData->Add);

						AddRelevantStaticLightGUIDs(QuantizedLightmapData, StaticMeshInstances.Elements[InstanceIndex].WorldBounds);

						// Transencode stationary light shadow masks
						TMap<ULightComponent*, FShadowMapData2D*> ShadowMaps;
						{
							auto TransencodeShadowMap = [&Lightmap, &ShadowMaps](
								FLocalLightBuildInfo& LightBuildInfo,
								FLocalLightRenderState& Light
								)
							{
								check(Light.bStationary && Light.bCastShadow);
								check(Light.ShadowMapChannel != INDEX_NONE);
								FQuantizedShadowSignedDistanceFieldData2D* ShadowMap = new FQuantizedShadowSignedDistanceFieldData2D(Lightmap.GetSize().X, Lightmap.GetSize().Y);

								int32 SrcRowPitchInPixels = GPreviewLightmapVirtualTileSize;
								int32 DstRowPitchInPixels = Lightmap.GetSize().X;

								CopyRectTiled(
									FIntPoint(0, 0),
									FIntRect(FIntPoint(0, 0), Lightmap.GetSize()),
									SrcRowPitchInPixels,
									DstRowPitchInPixels,
									[&Lightmap, &ShadowMap, &Light](int32 DstLinearIndex, FIntPoint SrcTilePosition, int32 SrcLinearIndex) mutable
								{
									ShadowMap->GetData()[DstLinearIndex] = ConvertToShadowSample(Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[2]->Data[SrcLinearIndex], Light.ShadowMapChannel);
								});

								ShadowMaps.Add(LightBuildInfo.GetComponentUObject(), ShadowMap);
							};

							// For all relevant lights
							// Directional lights are always relevant
							for (FDirectionalLightBuildInfo& DirectionalLight : LightScene.DirectionalLights.Elements)
							{
								if (!DirectionalLight.CastsStationaryShadow())
								{
									continue;
								}

								int32 ElementId = &DirectionalLight - LightScene.DirectionalLights.Elements.GetData();
								TransencodeShadowMap(DirectionalLight, RenderState.LightSceneRenderState.DirectionalLights.Elements[ElementId]);
							}

							for (FPointLightRenderStateRef& PointLight : Lightmap.RelevantPointLights)
							{
								int32 ElementId = PointLight.GetElementIdChecked();
								TransencodeShadowMap(LightScene.PointLights.Elements[ElementId], PointLight);
							}

							for (FSpotLightRenderStateRef& SpotLight : Lightmap.RelevantSpotLights)
							{
								int32 ElementId = SpotLight.GetElementIdChecked();
								TransencodeShadowMap(LightScene.SpotLights.Elements[ElementId], SpotLight);
							}

							for (FRectLightRenderStateRef& RectLight : Lightmap.RelevantRectLights)
							{
								int32 ElementId = RectLight.GetElementIdChecked();
								TransencodeShadowMap(LightScene.RectLights.Elements[ElementId], RectLight);
							}
						}

						{
							UStaticMeshComponent* StaticMeshComponent = StaticMeshInstances.Elements[InstanceIndex].ComponentUObject;
							if (StaticMeshComponent && StaticMeshComponent->GetOwner() && StaticMeshComponent->GetOwner()->GetLevel())
							{
								// Should have happened at a higher level
								check(!StaticMeshComponent->IsRenderStateCreated());
								// The rendering thread reads from LODData and IrrelevantLights, therefore
								// the component must have finished detaching from the scene on the rendering
								// thread before it is safe to continue.
								check(StaticMeshComponent->GetSceneData().AttachmentCounter.GetValue() == 0);

								// Ensure LODData has enough entries in it, free not required.
								const bool bLODDataCountChanged = StaticMeshComponent->SetLODDataCount(LODIndex + 1, StaticMeshComponent->GetStaticMesh()->GetNumLODs());
								if (bLODDataCountChanged)
								{
									StaticMeshComponent->MarkPackageDirty();
								}

								ELightMapPaddingType PaddingType = GAllowLightmapPadding ? LMPT_NormalPadding : LMPT_NoPadding;
								const bool bHasNonZeroData = QuantizedLightmapData->HasNonZeroData();

								ULevel* StorageLevel = LightingScenario ? LightingScenario : StaticMeshComponent->GetOwner()->GetLevel();
								UMapBuildDataRegistry* Registry = StorageLevel->GetOrCreateMapBuildData();

								// Nanite meshes have only LOD0 technically, but that has some weird interaction with ClampedMinLOD
								// Allocate for LOD0 only in that case
								int32 LODLevelToStoreLODInfo = StaticMeshComponent->GetStaticMesh()->HasValidNaniteData() ? 0 : LODIndex;
								FStaticMeshComponentLODInfo& ComponentLODInfo = StaticMeshComponent->LODData[LODLevelToStoreLODInfo];

								// Detect duplicated MapBuildDataId
								if (Registry->GetMeshBuildDataDuringBuild(ComponentLODInfo.MapBuildDataId))
								{
									ComponentLODInfo.MapBuildDataId.Invalidate();

									if (LODLevelToStoreLODInfo > 0)
									{
										// Non-zero LODs derive their MapBuildDataId from LOD0. In this case also regenerate LOD0 GUID
										StaticMeshComponent->LODData[0].MapBuildDataId.Invalidate();
										verify(StaticMeshComponent->LODData[0].CreateMapBuildDataId(0));
									}
								}

								if (ComponentLODInfo.CreateMapBuildDataId(LODLevelToStoreLODInfo))
								{
									StaticMeshComponent->MarkPackageDirty();
								}

								FMeshMapBuildData& MeshBuildData = Registry->AllocateMeshBuildData(ComponentLODInfo.MapBuildDataId, true);

								const bool bNeedsLightMap = bHasNonZeroData || (bUseVirtualTextures && ShadowMaps.Num() > 0);
								if (bNeedsLightMap)
								{
									// Create a light-map for the primitive.
									TMap<ULightComponent*, FShadowMapData2D*> EmptyShadowMapData;
									MeshBuildData.LightMap = FLightMap2D::AllocateLightMap(
										Registry,
										QuantizedLightmapData,
										bUseVirtualTextures ? ShadowMaps : EmptyShadowMapData,
										StaticMeshComponent->Bounds,
										PaddingType,
										LMF_Streamed
									);
								}
								else
								{
									MeshBuildData.LightMap = NULL;
									delete QuantizedLightmapData;
								}

								if (ShadowMaps.Num() > 0 && !bUseVirtualTextures)
								{
									MeshBuildData.ShadowMap = FShadowMap2D::AllocateShadowMap(
										Registry,
										ShadowMaps,
										StaticMeshComponent->Bounds,
										PaddingType,
										SMF_Streamed
									);
								}
								else
								{
									MeshBuildData.ShadowMap = NULL;
								}
							}
						}

						FTileDataLayer::Evict();
					}
				}
			}

			for (int32 InstanceGroupIndex = 0; InstanceGroupIndex < InstanceGroups.Elements.Num(); InstanceGroupIndex++)
			{
				for (int32 LODIndex = 0; LODIndex < InstanceGroups.Elements[InstanceGroupIndex].LODLightmaps.Num(); LODIndex++)
				{
					if (InstanceGroups.Elements[InstanceGroupIndex].LODLightmaps[LODIndex].IsValid())
					{
						if (Settings->DenoisingOptions == EGPULightmassDenoisingOptions::OnCompletion)
						{
							SubSlowTask.EnterProgressFrame(1, LOCTEXT("DenoisingAndTranscodingLightmaps", "Denoising & transcoding lightmaps"));
						}
						else
						{
							SubSlowTask.EnterProgressFrame(1, LOCTEXT("TranscodingLightmaps", "Transcoding lightmaps"));
						}

						FLightmapRenderState& Lightmap = RenderState.LightmapRenderStates.Elements[InstanceGroups.Elements[InstanceGroupIndex].LODLightmaps[LODIndex].GetElementId()];

						for (auto& Tile : Lightmap.TileStorage)
						{
							Tile.Value.CPUTextureData[0]->Decompress();
							Tile.Value.CPUTextureData[1]->Decompress();
							Tile.Value.CPUTextureData[2]->Decompress();
							Tile.Value.CPUTextureData[3]->Decompress();
						}

						FInstanceGroup& InstanceGroup = InstanceGroups.Elements[InstanceGroupIndex];

						int32 BaseLightMapWidth = InstanceGroup.LODPerInstanceLightmapSize[LODIndex].X;
						int32 BaseLightMapHeight = InstanceGroup.LODPerInstanceLightmapSize[LODIndex].Y;

						int32 InstancesPerRow = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(InstanceGroup.ComponentUObject->PerInstanceSMData.Num())));

						// Transencode GI layers
						TArray<TArray<FLightSampleData>> InstanceGroupLightSampleData;
						TArray<TUniquePtr<FQuantizedLightmapData>> InstancedSourceQuantizedData;
						TArray<TMap<ULightComponent*, TUniquePtr<FShadowMapData2D>>> InstancedShadowMapData;
						InstanceGroupLightSampleData.AddDefaulted(InstanceGroup.ComponentUObject->PerInstanceSMData.Num());
						InstancedSourceQuantizedData.AddDefaulted(InstanceGroup.ComponentUObject->PerInstanceSMData.Num());
						InstancedShadowMapData.AddDefaulted(InstanceGroup.ComponentUObject->PerInstanceSMData.Num());

						for (int32 InstanceIndex = 0; InstanceIndex < InstanceGroupLightSampleData.Num(); InstanceIndex++)
						{
							TArray<FLinearColor> IncidentLighting;
							TArray<FLinearColor> LuminanceSH;
							TArray<FVector3f> SkyBentNormal;
							IncidentLighting.AddZeroed(BaseLightMapWidth * BaseLightMapHeight);
							LuminanceSH.AddZeroed(BaseLightMapWidth * BaseLightMapHeight);
							
							TArray<FLightSampleData>& LightSampleData = InstanceGroupLightSampleData[InstanceIndex];
							LightSampleData.AddZeroed(BaseLightMapWidth * BaseLightMapHeight);
							InstancedSourceQuantizedData[InstanceIndex] = MakeUnique<FQuantizedLightmapData>();

							int32 SrcRowPitchInPixels = GPreviewLightmapVirtualTileSize;
							int32 DstRowPitchInPixels = BaseLightMapWidth;

							int32 RenderIndex = InstanceGroup.ComponentUObject->GetRenderIndex(InstanceIndex);

							if (RenderIndex != INDEX_NONE)
							{
								FIntPoint InstanceTilePos = FIntPoint(RenderIndex % InstancesPerRow, RenderIndex / InstancesPerRow);
								FIntPoint InstanceTileMin = FIntPoint(InstanceTilePos.X * BaseLightMapWidth, InstanceTilePos.Y * BaseLightMapHeight);
								
								CopyRectTiled(
									InstanceTileMin,
									FIntRect(FIntPoint(0, 0), FIntPoint(BaseLightMapWidth, BaseLightMapHeight)),
									SrcRowPitchInPixels,
									DstRowPitchInPixels,
									[&Lightmap, &IncidentLighting, &LuminanceSH](int32 DstLinearIndex, FIntPoint SrcTilePosition, int32 SrcLinearIndex) mutable
								{
									IncidentLighting[DstLinearIndex] = Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[0]->Data[SrcLinearIndex];
									LuminanceSH[DstLinearIndex] = Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[1]->Data[SrcLinearIndex];
								});

								if (bHasSkyShadowing)
								{
									SkyBentNormal.AddZeroed(BaseLightMapWidth * BaseLightMapHeight);
									
									CopyRectTiled(
										InstanceTileMin,
										FIntRect(FIntPoint(0, 0), FIntPoint(BaseLightMapWidth, BaseLightMapHeight)),
										SrcRowPitchInPixels,
										DstRowPitchInPixels,
										[&Lightmap, &SkyBentNormal](int32 DstLinearIndex, FIntPoint SrcTilePosition, int32 SrcLinearIndex) mutable
										{
											FLinearColor SkyOcclusion = Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[3]->Data[SrcLinearIndex];

											// Revert sqrt in LightmapEncoding.ush which was done for preview
											float Length = SkyOcclusion.A * SkyOcclusion.A;
											FVector3f UnpackedBentNormalVector = FVector3f(SkyOcclusion) * 2.0f - 1.0f;
											SkyBentNormal[DstLinearIndex] = UnpackedBentNormalVector * Length;
										});
								}
								
								if (Settings->DenoisingOptions == EGPULightmassDenoisingOptions::OnCompletion)
								{
									if (Settings->Denoiser == EGPULightmassDenoiser::SimpleFireflyRemover)
									{
										TLightSampleDataProvider<FLinearColor> SampleData(FIntPoint{BaseLightMapWidth, BaseLightMapHeight}, IncidentLighting, LuminanceSH);
										SimpleFireflyFilter(SampleData);
										
										if (bHasSkyShadowing)
										{
											TLightSampleDataProvider<FVector3f> SkyBentNormalSampleData(FIntPoint{BaseLightMapWidth, BaseLightMapHeight}, IncidentLighting, SkyBentNormal);
											SimpleFireflyFilter(SkyBentNormalSampleData);
										}
									}
									else
									{
										if (bHasSkyShadowing)
										{
											DenoiseSkyBentNormal(FIntPoint{BaseLightMapWidth, BaseLightMapHeight}, IncidentLighting, SkyBentNormal, DenoiserContext);
										}
										DenoiseRawData(FIntPoint{BaseLightMapWidth, BaseLightMapHeight}, IncidentLighting, LuminanceSH, DenoiserContext);
									}
								}

								for (int32 Y = 0; Y < BaseLightMapHeight; Y++)
								{
									for (int32 X = 0; X < BaseLightMapWidth; X++)
									{
										int32 LinearIndex = Y * BaseLightMapWidth + X;
										LightSampleData[LinearIndex] = ConvertToLightSample(IncidentLighting[LinearIndex], LuminanceSH[LinearIndex]);
									}
								}

								if (bHasSkyShadowing)
								{
									for (int32 Y = 0; Y < BaseLightMapHeight; Y++)
									{
										for (int32 X = 0; X < BaseLightMapWidth; X++)
										{
											int32 LinearIndex = Y * BaseLightMapWidth + X;
											LightSampleData[LinearIndex].SkyOcclusion[0] = SkyBentNormal[LinearIndex].X;
											LightSampleData[LinearIndex].SkyOcclusion[1] = SkyBentNormal[LinearIndex].Y;
											LightSampleData[LinearIndex].SkyOcclusion[2] = SkyBentNormal[LinearIndex].Z;
										}
									}
								}
							}

							FQuantizedLightmapData& QuantizedLightmapData = *InstancedSourceQuantizedData[InstanceIndex];
							QuantizedLightmapData.SizeX = BaseLightMapWidth;
							QuantizedLightmapData.SizeY = BaseLightMapHeight;
							QuantizedLightmapData.bHasSkyShadowing = bHasSkyShadowing;

							QuantizeLightSamples(LightSampleData, QuantizedLightmapData.Data, QuantizedLightmapData.Scale, QuantizedLightmapData.Add);

							// Transencode stationary light shadow masks
							TMap<ULightComponent*, TUniquePtr<FShadowMapData2D>>& ShadowMaps = InstancedShadowMapData[InstanceIndex];

							{
								// For all relevant lights
								// Directional lights are always relevant
								for (FDirectionalLightBuildInfo& DirectionalLight : LightScene.DirectionalLights.Elements)
								{
									if (!DirectionalLight.CastsStationaryShadow())
									{
										continue;
									}

									check(DirectionalLight.ShadowMapChannel != INDEX_NONE);
									TUniquePtr<FQuantizedShadowSignedDistanceFieldData2D> ShadowMap = MakeUnique<FQuantizedShadowSignedDistanceFieldData2D>(BaseLightMapWidth, BaseLightMapHeight);

									if (RenderIndex != INDEX_NONE)
									{
										FIntPoint InstanceTilePos = FIntPoint(RenderIndex % InstancesPerRow, RenderIndex / InstancesPerRow);
										FIntPoint InstanceTileMin = FIntPoint(InstanceTilePos.X * BaseLightMapWidth, InstanceTilePos.Y * BaseLightMapHeight);

										CopyRectTiled(											
											InstanceTileMin,
											FIntRect(FIntPoint(0, 0), FIntPoint(BaseLightMapWidth, BaseLightMapHeight)),
											SrcRowPitchInPixels,
											DstRowPitchInPixels,
											[&Lightmap, &ShadowMap, &DirectionalLight](int32 DstLinearIndex, FIntPoint SrcTilePosition, int32 SrcLinearIndex) mutable
										{
											ShadowMap->GetData()[DstLinearIndex] = ConvertToShadowSample(Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[2]->Data[SrcLinearIndex], DirectionalLight.ShadowMapChannel);
										});
									}

									ShadowMaps.Add(DirectionalLight.ComponentUObject, MoveTemp(ShadowMap));
								}

								for (FPointLightRenderStateRef& PointLight : Lightmap.RelevantPointLights)
								{
									check(PointLight->bStationary && PointLight->bCastShadow);
									check(PointLight->ShadowMapChannel != INDEX_NONE);
									TUniquePtr<FQuantizedShadowSignedDistanceFieldData2D> ShadowMap = MakeUnique<FQuantizedShadowSignedDistanceFieldData2D>(BaseLightMapWidth, BaseLightMapHeight);

									if (RenderIndex != INDEX_NONE)
									{
										FIntPoint InstanceTilePos = FIntPoint(RenderIndex % InstancesPerRow, RenderIndex / InstancesPerRow);
										FIntPoint InstanceTileMin = FIntPoint(InstanceTilePos.X * BaseLightMapWidth, InstanceTilePos.Y * BaseLightMapHeight);

										CopyRectTiled(
											InstanceTileMin,
											FIntRect(FIntPoint(0, 0), FIntPoint(BaseLightMapWidth, BaseLightMapHeight)),
											SrcRowPitchInPixels,
											DstRowPitchInPixels,
											[&Lightmap, &ShadowMap, &PointLight](int32 DstLinearIndex, FIntPoint SrcTilePosition, int32 SrcLinearIndex) mutable
										{
											ShadowMap->GetData()[DstLinearIndex] = ConvertToShadowSample(Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[2]->Data[SrcLinearIndex], PointLight->ShadowMapChannel);
										});
									}

									ShadowMaps.Add(LightScene.PointLights.Elements[PointLight.GetElementIdChecked()].ComponentUObject, MoveTemp(ShadowMap));
								}

								for (FSpotLightRenderStateRef& SpotLight : Lightmap.RelevantSpotLights)
								{
									check(SpotLight->bStationary && SpotLight->bCastShadow);
									check(SpotLight->ShadowMapChannel != INDEX_NONE);
									TUniquePtr<FQuantizedShadowSignedDistanceFieldData2D> ShadowMap = MakeUnique<FQuantizedShadowSignedDistanceFieldData2D>(BaseLightMapWidth, BaseLightMapHeight);

									if (RenderIndex != INDEX_NONE)
									{
										FIntPoint InstanceTilePos = FIntPoint(RenderIndex % InstancesPerRow, RenderIndex / InstancesPerRow);
										FIntPoint InstanceTileMin = FIntPoint(InstanceTilePos.X * BaseLightMapWidth, InstanceTilePos.Y * BaseLightMapHeight);

										CopyRectTiled(
											InstanceTileMin,
											FIntRect(FIntPoint(0, 0), FIntPoint(BaseLightMapWidth, BaseLightMapHeight)),
											SrcRowPitchInPixels,
											DstRowPitchInPixels,
											[&Lightmap, &ShadowMap, &SpotLight](int32 DstLinearIndex, FIntPoint SrcTilePosition, int32 SrcLinearIndex) mutable
										{
											ShadowMap->GetData()[DstLinearIndex] = ConvertToShadowSample(Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[2]->Data[SrcLinearIndex], SpotLight->ShadowMapChannel);
										});
									}

									ShadowMaps.Add(LightScene.SpotLights.Elements[SpotLight.GetElementIdChecked()].ComponentUObject, MoveTemp(ShadowMap));
								}

								for (FRectLightRenderStateRef& RectLight : Lightmap.RelevantRectLights)
								{
									check(RectLight->bStationary && RectLight->bCastShadow);
									check(RectLight->ShadowMapChannel != INDEX_NONE);
									TUniquePtr<FQuantizedShadowSignedDistanceFieldData2D> ShadowMap = MakeUnique<FQuantizedShadowSignedDistanceFieldData2D>(BaseLightMapWidth, BaseLightMapHeight);

									if (RenderIndex != INDEX_NONE)
									{
										FIntPoint InstanceTilePos = FIntPoint(RenderIndex % InstancesPerRow, RenderIndex / InstancesPerRow);
										FIntPoint InstanceTileMin = FIntPoint(InstanceTilePos.X * BaseLightMapWidth, InstanceTilePos.Y * BaseLightMapHeight);

										CopyRectTiled(
											InstanceTileMin,
											FIntRect(FIntPoint(0, 0), FIntPoint(BaseLightMapWidth, BaseLightMapHeight)),
											SrcRowPitchInPixels,
											DstRowPitchInPixels,
											[&Lightmap, &ShadowMap, &RectLight](int32 DstLinearIndex, FIntPoint SrcTilePosition, int32 SrcLinearIndex) mutable
										{
											ShadowMap->GetData()[DstLinearIndex] = ConvertToShadowSample(Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[2]->Data[SrcLinearIndex], RectLight->ShadowMapChannel);
										});
									}

									ShadowMaps.Add(LightScene.RectLights.Elements[RectLight.GetElementIdChecked()].ComponentUObject, MoveTemp(ShadowMap));
								}
							}
						}

						// Add static lights to lightmap data
						// Instanced lightmaps will eventually be merged together, so just add to the first one
						if (InstancedSourceQuantizedData.Num() > 0)
						{
							TUniquePtr<FQuantizedLightmapData>& QuantizedLightmapData = InstancedSourceQuantizedData[0];
							
							AddRelevantStaticLightGUIDs(QuantizedLightmapData.Get(), InstanceGroup.WorldBounds);
						}

						UStaticMesh* ResolvedMesh = InstanceGroup.ComponentUObject->GetStaticMesh();
						if (InstanceGroup.ComponentUObject->LODData.Num() != ResolvedMesh->GetNumLODs())
						{
							InstanceGroup.ComponentUObject->MarkPackageDirty();
						}

						// Ensure LODData has enough entries in it, free not required.
						InstanceGroup.ComponentUObject->SetLODDataCount(ResolvedMesh->GetNumLODs(), ResolvedMesh->GetNumLODs());

						ULevel* StorageLevel = LightingScenario ? LightingScenario : InstanceGroup.ComponentUObject->GetOwner()->GetLevel();
						UMapBuildDataRegistry* Registry = StorageLevel->GetOrCreateMapBuildData();

						// Nanite meshes have only LOD0 technically, but that has some weird interaction with ClampedMinLOD
						// Allocate for LOD0 only in that case
						int32 LODLevelToStoreLODInfo = InstanceGroup.ComponentUObject->GetStaticMesh()->HasValidNaniteData() ? 0 : LODIndex;
						FStaticMeshComponentLODInfo& ComponentLODInfo = InstanceGroup.ComponentUObject->LODData[LODLevelToStoreLODInfo];
						
						// Detect duplicated MapBuildDataId
						if (Registry->GetMeshBuildDataDuringBuild(ComponentLODInfo.MapBuildDataId))
						{
							ComponentLODInfo.MapBuildDataId.Invalidate();

							if (LODLevelToStoreLODInfo > 0)
							{
								// Non-zero LODs derive their MapBuildDataId from LOD0. In this case also regenerate LOD0 GUID
								InstanceGroup.ComponentUObject->LODData[0].MapBuildDataId.Invalidate();
								check(InstanceGroup.ComponentUObject->LODData[0].CreateMapBuildDataId(0));
							}
						}

						if (ComponentLODInfo.CreateMapBuildDataId(LODLevelToStoreLODInfo))
						{
							InstanceGroup.ComponentUObject->MarkPackageDirty();
						}

						FMeshMapBuildData& MeshBuildData = Registry->AllocateMeshBuildData(InstanceGroup.ComponentUObject->LODData[LODIndex].MapBuildDataId, true);

						MeshBuildData.PerInstanceLightmapData.Empty(InstancedSourceQuantizedData.Num());
						MeshBuildData.PerInstanceLightmapData.AddZeroed(InstancedSourceQuantizedData.Num());

						// Create a light-map for the primitive.
						// When using VT, shadow map data is included with lightmap allocation
						const ELightMapPaddingType PaddingType = GAllowLightmapPadding ? LMPT_NormalPadding : LMPT_NoPadding;

						TArray<TMap<ULightComponent*, TUniquePtr<FShadowMapData2D>>> EmptyShadowMapData;
						TRefCountPtr<FLightMap2D> NewLightMap = FLightMap2D::AllocateInstancedLightMap(Registry, InstanceGroup.ComponentUObject,
							MoveTemp(InstancedSourceQuantizedData),
							bUseVirtualTextures ? MoveTemp(InstancedShadowMapData) : MoveTemp(EmptyShadowMapData),
							Registry, InstanceGroup.ComponentUObject->LODData[LODIndex].MapBuildDataId, InstanceGroup.ComponentUObject->Bounds, PaddingType, LMF_Streamed);

						MeshBuildData.LightMap = NewLightMap;
						
						for (auto It = InstancedShadowMapData.CreateIterator(); It; It++)
						{
							if (It->Num() == 0)
							{
								It.RemoveCurrent();
							}
						}
						
						if (InstancedShadowMapData.Num() > 0 && !bUseVirtualTextures)
						{
							TRefCountPtr<FShadowMap2D> NewShadowMap = FShadowMap2D::AllocateInstancedShadowMap(Registry, InstanceGroup.ComponentUObject,
								MoveTemp(InstancedShadowMapData),
								Registry, InstanceGroup.ComponentUObject->LODData[LODIndex].MapBuildDataId, InstanceGroup.ComponentUObject->Bounds, PaddingType, SMF_Streamed);

							MeshBuildData.ShadowMap = NewShadowMap;
						}

						FTileDataLayer::Evict();
					}
				}
			}

			for (int32 LandscapeIndex = 0; LandscapeIndex < Landscapes.Elements.Num(); LandscapeIndex++)
			{
				for (int32 LODIndex = 0; LODIndex < Landscapes.Elements[LandscapeIndex].LODLightmaps.Num(); LODIndex++)
				{
					if (Landscapes.Elements[LandscapeIndex].LODLightmaps[LODIndex].IsValid())
					{
						if (Settings->DenoisingOptions == EGPULightmassDenoisingOptions::OnCompletion)
						{
							SubSlowTask.EnterProgressFrame(1, LOCTEXT("DenoisingAndTranscodingLightmaps", "Denoising & transcoding lightmaps"));
						}
						else
						{
							SubSlowTask.EnterProgressFrame(1, LOCTEXT("TranscodingLightmaps", "Transcoding lightmaps"));
						}

						FLightmapRenderState& Lightmap = RenderState.LightmapRenderStates.Elements[Landscapes.Elements[LandscapeIndex].LODLightmaps[LODIndex].GetElementId()];

						for (auto& Tile : Lightmap.TileStorage)
						{
							Tile.Value.CPUTextureData[0]->Decompress();
							Tile.Value.CPUTextureData[1]->Decompress();
							Tile.Value.CPUTextureData[2]->Decompress();
							Tile.Value.CPUTextureData[3]->Decompress();
						}

						// Transencode GI layers
						TArray<FLightSampleData> LightSampleData;
						LightSampleData.AddZeroed(Lightmap.GetSize().X * Lightmap.GetSize().Y); // LightSampleData will have different row pitch as VT is padded to tiles

						TArray<FLinearColor> IncidentLighting;
						TArray<FLinearColor> LuminanceSH;
						TArray<FVector3f> SkyBentNormal;
						IncidentLighting.AddZeroed(Lightmap.GetSize().X * Lightmap.GetSize().Y);
						LuminanceSH.AddZeroed(Lightmap.GetSize().X * Lightmap.GetSize().Y);
						
						{
							int32 SrcRowPitchInPixels = GPreviewLightmapVirtualTileSize;
							int32 DstRowPitchInPixels = Lightmap.GetSize().X;

							CopyRectTiled(
								FIntPoint(0, 0),
								FIntRect(FIntPoint(0, 0), Lightmap.GetSize()),
								SrcRowPitchInPixels,
								DstRowPitchInPixels,
								[&Lightmap, &IncidentLighting, &LuminanceSH](int32 DstLinearIndex, FIntPoint SrcTilePosition, int32 SrcLinearIndex) mutable
							{
								IncidentLighting[DstLinearIndex] = Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[0]->Data[SrcLinearIndex];
								LuminanceSH[DstLinearIndex] = Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[1]->Data[SrcLinearIndex];
							});
							
							if (bHasSkyShadowing)
							{
								SkyBentNormal.AddZeroed(Lightmap.GetSize().X * Lightmap.GetSize().Y);

								CopyRectTiled(
									FIntPoint(0, 0),
									FIntRect(FIntPoint(0, 0), Lightmap.GetSize()),
									SrcRowPitchInPixels,
									DstRowPitchInPixels,
									[&Lightmap, &SkyBentNormal](int32 DstLinearIndex, FIntPoint SrcTilePosition, int32 SrcLinearIndex) mutable
									{
										FLinearColor SkyOcclusion = Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[3]->Data[SrcLinearIndex];
										
										// Revert sqrt in LightmapEncoding.ush which was done for preview
										float Length = SkyOcclusion.A * SkyOcclusion.A;
										FVector3f UnpackedBentNormalVector = FVector3f(SkyOcclusion) * 2.0f - 1.0f;
										SkyBentNormal[DstLinearIndex] = UnpackedBentNormalVector * Length;
									});
							}

							if (Settings->DenoisingOptions == EGPULightmassDenoisingOptions::OnCompletion)
							{
								if (Settings->Denoiser == EGPULightmassDenoiser::SimpleFireflyRemover)
								{
									TLightSampleDataProvider<FLinearColor> GISampleData(Lightmap.GetSize(), IncidentLighting, LuminanceSH);
									SimpleFireflyFilter(GISampleData);

									if (bHasSkyShadowing)
									{
										TLightSampleDataProvider<FVector3f> SkyBentNormalSampleData(Lightmap.GetSize(), IncidentLighting, SkyBentNormal);
										SimpleFireflyFilter(SkyBentNormalSampleData);
									}
								}
								else
								{
									if (bHasSkyShadowing)
									{
										DenoiseSkyBentNormal(Lightmap.GetSize(), IncidentLighting, SkyBentNormal, DenoiserContext);
									}
									DenoiseRawData(Lightmap.GetSize(), IncidentLighting, LuminanceSH, DenoiserContext);
								}
							}
							
							for (int32 Y = 0 ; Y < Lightmap.GetSize().Y; Y++)
							{
								for (int32 X = 0 ; X < Lightmap.GetSize().X; X++)
								{
									int32 LinearIndex = Y * Lightmap.GetSize().X + X;
									LightSampleData[LinearIndex] = ConvertToLightSample(IncidentLighting[LinearIndex], LuminanceSH[LinearIndex]);
								}
							}

							if (bHasSkyShadowing)
							{
								for (int32 Y = 0 ; Y < Lightmap.GetSize().Y; Y++)
								{
									for (int32 X = 0 ; X < Lightmap.GetSize().X; X++)
									{
										int32 LinearIndex = Y * Lightmap.GetSize().X + X;
										LightSampleData[LinearIndex].SkyOcclusion[0] = SkyBentNormal[LinearIndex].X;
										LightSampleData[LinearIndex].SkyOcclusion[1] = SkyBentNormal[LinearIndex].Y;
										LightSampleData[LinearIndex].SkyOcclusion[2] = SkyBentNormal[LinearIndex].Z;
									}
								}
							}
						}

						FQuantizedLightmapData* QuantizedLightmapData = new FQuantizedLightmapData();
						QuantizedLightmapData->SizeX = Lightmap.GetSize().X;
						QuantizedLightmapData->SizeY = Lightmap.GetSize().Y;
						QuantizedLightmapData->bHasSkyShadowing = bHasSkyShadowing;

						QuantizeLightSamples(LightSampleData, QuantizedLightmapData->Data, QuantizedLightmapData->Scale, QuantizedLightmapData->Add);

						AddRelevantStaticLightGUIDs(QuantizedLightmapData, Landscapes.Elements[LandscapeIndex].WorldBounds);

						// Transencode stationary light shadow masks
						TMap<ULightComponent*, FShadowMapData2D*> ShadowMaps;
						{
							auto TransencodeShadowMap = [&Lightmap, &ShadowMaps](
								FLocalLightBuildInfo& LightBuildInfo,
								FLocalLightRenderState& Light
								)
							{
								check(Light.bStationary && Light.bCastShadow);
								check(Light.ShadowMapChannel != INDEX_NONE);
								FQuantizedShadowSignedDistanceFieldData2D* ShadowMap = new FQuantizedShadowSignedDistanceFieldData2D(Lightmap.GetSize().X, Lightmap.GetSize().Y);

								int32 SrcRowPitchInPixels = GPreviewLightmapVirtualTileSize;
								int32 DstRowPitchInPixels = Lightmap.GetSize().X;

								CopyRectTiled(
									FIntPoint(0, 0),
									FIntRect(FIntPoint(0, 0), Lightmap.GetSize()),
									SrcRowPitchInPixels,
									DstRowPitchInPixels,
									[&Lightmap, &ShadowMap, &Light](int32 DstLinearIndex, FIntPoint SrcTilePosition, int32 SrcLinearIndex) mutable
								{
									ShadowMap->GetData()[DstLinearIndex] = ConvertToShadowSample(Lightmap.TileStorage[FTileVirtualCoordinates(SrcTilePosition, 0)].CPUTextureData[2]->Data[SrcLinearIndex], Light.ShadowMapChannel);
								});

								ShadowMaps.Add(LightBuildInfo.GetComponentUObject(), ShadowMap);
							};

							// For all relevant lights
							// Directional lights are always relevant
							for (FDirectionalLightBuildInfo& DirectionalLight : LightScene.DirectionalLights.Elements)
							{
								if (!DirectionalLight.CastsStationaryShadow())
								{
									continue;
								}

								int32 ElementId = &DirectionalLight - LightScene.DirectionalLights.Elements.GetData();
								TransencodeShadowMap(DirectionalLight, RenderState.LightSceneRenderState.DirectionalLights.Elements[ElementId]);
							}

							for (FPointLightRenderStateRef& PointLight : Lightmap.RelevantPointLights)
							{
								int32 ElementId = PointLight.GetElementIdChecked();
								TransencodeShadowMap(LightScene.PointLights.Elements[ElementId], PointLight);
							}

							for (FSpotLightRenderStateRef& SpotLight : Lightmap.RelevantSpotLights)
							{
								int32 ElementId = SpotLight.GetElementIdChecked();
								TransencodeShadowMap(LightScene.SpotLights.Elements[ElementId], SpotLight);
							}

							for (FRectLightRenderStateRef& RectLight : Lightmap.RelevantRectLights)
							{
								int32 ElementId = RectLight.GetElementIdChecked();
								TransencodeShadowMap(LightScene.RectLights.Elements[ElementId], RectLight);
							}
						}

						{
							ULandscapeComponent* LandscapeComponent = Landscapes.Elements[LandscapeIndex].ComponentUObject;
							ELightMapPaddingType PaddingType = LMPT_NoPadding;
							const bool bHasNonZeroData = QuantizedLightmapData->HasNonZeroData() || (bUseVirtualTextures && ShadowMaps.Num() > 0);

							ULevel* StorageLevel = LightingScenario ? LightingScenario : LandscapeComponent->GetOwner()->GetLevel();
							UMapBuildDataRegistry* Registry = StorageLevel->GetOrCreateMapBuildData();
							FMeshMapBuildData& MeshBuildData = Registry->AllocateMeshBuildData(LandscapeComponent->MapBuildDataId, true);

							const bool bNeedsLightMap = true;// bHasNonZeroData;
							if (bNeedsLightMap)
							{
								// Create a light-map for the primitive.
								TMap<ULightComponent*, FShadowMapData2D*> EmptyShadowMapData;
								MeshBuildData.LightMap = FLightMap2D::AllocateLightMap(
									Registry,
									QuantizedLightmapData,
									bUseVirtualTextures ? ShadowMaps : EmptyShadowMapData,
									LandscapeComponent->Bounds,
									PaddingType,
									LMF_Streamed
								);
							}
							else
							{
								MeshBuildData.LightMap = NULL;
								delete QuantizedLightmapData;
							}

							if (ShadowMaps.Num() > 0 && !bUseVirtualTextures)
							{
								MeshBuildData.ShadowMap = FShadowMap2D::AllocateShadowMap(
									Registry,
									ShadowMaps,
									LandscapeComponent->Bounds,
									PaddingType,
									SMF_Streamed
								);
							}
							else
							{
								MeshBuildData.ShadowMap = NULL;
							}

							if (ALandscapeProxy* Proxy = Cast<ALandscapeProxy>(LandscapeComponent->GetOuter()))
							{
								TSet<ULandscapeComponent*> Components;
								Components.Add(LandscapeComponent);
								Proxy->FlushGrassComponents(&Components, false);
							}
						}

						FTileDataLayer::Evict();
					}
				}
			}

		}

		GCompressLightmaps = Settings->bCompressLightmaps;

		// Would be better to acquire this from the IStaticLightingSystem interface
		FStaticLightingBuildContext LightingContext (World, LightingScenario);

		FLightMap2D::EncodeTextures(&LightingContext, true, true);
		FShadowMap2D::EncodeTextures(&LightingContext, true, true);

		SlowTask.EnterProgressFrame(1, LOCTEXT("ApplyingNewLighting", "Applying new lighting"));

		for (int32 LevelIndex = 0; LevelIndex < World->GetNumLevels(); LevelIndex++)
		{
			bool bMarkLevelDirty = false;
			ULevel* Level = World->GetLevel(LevelIndex);

			if (Level)
			{
				if (Level->bIsVisible && (!Level->bIsLightingScenario || Level == LightingScenario))
				{
					ULevel* StorageLevel = LightingScenario ? LightingScenario : Level;
					UMapBuildDataRegistry* Registry = StorageLevel->GetOrCreateMapBuildData();

					Registry->SetupLightmapResourceClusters();

					Level->InitializeRenderingResources();
				}
			}
		}
	}

	// Always turn Realtime back on after baking lighting
	if (GCurrentLevelEditingViewportClient)
	{
		GCurrentLevelEditingViewportClient->SetRealtime(true);
	} 
}

void FScene::RemoveAllComponents()
{
	TArray<UStaticMeshComponent*> RegisteredStaticMeshComponents;
	TArray<UInstancedStaticMeshComponent*> RegisteredInstancedStaticMeshComponents;
	TArray<ULandscapeComponent*> RegisteredLandscapeComponents;
	RegisteredStaticMeshComponentUObjects.GetKeys(RegisteredStaticMeshComponents);
	RegisteredInstancedStaticMeshComponentUObjects.GetKeys(RegisteredInstancedStaticMeshComponents);
	RegisteredLandscapeComponentUObjects.GetKeys(RegisteredLandscapeComponents);

	for (auto Component : RegisteredStaticMeshComponents)
	{
		RemoveGeometryInstanceFromComponent(Component);
	}
	for (auto Component : RegisteredInstancedStaticMeshComponents)
	{
		RemoveGeometryInstanceFromComponent(Component);
	}
	for (auto Component : RegisteredLandscapeComponents)
	{
		RemoveGeometryInstanceFromComponent(Component);
	}

	TArray<UDirectionalLightComponent*> RegisteredDirectionalLightComponents;
	TArray<UPointLightComponent*> RegisteredPointLightComponents;
	TArray<USpotLightComponent*> RegisteredSpotLightComponents;
	TArray<URectLightComponent*> RegisteredRectLightComponents;
	LightScene.RegisteredDirectionalLightComponentUObjects.GetKeys(RegisteredDirectionalLightComponents);
	LightScene.RegisteredPointLightComponentUObjects.GetKeys(RegisteredPointLightComponents);
	LightScene.RegisteredSpotLightComponentUObjects.GetKeys(RegisteredSpotLightComponents);
	LightScene.RegisteredRectLightComponentUObjects.GetKeys(RegisteredRectLightComponents);

	for (auto Light : RegisteredDirectionalLightComponents)
	{
		RemoveLight(Light);
	}
	for (auto Light : RegisteredPointLightComponents)
	{
		RemoveLight(Light);
	}
	for (auto Light : RegisteredSpotLightComponents)
	{
		RemoveLight(Light);
	}
	for (auto Light : RegisteredRectLightComponents)
	{
		RemoveLight(Light);
	}

	if (LightScene.SkyLight.IsSet())
	{
		RemoveLight(LightScene.SkyLight->ComponentUObject);
	}
}

}


#undef LOCTEXT_NAMESPACE
