// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Lights.h"
#include "Scene/StaticMesh.h"
#include "Scene/InstancedStaticMesh.h"
#include "Scene/Landscape.h"
#include "MeshPassProcessor.h"
#include "RayTracingMeshDrawCommands.h"
#include "RayTracingGeometryInstance.h"
#include "RayTracing/RayTracingShaderBindingTable.h"
#include "IrradianceCaching.h"
#include "GPULightmassSettings.h"
#include "Templates/UniquePtr.h"
#include "SceneUniformBuffer.h"

class FGPULightmass;

namespace GPULightmass
{
class FLightmapRenderer;
class FVolumetricLightmapRenderer;

class FFullyCachedRayTracingMeshCommandContext : public FRayTracingMeshCommandContext
{
public:
	FFullyCachedRayTracingMeshCommandContext(
		TChunkedArray<FRayTracingMeshCommand>& CommandStorage,
		TArray<FRayTracingShaderBindingData>& InDirtyShaderBindingsStorage,
		const FRHIRayTracingGeometry* InRayTracingGeometry,
		uint32 InGeometrySegmentIndex,
		bool bInDynamicRayTracingGeometry,
		FRayTracingSBTAllocation* InSBTAllocation
	)
		: CommandStorage(CommandStorage)
		, DirtyShaderBindingsStorage(InDirtyShaderBindingsStorage)
		, RayTracingGeometry(InRayTracingGeometry)
		, GeometrySegmentIndex(InGeometrySegmentIndex)
		, bDynamicRayTracingGeometry(bInDynamicRayTracingGeometry)
		, SBTAllocation(InSBTAllocation) {}

	virtual FRayTracingMeshCommand& AddCommand(const FRayTracingMeshCommand& Initializer) override final
	{
		const int32 Index = CommandStorage.AddElement(Initializer);
		FRayTracingMeshCommand& NewCommand = CommandStorage[Index];
		NewCommand.GeometrySegmentIndex = GeometrySegmentIndex;
		return NewCommand;
	}

	virtual void FinalizeCommand(FRayTracingMeshCommand& RayTracingMeshCommand) override final 
	{
		check(GeometrySegmentIndex == RayTracingMeshCommand.GeometrySegmentIndex);
		bool bHidden = false;
		const uint32 RecordIndex = SBTAllocation->GetRecordIndex(ERayTracingShaderBindingLayer::Base, RayTracingMeshCommand.GeometrySegmentIndex);
		FRayTracingShaderBindingData DirtyShaderBinding(&RayTracingMeshCommand, RayTracingGeometry, bDynamicRayTracingGeometry, RecordIndex, ERayTracingLocalShaderBindingType::Transient, bHidden);
		DirtyShaderBindingsStorage.Add(DirtyShaderBinding);
		check(DirtyShaderBinding.RayTracingMeshCommand);
	}

private:
	TChunkedArray<FRayTracingMeshCommand>& CommandStorage;
	TArray<FRayTracingShaderBindingData>& DirtyShaderBindingsStorage;

	const FRHIRayTracingGeometry* RayTracingGeometry;
	uint32 GeometrySegmentIndex;
	bool bDynamicRayTracingGeometry;
	FRayTracingSBTAllocation* SBTAllocation;
};

struct FCachedRayTracingSceneData
{
	~FCachedRayTracingSceneData();

	FRayTracingShaderBindingTable RaytracingSBT;
	TArray<FRayTracingSBTAllocation*> StaticSBTAllocations;
	
	TArray<TArray<FRayTracingShaderBindingData>> ShaderBindingsPerLOD;
	TChunkedArray<FRayTracingMeshCommand> MeshCommandStorage;

	FBufferRHIRef InstanceIdsIdentityBufferRHI;
	FShaderResourceViewRHIRef InstanceIdsIdentityBufferSRV;
	TArray<uint32> InstanceDataOriginalOffsets;

	TArray<TArray<FRayTracingGeometryInstance>> RayTracingGeometryInstancesPerLOD;
	TArray<TUniquePtr<FMatrix>> OwnedRayTracingInstanceTransforms;

	TArray<uint32> RayTracingNumSegmentsPerLOD;

	TRefCountPtr<FRDGPooledBuffer> GPUScenePrimitiveDataBuffer;
	TRefCountPtr<FRDGPooledBuffer> GPUSceneLightmapDataBuffer;
	TRefCountPtr<FRDGPooledBuffer> GPUSceneInstanceDataBuffer;
	int							   GPUSceneInstanceDataSOAStride;
	int32						   GPUSceneNumInstances;
	TRefCountPtr<FRDGPooledBuffer> GPUSceneInstancePayloadDataBuffer;
	TRefCountPtr<FRDGPooledBuffer> GPUSceneLightDataBuffer;

	void SetupViewAndSceneUniformBufferFromSceneRenderState(FRDGBuilder& GraphBuilder, class FSceneRenderState& Scene, FSceneUniformBuffer& SceneUniforms);
	void SetupFromSceneRenderState(class FSceneRenderState& Scene);
	void RestoreCachedBuffers(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniforms);
};

class FSceneRenderState
{
public:
	UGPULightmassSettings* Settings;

	void RenderThreadInit(FRHICommandList& RHICmdList);
	void BackgroundTick(FRHICommandListImmediate& RHICmdList);

	FRayTracingSceneRHIRef RayTracingScene;
	FShaderResourceViewRHIRef RayTracingSceneSRV;
	FBufferRHIRef RayTracingSceneBuffer;
	FBufferRHIRef RayTracingScratchBuffer;

	FShaderBindingTableRHIRef SBT;

	FRayTracingPipelineState* RayTracingPipelineState;
	TSharedPtr<FViewInfo> ReferenceView;

	TUniquePtr<FCachedRayTracingSceneData> CachedRayTracingScene;

	TGeometryInstanceRenderStateCollection<FStaticMeshInstanceRenderState> StaticMeshInstanceRenderStates;
	TGeometryInstanceRenderStateCollection<FInstanceGroupRenderState> InstanceGroupRenderStates;
	TGeometryInstanceRenderStateCollection<FLandscapeRenderState> LandscapeRenderStates;

	TEntityArray<FLightmapRenderState> LightmapRenderStates;

	FLightSceneRenderState LightSceneRenderState;

	TUniquePtr<FLightmapRenderer> LightmapRenderer;
	TUniquePtr<FVolumetricLightmapRenderer> VolumetricLightmapRenderer;
	TUniquePtr<FIrradianceCache> IrradianceCache;

	FBox CombinedImportanceVolume;
	TArray<FBox> ImportanceVolumes;
	
	ERHIFeatureLevel::Type FeatureLevel;

	int32 GetPrimitiveIdForGPUScene(const FGeometryInstanceRenderStateRef& GeometryInstanceRef) const;

	bool SetupRayTracingScene(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniforms, int32 LODIndex = INDEX_NONE);
	void DestroyRayTracingScene();

	void CalculateDistributionPrefixSumForAllLightmaps();

	TArray<FLightmapRenderStateRef> MortonSortedLightmapRefList;
	void BuildMortonSortedLightmapRefList();

	volatile int32 Percentage = 0;
};

class FScene;

class FGeometryRange
{
public:
	FGeometryRange(FScene& Scene) : Scene(Scene) {}

	FGeometryIterator begin();
	FGeometryIterator end();

private:
	FScene& Scene;
};

class FScene
{
public:
	FScene(FGPULightmass* InGPULightmass);

	FGPULightmass* GPULightmass;
	UGPULightmassSettings* Settings;

	const FMeshMapBuildData* GetComponentLightmapData(const UPrimitiveComponent* InComponent, int32 LODIndex);
	const FLightComponentMapBuildData* GetComponentLightmapData(const ULightComponent* InComponent);

	void AddGeometryInstanceFromComponent(UStaticMeshComponent* InComponent);
	void RemoveGeometryInstanceFromComponent(UStaticMeshComponent* InComponent);

	void AddGeometryInstanceFromComponent(UInstancedStaticMeshComponent* InComponent);
	void RemoveGeometryInstanceFromComponent(UInstancedStaticMeshComponent* InComponent);

	void AddGeometryInstanceFromComponent(ULandscapeComponent* InComponent);
	void RemoveGeometryInstanceFromComponent(ULandscapeComponent* InComponent);

	void AddLight(USkyLightComponent* SkyLight);
	void RemoveLight(USkyLightComponent* SkyLight);

	template<typename LightComponentType>
	void AddLight(LightComponentType* Light);

	template<typename LightComponentType>
	void RemoveLight(LightComponentType* Light);

	template<typename LightComponentType>
	bool HasLight(LightComponentType* Light);

	void GatherImportanceVolumes();
	void OnSkyAtmosphereModified();
	void ConditionalTriggerSkyLightRecapture();
	
	void BackgroundTick();
	void AddRelevantStaticLightGUIDs(FQuantizedLightmapData* QuantizedLightmapData, const FBoxSphereBounds& WorldBounds);
	void ApplyFinishedLightmapsToWorld();
	void RemoveAllComponents();

	TGeometryArray<FStaticMeshInstance> StaticMeshInstances;
	TGeometryArray<FInstanceGroup> InstanceGroups;
	TGeometryArray<FLandscape> Landscapes;

	FGeometryRange Geometries;

	TEntityArray<FLightmap> Lightmaps;

	FLightScene LightScene;

	FSceneRenderState RenderState;

	ERHIFeatureLevel::Type FeatureLevel;

	bool bNeedsVoxelization = true;

private:
	TMap<UStaticMeshComponent*, FStaticMeshInstanceRef> RegisteredStaticMeshComponentUObjects;
	TMap<UInstancedStaticMeshComponent*, FInstanceGroupRef> RegisteredInstancedStaticMeshComponentUObjects;
	TMap<ULandscapeComponent*, FLandscapeRef> RegisteredLandscapeComponentUObjects;
};

}
