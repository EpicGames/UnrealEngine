// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/StaticMesh.h"
#include "Scene/Lights.h"
#include "Engine/StaticMesh.h"
#include "LightmapGBuffer.h"
#include "StaticMeshResources.h"

namespace GPULightmass
{

FStaticMeshInstance::FStaticMeshInstance(UStaticMeshComponent* ComponentUObject)
	: ComponentUObject(ComponentUObject)
{

}

const FMeshMapBuildData* FStaticMeshInstance::GetMeshMapBuildDataForLODIndex(int32 LODIndex)
{
	if (bLODsShareStaticLighting)
	{
		for (int32 Index = 0; Index < LODLightmaps.Num(); Index++)
		{
			if (LODLightmaps[Index].IsValid())
			{
				return LODLightmaps[Index]->MeshMapBuildData.Get();
			}
		}
	}

	return LODLightmaps[LODIndex].IsValid() ? LODLightmaps[LODIndex]->MeshMapBuildData.Get() : nullptr;
}

void FStaticMeshInstance::AllocateLightmaps(TEntityArray<FLightmap>& LightmapContainer)
{
	for (int32 LODIndex = 0; LODIndex < ComponentUObject->GetStaticMesh()->GetRenderData()->LODResources.Num(); LODIndex++)
	{
		FStaticMeshLODResources& LODModel = ComponentUObject->GetStaticMesh()->GetRenderData()->LODResources[LODIndex];

		int32 BaseLightMapWidth;
		int32 BaseLightMapHeight;
		ComponentUObject->GetLightMapResolution(BaseLightMapWidth, BaseLightMapHeight);

		bool bValidTextureMap = false;
		if (BaseLightMapWidth > 0
			&& BaseLightMapHeight > 0
			&& ComponentUObject->GetStaticMesh()->GetLightMapCoordinateIndex() >= 0
			&& (uint32)ComponentUObject->GetStaticMesh()->GetLightMapCoordinateIndex() < LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords())
		{
			bValidTextureMap = true;
		}

		if (bValidTextureMap && (!bLODsShareStaticLighting ? LODIndex >= ClampedMinLOD : LODIndex == ClampedMinLOD) && ComponentUObject->GetLightmapType() != ELightmapType::ForceVolumetric)
		{
			// Shrink LOD texture lightmaps by half for each LOD level
			const int32 LightMapWidth = LODIndex > 0 ? FMath::Max(BaseLightMapWidth / (2 << (LODIndex - 1)), 32) : BaseLightMapWidth;
			const int32 LightMapHeight = LODIndex > 0 ? FMath::Max(BaseLightMapHeight / (2 << (LODIndex - 1)), 32) : BaseLightMapHeight;
			
			FString LightmapName;
			if (ComponentUObject->GetStaticMesh()->GetRenderData()->LODResources.Num() > 1)
			{
				LightmapName = FString::Printf(
					TEXT("Lightmap_%s:LOD%d"),
					*(ComponentUObject->GetOwner() ? ComponentUObject->GetOwner()->GetActorLabel() : FString()),
					LODIndex);
			}
			else
			{
				LightmapName = FString::Printf(
					TEXT("Lightmap_%s"), *(ComponentUObject->GetOwner()
						                             ? ComponentUObject->GetOwner()->GetActorLabel()
						                             : FString()));
			}
			
			LODLightmaps.Add(LightmapContainer.Emplace(LightmapName, FIntPoint(LightMapWidth, LightMapHeight)));
		}
		else
		{
			LODLightmaps.Add(LightmapContainer.CreateNullRef());
		}
	}
}

TArray<FMeshBatch> FStaticMeshInstanceRenderState::GetMeshBatchesForGBufferRendering(int32 LODIndex)
{
	LODIndex = FMath::Max(LODIndex, ClampedMinLOD);

	TArray<FMeshBatch> MeshBatches;

	// TODO: potential race conditions between GT & RT everywhere in the following code
	FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];
	for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
	{
		const FStaticMeshSection& Section = LODModel.Sections[SectionIndex];
		if (Section.NumTriangles == 0)
		{
			continue;
		}

		FMeshBatch MeshBatch;

		MeshBatch.VertexFactory = &RenderData->LODVertexFactories[LODIndex].VertexFactory;

		FMeshBatchElement& MeshBatchElement = MeshBatch.Elements[0];
		if (LODOverrideColorVertexBuffers[LODIndex] != nullptr)
		{
			MeshBatch.VertexFactory = &RenderData->LODVertexFactories[LODIndex].VertexFactoryOverrideColorVertexBuffer;

			MeshBatchElement.VertexFactoryUserData = LODOverrideColorVFUniformBuffers[LODIndex].GetReference();
			MeshBatchElement.UserData = LODOverrideColorVertexBuffers[LODIndex];
			MeshBatchElement.bUserDataIsColorVertexBuffer = true;
		}
		else
		{
			MeshBatch.VertexFactory = &RenderData->LODVertexFactories[LODIndex].VertexFactory;

			MeshBatchElement.VertexFactoryUserData = ((FLocalVertexFactory*)MeshBatch.VertexFactory)->GetUniformBuffer();
		}

		MeshBatchElement.IndexBuffer = &LODModel.IndexBuffer;
		MeshBatchElement.FirstIndex = Section.FirstIndex;
		MeshBatchElement.NumPrimitives = Section.NumTriangles;
		MeshBatchElement.MinVertexIndex = Section.MinVertexIndex;
		MeshBatchElement.MaxVertexIndex = Section.MaxVertexIndex;
		MeshBatchElement.PrimitiveIdMode = PrimID_DynamicPrimitiveShaderData;

		MeshBatch.LODIndex = LODIndex;
		MeshBatch.SegmentIndex = SectionIndex;
		MeshBatch.CastShadow = bCastShadow && Section.bCastShadow;

		if (MeshBatchElement.NumPrimitives > 0)
		{
			if (ComponentUObject->GetMaterial(Section.MaterialIndex) != nullptr)
			{
				MeshBatch.MaterialRenderProxy = ComponentUObject->GetMaterial(Section.MaterialIndex)->GetRenderProxy();

				MeshBatches.Add(MeshBatch);
			}
		}
	}

	return MeshBatches;
}

template<>
TArray<FMeshBatch> TGeometryInstanceRenderStateCollection<FStaticMeshInstanceRenderState>::GetMeshBatchesForGBufferRendering(const FGeometryInstanceRenderStateRef& GeometryInstanceRef, FTileVirtualCoordinates CoordsForCulling)
{
	FStaticMeshInstanceRenderState& Instance = ResolveGeometryInstanceRef(GeometryInstanceRef);
	
	return Instance.GetMeshBatchesForGBufferRendering(GeometryInstanceRef.LODIndex);
}

}
