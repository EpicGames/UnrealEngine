// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightmapStorage.h"
#include "GeometryInterface.h"
#include "Components/StaticMeshComponent.h"
#include "LocalVertexFactory.h"

namespace GPULightmass
{

class FStaticMeshInstanceRenderState : public FGeometryRenderState
{
public:
	UStaticMeshComponent* ComponentUObject;
	FStaticMeshRenderData* RenderData;

	TArray<FColorVertexBuffer*> LODOverrideColorVertexBuffers;
	TArray<TUniformBufferRef<FLocalVertexFactoryUniformShaderParameters>> LODOverrideColorVFUniformBuffers;

	TArray<FMeshBatch> GetMeshBatchesForGBufferRendering(int32 LODIndex);

	int32 ClampedMinLOD;
};

using FStaticMeshInstanceRenderStateRef = TEntityArray<FStaticMeshInstanceRenderState>::EntityRefType;

class FStaticMeshInstance : public FGeometry
{
public:
	FStaticMeshInstance(UStaticMeshComponent* ComponentUObject);

	const class FMeshMapBuildData* GetMeshMapBuildDataForLODIndex(int32 LODIndex);

	void AllocateLightmaps(TEntityArray<FLightmap>& LightmapContainer);

	UStaticMeshComponent* ComponentUObject;

	virtual UPrimitiveComponent* GetComponentUObject() const override { return ComponentUObject; }

	int32 ClampedMinLOD;
};

using FStaticMeshInstanceRef = TEntityArray<FStaticMeshInstance>::EntityRefType;

}
