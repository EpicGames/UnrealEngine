// Copyright Epic Games, Inc. All Rights Reserved.

#include "VolumetricLightmap.h"
#include "VolumetricLightmapVoxelization.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "LightmapRayTracing.h"
#include "PathTracingLightParameters.inl"
#include "GPULightmassModule.h"
#include "RHIGPUReadback.h"
#include "LevelEditorViewport.h"
#include "Editor.h"
#include "RectLightTextureManager.h"
#include "RenderTargetPool.h"
#include "Engine/SubsurfaceProfile.h"
#include "IESTextureManager.h"
#include "GPUSort.h"

IMPLEMENT_MATERIAL_SHADER_TYPE(, FVLMVoxelizationVS, TEXT("/Plugin/GPULightmass/Private/VolumetricLightmapVoxelization.usf"), TEXT("VLMVoxelizationVS"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(, FVLMVoxelizationGS, TEXT("/Plugin/GPULightmass/Private/VolumetricLightmapVoxelization.usf"), TEXT("VLMVoxelizationGS"), SF_Geometry);
IMPLEMENT_MATERIAL_SHADER_TYPE(, FVLMVoxelizationPS, TEXT("/Plugin/GPULightmass/Private/VolumetricLightmapVoxelization.usf"), TEXT("VLMVoxelizationPS"), SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FClearVolumeCS, "/Plugin/GPULightmass/Private/BrickAllocationManagement.usf", "ClearVolumeCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FVoxelizeImportanceVolumeCS, "/Plugin/GPULightmass/Private/BrickAllocationManagement.usf", "VoxelizeImportanceVolumeCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FDownsampleVolumeCS, "/Plugin/GPULightmass/Private/BrickAllocationManagement.usf", "DownsampleVolumeCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FCountNumBricksCS, "/Plugin/GPULightmass/Private/BrickAllocationManagement.usf", "CountNumBricksCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FGatherBrickRequestsCS, "/Plugin/GPULightmass/Private/BrickAllocationManagement.usf", "GatherBrickRequestsCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FSplatVolumeCS, "/Plugin/GPULightmass/Private/BrickAllocationManagement.usf", "SplatVolumeCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FStitchBorderCS, "/Plugin/GPULightmass/Private/BrickAllocationManagement.usf", "StitchBorderCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FCopyPaddingStripsCS, "/Plugin/GPULightmass/Private/BrickAllocationManagement.usf", "CopyPaddingStripsCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FFillInvalidCellCS, "/Plugin/GPULightmass/Private/BrickAllocationManagement.usf", "FillInvalidCellCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FFinalizeBrickResultsCS, "/Plugin/GPULightmass/Private/BrickAllocationManagement.usf", "FinalizeBrickResultsCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FWriteSortedBrickRequestsCS, "/Plugin/GPULightmass/Private/BrickAllocationManagement.usf", "WriteSortedBrickRequestsCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FPermuteVoxelizeVolumeCS, "/Plugin/GPULightmass/Private/BrickAllocationManagement.usf", "PermuteVoxelizeVolumeCS", SF_Compute);

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FVLMVoxelizationParams, "VLMVoxelizationParams", SceneTextures);

void InitializeBrickData(FRHICommandListBase& RHICmdList, FIntVector BrickDataDimensions, FVolumetricLightmapBrickData& BrickData, const bool bForAccumulation)
{
	BrickData.AmbientVector.Format = bForAccumulation ? PF_A32B32G32R32F : PF_FloatR11G11B10;
	BrickData.SkyBentNormal.Format = bForAccumulation ? PF_A32B32G32R32F : PF_R8G8B8A8;
	BrickData.DirectionalLightShadowing.Format = PF_G8;

	for (int32 i = 0; i < UE_ARRAY_COUNT(BrickData.SHCoefficients); i++)
	{
		BrickData.SHCoefficients[i].Format = bForAccumulation ? PF_A32B32G32R32F : PF_R8G8B8A8;
	}

	BrickData.AmbientVector.CreateTargetTexture(RHICmdList, BrickDataDimensions);
	BrickData.AmbientVector.CreateUAV(RHICmdList);

	for (int32 i = 0; i < UE_ARRAY_COUNT(BrickData.SHCoefficients); i++)
	{
		BrickData.SHCoefficients[i].CreateTargetTexture(RHICmdList, BrickDataDimensions);
		BrickData.SHCoefficients[i].CreateUAV(RHICmdList);
	}

	BrickData.SkyBentNormal.CreateTargetTexture(RHICmdList, BrickDataDimensions);
	BrickData.SkyBentNormal.CreateUAV(RHICmdList);

	BrickData.DirectionalLightShadowing.CreateTargetTexture(RHICmdList, BrickDataDimensions);
	BrickData.DirectionalLightShadowing.CreateUAV(RHICmdList);

	size_t TotalSize = 0;
	TotalSize += GPixelFormats[BrickData.AmbientVector.Format].BlockBytes;
	TotalSize += UE_ARRAY_COUNT(BrickData.SHCoefficients) * GPixelFormats[BrickData.SHCoefficients[0].Format].BlockBytes;
	TotalSize += GPixelFormats[BrickData.SkyBentNormal.Format].BlockBytes;
	TotalSize += GPixelFormats[BrickData.DirectionalLightShadowing.Format].BlockBytes;
	TotalSize *= BrickDataDimensions.X * BrickDataDimensions.Y * BrickDataDimensions.Z;
	
	UE_LOGF(LogGPULightmass, Log, "Allocated %.2fMB for volumetric lightmap %ls brick data", TotalSize / 1024.0f / 1024.0f, bForAccumulation ? TEXT("accumulation") : TEXT("display"));
}

void ReleaseBrickData(FVolumetricLightmapBrickData& BrickData)
{
	BrickData.AmbientVector.Texture.SafeRelease();
	for (int32 i = 0; i < UE_ARRAY_COUNT(BrickData.SHCoefficients); i++)
	{
		BrickData.SHCoefficients[i].Texture.SafeRelease();
	}
	BrickData.SkyBentNormal.Texture.SafeRelease();
	BrickData.DirectionalLightShadowing.Texture.SafeRelease();

	BrickData.AmbientVector.UAV.SafeRelease();
	for (int32 i = 0; i < UE_ARRAY_COUNT(BrickData.SHCoefficients); i++)
	{
		BrickData.SHCoefficients[i].UAV.SafeRelease();
	}
	BrickData.SkyBentNormal.UAV.SafeRelease();
	BrickData.DirectionalLightShadowing.UAV.SafeRelease();
}

namespace GPULightmass
{

const int32 BrickSize = 4;
const int32 MaxRefinementLevels = 3;

FVolumetricLightmapRenderer::FVolumetricLightmapRenderer(FSceneRenderState* Scene)
	: Scene(Scene)
{
	VolumetricLightmap.Data = &VolumetricLightmapData;
	
	NumTotalPassesToRender = Scene->Settings->GISamples * Scene->Settings->VolumetricLightmapQualityMultiplier;
	
	if (Scene->Settings->bUseIrradianceCaching)
	{
		NumTotalPassesToRender += Scene->Settings->IrradianceCacheQuality;	
	}
}

FPrecomputedVolumetricLightmap* FVolumetricLightmapRenderer::GetPrecomputedVolumetricLightmapForPreview()
{
	return &VolumetricLightmap;
}

BEGIN_SHADER_PARAMETER_STRUCT(FVoxelizeMeshPassParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVLMVoxelizationParams, PassUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FInstanceCullingGlobalUniforms, InstanceCulling)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSortBrickRequestsPassParameters, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, BrickRequestKeysSRV)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, BrickRequestKeysSRV2)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, BrickRequestKeysUAV)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, BrickRequestKeysUAV2)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, BrickRequestSortedIndicesSRV)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, BrickRequestSortedIndicesSRV2)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, BrickRequestSortedIndicesUAV)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, BrickRequestSortedIndicesUAV2)
END_SHADER_PARAMETER_STRUCT()
	
void FVolumetricLightmapRenderer::VoxelizeScene(FRHICommandListImmediate& RHICmdList)
{
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(Scene->FeatureLevel);

	for (int32 MipLevel = 0; MipLevel < VoxelizationVolumeMips.Num(); MipLevel++)
	{
		VoxelizationVolumeMips[MipLevel].SafeRelease();
	}

	IndirectionTexture.SafeRelease();
	ValidityBrickData.SafeRelease();

	ReleaseBrickData(VolumetricLightmapData.BrickData);
	ReleaseBrickData(AccumulationBrickData);

	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

	VolumeMin = Scene->CombinedImportanceVolume.Min;

	FIntVector FullGridSize(
		FMath::TruncToInt(2 * Scene->CombinedImportanceVolume.GetExtent().X / TargetDetailCellSize) + 1,
		FMath::TruncToInt(2 * Scene->CombinedImportanceVolume.GetExtent().Y / TargetDetailCellSize) + 1,
		FMath::TruncToInt(2 * Scene->CombinedImportanceVolume.GetExtent().Z / TargetDetailCellSize) + 1);

	const int32 BrickSizeLog2 = FMath::FloorLog2(BrickSize);
	const int32 DetailCellsPerTopLevelBrick = 1 << (MaxRefinementLevels * BrickSizeLog2);

	FIntVector TopLevelGridSize = FIntVector::DivideAndRoundUp(FullGridSize, DetailCellsPerTopLevelBrick);

	VolumeSize = FVector(TopLevelGridSize) * DetailCellsPerTopLevelBrick * TargetDetailCellSize;
	FBox FinalVolume(VolumeMin, VolumeMin + VolumeSize);

	const int32 IndirectionCellsPerTopLevelCell = DetailCellsPerTopLevelBrick / BrickSize;

	IndirectionTextureDimensions = TopLevelGridSize * IndirectionCellsPerTopLevelCell;

	VoxelizationVolumeMips.Empty();

	{
		FRDGBuilder GraphBuilder(RHICmdList);
		ON_SCOPE_EXIT{ GraphBuilder.Execute(); };

		FRDGTextureUAV* IndirectionTextureUAV = nullptr;

		{
			const FRDGTextureDesc Desc = FRDGTextureDesc::Create3D(
				FIntVector(IndirectionTextureDimensions.X, IndirectionTextureDimensions.Y, IndirectionTextureDimensions.Z),
				PF_R8G8B8A8_UINT,
				FClearValueBinding::Black,
				ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV);

			FRDGTexture* Texture = GraphBuilder.CreateTexture(Desc, TEXT("GPULightmassVLMIndirectionTexture"));
			IndirectionTexture = GraphBuilder.ConvertToExternalTexture(Texture);
			IndirectionTextureUAV = GraphBuilder.CreateUAV(Texture);

			UE_LOGF(LogGPULightmass, Log, "Allocated %.2fMB for volumetric lightmap indirection texture",
				IndirectionTextureDimensions.X * IndirectionTextureDimensions.Y * IndirectionTextureDimensions.Z * GPixelFormats[Desc.Format].BlockBytes / 1024.0f / 1024.0f);
		}

		TArray<FRDGTextureUAV*, FRDGArrayAllocator> VoxelizationVolumeMipUAVs;

		for (int32 Level = 0; Level < MaxRefinementLevels; Level++)
		{
			const FRDGTextureDesc Desc = FRDGTextureDesc::Create3D(
				FIntVector(IndirectionTextureDimensions.X >> (Level * BrickSizeLog2), IndirectionTextureDimensions.Y >> (Level * BrickSizeLog2), IndirectionTextureDimensions.Z >> (Level * BrickSizeLog2)),
				PF_R32_UINT,
				FClearValueBinding::Black,
				ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV);

			FRDGTexture* Texture = GraphBuilder.CreateTexture(Desc, TEXT("GPULightmassVLMVoxelizationVolumeMips"));

			VoxelizationVolumeMips.Emplace(GraphBuilder.ConvertToExternalTexture(Texture));
			VoxelizationVolumeMipUAVs.Emplace(GraphBuilder.CreateUAV(Texture));
		}

		VolumetricLightmapData.Bounds = FinalVolume;
		VolumetricLightmapData.IndirectionTexture.Texture = IndirectionTexture->GetRHI();
		VolumetricLightmapData.IndirectionTexture.Format = PF_R8G8B8A8_UINT;
		VolumetricLightmapData.IndirectionTextureDimensions = FIntVector(IndirectionTextureDimensions);
		VolumetricLightmapData.BrickSize = 4;

		FBox CubeVolume(VolumeMin, VolumeMin + FVector(FMath::Max3(VolumeSize.X, VolumeSize.Y, VolumeSize.Z)));
		int32 CubeMaxDim = FMath::Max3(IndirectionTextureDimensions.X, IndirectionTextureDimensions.Y, IndirectionTextureDimensions.Z);

		FRDGTexture* VoxelizationVolumeMipsRDG = GraphBuilder.RegisterExternalTexture(VoxelizationVolumeMips[0]);
		FRDGTexture* IndirectTextureRDG = GraphBuilder.RegisterExternalTexture(IndirectionTexture);

		FVLMVoxelizationParams* VLMVoxelizationParams = GraphBuilder.AllocParameters<FVLMVoxelizationParams>();
		VLMVoxelizationParams->VolumeCenter = (FVector3f)CubeVolume.GetCenter(); // LWC_TODO: precision loss
		VLMVoxelizationParams->VolumeExtent = (FVector3f)CubeVolume.GetExtent(); // LWC_TODO: precision loss
		VLMVoxelizationParams->VolumeMaxDim = CubeMaxDim;
		VLMVoxelizationParams->VoxelizeVolume = VoxelizationVolumeMipUAVs[0];
		VLMVoxelizationParams->IndirectionTexture = IndirectionTextureUAV;
		TRDGUniformBufferRef<FVLMVoxelizationParams> PassUniformBuffer = GraphBuilder.CreateUniformBuffer(VLMVoxelizationParams);

		for (int32 MipLevel = 0; MipLevel < VoxelizationVolumeMips.Num(); MipLevel++)
		{
			FClearVolumeCS::FParameters* Parameters = GraphBuilder.AllocParameters<FClearVolumeCS::FParameters>();
			Parameters->VolumeSize = VoxelizationVolumeMips[MipLevel]->GetDesc().GetSize();
			Parameters->VoxelizeVolume = VoxelizationVolumeMipUAVs[MipLevel];

			TShaderMapRef<FClearVolumeCS> ComputeShader(GlobalShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("ClearVolume"),
				ComputeShader,
				Parameters,
				FComputeShaderUtils::GetGroupCount(VoxelizationVolumeMips[MipLevel]->GetDesc().GetSize(), FIntVector(4)));
		}

		for (FBox ImportanceVolume : Scene->ImportanceVolumes)
		{
			TShaderMapRef<FVoxelizeImportanceVolumeCS> ComputeShader(GlobalShaderMap);

			FVoxelizeImportanceVolumeCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVoxelizeImportanceVolumeCS::FParameters>();
			Parameters->VolumeSize = VoxelizationVolumeMips[0]->GetDesc().GetSize();
			Parameters->ImportanceVolumeMin = (FVector3f)ImportanceVolume.Min - FVector3f(2 * TargetDetailCellSize);
			Parameters->ImportanceVolumeMax = (FVector3f)ImportanceVolume.Max + FVector3f(2 * TargetDetailCellSize);
			Parameters->VLMVoxelizationParams = PassUniformBuffer;
			Parameters->VoxelizeVolume = VoxelizationVolumeMipUAVs[0];

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("VoxelizeImportanceVolume"),
				ComputeShader,
				Parameters,
				FComputeShaderUtils::GetGroupCount(VoxelizationVolumeMips[0]->GetDesc().GetSize(), FIntVector(4)));
		}

		// Setup ray tracing scene with LOD 0
		FSceneUniformBuffer SceneUniforms {};
		if (!Scene->SetupRayTracingScene(GraphBuilder, SceneUniforms))
		{
			return;
		}

		auto* PassParameters = GraphBuilder.AllocParameters<FVoxelizeMeshPassParameters>();
		PassParameters->View = Scene->ReferenceView->ViewUniformBuffer;
		PassParameters->Scene = SceneUniforms.GetBuffer(GraphBuilder);
		PassParameters->PassUniformBuffer = GraphBuilder.CreateUniformBuffer(VLMVoxelizationParams);
		PassParameters->InstanceCulling = FInstanceCullingContext::CreateDummyInstanceCullingUniformBuffer(GraphBuilder);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("VLM Mesh Voxelization"),
			PassParameters,
			ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
			[this, CubeMaxDim, PassUniformBuffer, PassParameters](FRHICommandList& RHICmdList)
		{
			// Must match VolumetricLightmapVoxelization.usf
			const int32 SUPER_RESOLUTION_FACTOR = 4;
			
			RHICmdList.SetViewport(0, 0, 0, CubeMaxDim * SUPER_RESOLUTION_FACTOR, CubeMaxDim * SUPER_RESOLUTION_FACTOR, 1);

			SCOPED_DRAW_EVENTF(RHICmdList, GPULightmassVoxelizeScene, TEXT("GPULightmass VoxelizeScene"));

			DrawDynamicMeshPass(
				*Scene->ReferenceView,
				RHICmdList,
				[&Scene = Scene, View = Scene->ReferenceView.Get(), ImportanceVolumes = Scene->ImportanceVolumes](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FVLMVoxelizationMeshProcessor MeshProcessor(nullptr, View, DynamicMeshPassContext);

				for (int32 InstanceIndex = 0; InstanceIndex < Scene->StaticMeshInstanceRenderStates.Elements.Num(); InstanceIndex++)
				{
					FStaticMeshInstanceRenderState& Instance = Scene->StaticMeshInstanceRenderStates.Elements[InstanceIndex];

					bool bIntersectsAnyImportanceVolume = false;

					for (FBox ImportanceVolume : ImportanceVolumes)
					{
						if (Instance.WorldBounds.GetBox().Intersect(ImportanceVolume))
						{
							bIntersectsAnyImportanceVolume = true;
							break;
						}
					}

					if (!bIntersectsAnyImportanceVolume) continue;

					TArray<FMeshBatch> MeshBatches = Instance.GetMeshBatchesForGBufferRendering(0);

					for (auto& MeshBatch : MeshBatches)
					{
						MeshBatch.Elements[0].DynamicPrimitiveIndex = InstanceIndex;
						MeshProcessor.AddMeshBatch(MeshBatch, ~0ull, nullptr);
					};
				}

				for (int32 InstanceGroupIndex = 0; InstanceGroupIndex < Scene->InstanceGroupRenderStates.Elements.Num(); InstanceGroupIndex++)
				{
					FInstanceGroupRenderState& InstanceGroup = Scene->InstanceGroupRenderStates.Elements[InstanceGroupIndex];

					bool bIntersectsAnyImportanceVolume = false;

					for (FBox ImportanceVolume : ImportanceVolumes)
					{
						if (InstanceGroup.WorldBounds.GetBox().Intersect(ImportanceVolume))
						{
							bIntersectsAnyImportanceVolume = true;
							break;
						}
					}

					if (!bIntersectsAnyImportanceVolume) continue;

					TArray<FMeshBatch> MeshBatches = InstanceGroup.GetMeshBatchesForGBufferRendering(0, FTileVirtualCoordinates{});

					for (auto& MeshBatch : MeshBatches)
					{
						MeshBatch.Elements[0].DynamicPrimitiveIndex = Scene->StaticMeshInstanceRenderStates.Elements.Num() + InstanceGroupIndex;
						MeshProcessor.AddMeshBatch(MeshBatch, ~0ull, nullptr);
					};
				}

				for (int32 LandscapeIndex = 0; LandscapeIndex < Scene->LandscapeRenderStates.Elements.Num(); LandscapeIndex++)
				{
					FLandscapeRenderState& Landscape = Scene->LandscapeRenderStates.Elements[LandscapeIndex];

					bool bIntersectsAnyImportanceVolume = false;

					for (FBox ImportanceVolume : ImportanceVolumes)
					{
						if (Landscape.WorldBounds.GetBox().Intersect(ImportanceVolume))
						{
							bIntersectsAnyImportanceVolume = true;
							break;
						}
					}

					if (!bIntersectsAnyImportanceVolume) continue;

					TArray<FMeshBatch> MeshBatches = Landscape.GetMeshBatchesForGBufferRendering(0);

					for (auto& MeshBatch : MeshBatches)
					{
						MeshBatch.Elements[0].DynamicPrimitiveIndex = Scene->StaticMeshInstanceRenderStates.Elements.Num() + Scene->InstanceGroupRenderStates.Elements.Num() + LandscapeIndex;
						MeshProcessor.AddMeshBatch(MeshBatch, ~0ull, nullptr);
					};
				}
			});
		});

		for (int32 MipLevel = 1; MipLevel < VoxelizationVolumeMips.Num(); MipLevel++)
		{
			TShaderMapRef<FDownsampleVolumeCS> ComputeShader(GlobalShaderMap);

			FDownsampleVolumeCS::FParameters* Parameters = GraphBuilder.AllocParameters<FDownsampleVolumeCS::FParameters>();
			Parameters->bIsHighestMip = (MipLevel == VoxelizationVolumeMips.Num() - 1) ? 1 : 0;
			Parameters->VoxelizeVolume = VoxelizationVolumeMipUAVs[MipLevel];
			Parameters->VoxelizeVolumePrevMip = VoxelizationVolumeMipUAVs[MipLevel - 1];

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("DownsampleVolume"),
				ComputeShader,
				Parameters,
				VoxelizationVolumeMips[MipLevel]->GetDesc().GetSize());
		}

		{
			TResourceArray<int32> InitialBrickAllocatorParams;
			InitialBrickAllocatorParams.Add(0);
			InitialBrickAllocatorParams.Add(0);
			BrickAllocatorParameters.Initialize(RHICmdList, TEXT("VolumetricLightmapBrickAllocatorParameters"), 4, 2, PF_R32_SINT, BUF_UnorderedAccess | BUF_SourceCopy, &InitialBrickAllocatorParams);

			RHICmdList.Transition(FRHITransitionInfo(BrickAllocatorParameters.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
		}

		for (int32 MipLevel = VoxelizationVolumeMips.Num() - 1; MipLevel >= 0; MipLevel--)
		{
			TShaderMapRef<FCountNumBricksCS> ComputeShader(GlobalShaderMap);

			FCountNumBricksCS::FParameters* Parameters = GraphBuilder.AllocParameters<FCountNumBricksCS::FParameters>();
			Parameters->VolumeSize = VoxelizationVolumeMips[MipLevel]->GetDesc().GetSize();
			Parameters->VoxelizeVolume = VoxelizationVolumeMipUAVs[MipLevel];
			Parameters->BrickAllocatorParameters = BrickAllocatorParameters.UAV;

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("CountNumBricks"),
				ComputeShader,
				Parameters,
				FComputeShaderUtils::GetGroupCount(VoxelizationVolumeMips[MipLevel]->GetDesc().GetSize(), FIntVector(4)));
		}
	}

	{
		FRHIGPUBufferReadback NumBricksReadback(TEXT("NumBricksReadback"));
		NumBricksReadback.EnqueueCopy(RHICmdList, BrickAllocatorParameters.Buffer);
		RHICmdList.SubmitAndBlockUntilGPUIdle();
		check(NumBricksReadback.IsReady());

		int32* Buffer = (int32*)NumBricksReadback.Lock(8);
		NumTotalBricks = Buffer[0];
		UE_LOGF(LogGPULightmass, Log, "Volumetric lightmap NumTotalBricks = %d", NumTotalBricks);
		NumBricksReadback.Unlock();
	}

	if (NumTotalBricks == 0)
	{
		return;
	}

	int32 MaxBricksInLayoutOneDim = 256;

	FIntVector BrickLayoutDimensions;

	{
		int32 BrickTextureLinearAllocator = NumTotalBricks;
		BrickLayoutDimensions.X = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
		BrickTextureLinearAllocator = FMath::DivideAndRoundUp(BrickTextureLinearAllocator, BrickLayoutDimensions.X);
		BrickLayoutDimensions.Y = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
		BrickTextureLinearAllocator = FMath::DivideAndRoundUp(BrickTextureLinearAllocator, BrickLayoutDimensions.Y);
		BrickLayoutDimensions.Z = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
	}

	InitializeBrickData(RHICmdList, BrickLayoutDimensions * 5, VolumetricLightmapData.BrickData, false);

	FIntVector BrickLayoutDimensionsForAccumulation;

	{
		int32 BrickTextureLinearAllocator = BrickBatchSize;
		BrickLayoutDimensionsForAccumulation.X = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
		BrickTextureLinearAllocator = FMath::DivideAndRoundUp(BrickTextureLinearAllocator, BrickLayoutDimensionsForAccumulation.X);
		BrickLayoutDimensionsForAccumulation.Y = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
		BrickTextureLinearAllocator = FMath::DivideAndRoundUp(BrickTextureLinearAllocator, BrickLayoutDimensionsForAccumulation.Y);
		BrickLayoutDimensionsForAccumulation.Z = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
	}
	
	InitializeBrickData(RHICmdList, BrickLayoutDimensionsForAccumulation * 5, AccumulationBrickData, true);
	
	BrickRequests.Initialize(RHICmdList, TEXT("BrickRequests"), 16, NumTotalBricks, PF_R32G32B32A32_UINT, BUF_UnorderedAccess);

	VolumetricLightmapData.BrickDataDimensions = BrickLayoutDimensions * 5;

	{
		FRDGBuilder GraphBuilder(RHICmdList);

		FRDGTextureRef ValidityMask = GraphBuilder.CreateTexture(FRDGTextureDesc::Create3D(VolumetricLightmapData.BrickDataDimensions,
															   PF_R8_UINT,
															   FClearValueBinding::Black,
															   ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV), TEXT("ValidityBrickData"));
		ValidityBrickData = GraphBuilder.ConvertToExternalTexture(ValidityMask);

		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(ValidityMask), 0u);

		UE_LOGF(LogGPULightmass, Log, "Allocated %.2fMB for volumetric lightmap validity mask",
			VolumetricLightmapData.BrickDataDimensions.X * VolumetricLightmapData.BrickDataDimensions.Y * VolumetricLightmapData.BrickDataDimensions.Z * GPixelFormats[PF_R8_UINT].BlockBytes / 1024.0f / 1024.0f);
		
		FRDGTextureUAV* IndirectionTextureUAV = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(IndirectionTexture));
		FRDGBufferRef BrickRequestsUnsorted = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(FIntVector4), NumTotalBricks), TEXT("BrickRequestsUnsorted"));
		FRDGBufferRef BrickRequestKeys = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), NumTotalBricks), TEXT("BrickRequestKeys"));
		FRDGBufferRef BrickRequestSortedIndices = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), NumTotalBricks), TEXT("BrickRequestSortedIndices"));
		FRDGBufferRef BrickRequestKeys2 = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), NumTotalBricks), TEXT("BrickRequestKeys2"));
		FRDGBufferRef BrickRequestSortedIndices2 = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), NumTotalBricks), TEXT("BrickRequestSortedIndices2"));
		FRDGBufferRef BrickRequestSortedIndicesInverse = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), NumTotalBricks), TEXT("BrickRequestSortedIndices2"));

		TArray<FRDGTextureUAV*, FRDGArrayAllocator> VoxelizationVolumeMipUAVs;

		for (const TRefCountPtr<IPooledRenderTarget>& Mip : VoxelizationVolumeMips)
		{
			VoxelizationVolumeMipUAVs.Emplace(GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(Mip)));
		}

		for (int32 MipLevel = VoxelizationVolumeMips.Num() - 1; MipLevel >= 0; MipLevel--)
		{
			TShaderMapRef<FGatherBrickRequestsCS> ComputeShader(GlobalShaderMap);

			FGatherBrickRequestsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGatherBrickRequestsCS::FParameters>();
			PassParameters->VolumeSize = VoxelizationVolumeMips[MipLevel]->GetDesc().GetSize();
			PassParameters->BrickSize = 1 << (MipLevel * BrickSizeLog2);
			PassParameters->MipLevel = MipLevel;
			PassParameters->VoxelizeVolume = VoxelizationVolumeMipUAVs[MipLevel];
			PassParameters->BrickAllocatorParameters = BrickAllocatorParameters.UAV;
			PassParameters->BrickRequestsUnsorted = GraphBuilder.CreateUAV(BrickRequestsUnsorted, PF_R32G32B32A32_UINT);
			PassParameters->BrickRequestKeys = GraphBuilder.CreateUAV(BrickRequestKeys, PF_R32_UINT);
			PassParameters->BrickRequestSortedIndices = GraphBuilder.CreateUAV(BrickRequestSortedIndices, PF_R32_UINT);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("GatherBrickRequests"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(VoxelizationVolumeMips[MipLevel]->GetDesc().GetSize(), FIntVector(4)));
		}
		
		{
			FSortBrickRequestsPassParameters* PassParameters = GraphBuilder.AllocParameters<FSortBrickRequestsPassParameters>();
			PassParameters->BrickRequestKeysSRV = GraphBuilder.CreateSRV(BrickRequestKeys, PF_R32_UINT);
			PassParameters->BrickRequestKeysSRV2 = GraphBuilder.CreateSRV(BrickRequestKeys2, PF_R32_UINT);
			PassParameters->BrickRequestKeysUAV = GraphBuilder.CreateUAV(BrickRequestKeys, PF_R32_UINT);
			PassParameters->BrickRequestKeysUAV2 = GraphBuilder.CreateUAV(BrickRequestKeys2, PF_R32_UINT);
			PassParameters->BrickRequestSortedIndicesSRV = GraphBuilder.CreateSRV(BrickRequestSortedIndices, PF_R32_UINT);
			PassParameters->BrickRequestSortedIndicesSRV2 = GraphBuilder.CreateSRV(BrickRequestSortedIndices2, PF_R32_UINT);
			PassParameters->BrickRequestSortedIndicesUAV = GraphBuilder.CreateUAV(BrickRequestSortedIndices, PF_R32_UINT);
			PassParameters->BrickRequestSortedIndicesUAV2 = GraphBuilder.CreateUAV(BrickRequestSortedIndices2, PF_R32_UINT);

			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(BrickRequestKeys2, PF_R32_UINT), 0);
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(BrickRequestSortedIndices2, PF_R32_UINT), 0);
			
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("SortBrickRequests"),
				PassParameters,
				ERDGPassFlags::Compute,
				[
					PassParameters,
					NumTotalBricks = NumTotalBricks
				](FRHICommandList& RHICmdList)
				{
					FGPUSortBuffers SortBuffers;
					SortBuffers.RemoteKeySRVs[0] = PassParameters->BrickRequestKeysSRV->GetRHI();
					SortBuffers.RemoteKeySRVs[1] = PassParameters->BrickRequestKeysSRV2->GetRHI();
					SortBuffers.RemoteKeyUAVs[0] = PassParameters->BrickRequestKeysUAV->GetRHI();
					SortBuffers.RemoteKeyUAVs[1] = PassParameters->BrickRequestKeysUAV2->GetRHI();
					SortBuffers.RemoteValueSRVs[0] = PassParameters->BrickRequestSortedIndicesSRV->GetRHI();
					SortBuffers.RemoteValueSRVs[1] = PassParameters->BrickRequestSortedIndicesSRV2->GetRHI();
					SortBuffers.RemoteValueUAVs[0] = PassParameters->BrickRequestSortedIndicesUAV->GetRHI();
					SortBuffers.RemoteValueUAVs[1] = PassParameters->BrickRequestSortedIndicesUAV2->GetRHI();

					int32 ResultBufferIndex = SortGPUBuffers(RHICmdList, SortBuffers, 0, 0xFFFFFFFF, NumTotalBricks, GMaxRHIFeatureLevel);
					check(ResultBufferIndex == 0);
				}
			);
		}

		{
			FWriteSortedBrickRequestsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWriteSortedBrickRequestsCS::FParameters>();
			PassParameters->NumTotalBricks = NumTotalBricks;
			PassParameters->BrickRequests = BrickRequests.UAV;
			PassParameters->BrickRequestsUnsorted = GraphBuilder.CreateUAV(BrickRequestsUnsorted, PF_R32G32B32A32_UINT);
			PassParameters->BrickRequestSortedIndices = GraphBuilder.CreateUAV(BrickRequestSortedIndices, PF_R32_UINT);
			PassParameters->BrickRequestSortedIndicesInverse = GraphBuilder.CreateUAV(BrickRequestSortedIndicesInverse, PF_R32_UINT);

			TShaderMapRef<FWriteSortedBrickRequestsCS> ComputeShader(GlobalShaderMap);
			
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("WriteSortedBrickRequests"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(NumTotalBricks, 64));
		}

		for (int32 MipLevel = VoxelizationVolumeMips.Num() - 1; MipLevel >= 0; MipLevel--)
		{
			TShaderMapRef<FPermuteVoxelizeVolumeCS> ComputeShader(GlobalShaderMap);

			FPermuteVoxelizeVolumeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPermuteVoxelizeVolumeCS::FParameters>();
			PassParameters->VolumeSize = VoxelizationVolumeMips[MipLevel]->GetDesc().GetSize();
			PassParameters->VoxelizeVolume = VoxelizationVolumeMipUAVs[MipLevel];
			PassParameters->BrickRequestSortedIndicesInverse = GraphBuilder.CreateUAV(BrickRequestSortedIndicesInverse, PF_R32_UINT);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("PermuteVoxelizeVolume"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(VoxelizationVolumeMips[MipLevel]->GetDesc().GetSize(), FIntVector(4)));
		}
		
		for (int32 MipLevel = VoxelizationVolumeMips.Num() - 1; MipLevel >= 0; MipLevel--)
		{
			TShaderMapRef<FSplatVolumeCS> ComputeShader(GlobalShaderMap);

			FSplatVolumeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSplatVolumeCS::FParameters>();
			PassParameters->VolumeSize = IndirectionTextureDimensions;
			PassParameters->BrickSize = 1 << (MipLevel * BrickSizeLog2);
			PassParameters->bIsHighestMip = MipLevel == VoxelizationVolumeMips.Num() - 1;
			PassParameters->VoxelizeVolume = VoxelizationVolumeMipUAVs[MipLevel];
			PassParameters->IndirectionTexture = IndirectionTextureUAV;
			PassParameters->BrickAllocatorParameters = BrickAllocatorParameters.UAV;

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SplatVolume"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(IndirectionTextureDimensions, FIntVector(4)));
		}

		GraphBuilder.Execute();
	}

	Scene->DestroyRayTracingScene();
}

struct FVolumetricLightmapBrickDataRDG
{
	FRDGTextureUAVRef AmbientVector;
	FRDGTextureUAVRef SHCoefficients[6];
	FRDGTextureUAVRef SkyBentNormal;
	FRDGTextureUAVRef DirectionalLightShadowing;

	TRefCountPtr<IPooledRenderTarget> AmbientVectorWrapperRT;
	TRefCountPtr<IPooledRenderTarget> SHCoefficientsWrapperRT[6];
	TRefCountPtr<IPooledRenderTarget> SkyBentNormalWrapperRT;
	TRefCountPtr<IPooledRenderTarget> DirectionalLightShadowingWrapperRT;

	FRDGTextureRef AmbientVectorRDG;
	FRDGTextureRef SHCoefficientsRDG[6];
	FRDGTextureRef SkyBentNormalRDG;
	FRDGTextureRef DirectionalLightShadowingRDG;

	TRefCountPtr<IPooledRenderTarget> CreateWrapperRenderTargetForLayer(FVolumetricLightmapDataLayer& Layer, FIntVector Dimensions)
	{
		return TRefCountPtr<IPooledRenderTarget>(new FPooledRenderTarget(
				Layer.Texture,
				FPooledRenderTargetDesc::CreateVolumeDesc(
					Dimensions.X, Dimensions.Y, Dimensions.Z,
					Layer.Format,
					FClearValueBinding::None,
					ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV,
					ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV,
					false),
				nullptr
			)
		);
	}
	
	void CreateFromBrickData(FRDGBuilder& GraphBuilder, FVolumetricLightmapBrickData& BrickData, FIntVector Dimensions)
	{
		AmbientVectorWrapperRT = CreateWrapperRenderTargetForLayer(BrickData.AmbientVector, Dimensions);
		AmbientVectorRDG = GraphBuilder.RegisterExternalTexture(AmbientVectorWrapperRT);
		AmbientVector = GraphBuilder.CreateUAV(AmbientVectorRDG);

		for (int32 CoefficientIndex = 0; CoefficientIndex < UE_ARRAY_COUNT(SHCoefficients); CoefficientIndex++)
		{
			SHCoefficientsWrapperRT[CoefficientIndex] = CreateWrapperRenderTargetForLayer(BrickData.SHCoefficients[CoefficientIndex], Dimensions);
			SHCoefficientsRDG[CoefficientIndex] = GraphBuilder.RegisterExternalTexture(SHCoefficientsWrapperRT[CoefficientIndex]);
			SHCoefficients[CoefficientIndex] = GraphBuilder.CreateUAV(SHCoefficientsRDG[CoefficientIndex]);
		}

		SkyBentNormalWrapperRT = CreateWrapperRenderTargetForLayer(BrickData.SkyBentNormal, Dimensions);
		SkyBentNormalRDG = GraphBuilder.RegisterExternalTexture(SkyBentNormalWrapperRT);
		SkyBentNormal = GraphBuilder.CreateUAV(SkyBentNormalRDG);
		
		DirectionalLightShadowingWrapperRT = CreateWrapperRenderTargetForLayer(BrickData.DirectionalLightShadowing, Dimensions);
		DirectionalLightShadowingRDG = GraphBuilder.RegisterExternalTexture(DirectionalLightShadowingWrapperRT);
		DirectionalLightShadowing = GraphBuilder.CreateUAV(DirectionalLightShadowingRDG);		
	}

	void QueueTextureExtractions(FRDGBuilder& GraphBuilder)
	{
		GraphBuilder.QueueTextureExtraction(AmbientVectorRDG, &AmbientVectorWrapperRT);

		for (int32 CoefficientIndex = 0; CoefficientIndex < UE_ARRAY_COUNT(SHCoefficients); CoefficientIndex++)
		{
			GraphBuilder.QueueTextureExtraction(SHCoefficientsRDG[CoefficientIndex], &SHCoefficientsWrapperRT[CoefficientIndex]);
		}

		GraphBuilder.QueueTextureExtraction(SkyBentNormalRDG, &SkyBentNormalWrapperRT);		
		GraphBuilder.QueueTextureExtraction(DirectionalLightShadowingRDG, &DirectionalLightShadowingWrapperRT);
	}

	TRefCountPtr<IPooledRenderTarget> OldAmbientVectorWrapperRT;
	TRefCountPtr<IPooledRenderTarget> OldSHCoefficientsWrapperRT[6];
	TRefCountPtr<IPooledRenderTarget> OldSkyBentNormalWrapperRT;
	TRefCountPtr<IPooledRenderTarget> OldDirectionalLightShadowingWrapperRT;

	void DebugSetupInvariants()
	{
		OldAmbientVectorWrapperRT = AmbientVectorWrapperRT;
		for (int32 CoefficientIndex = 0; CoefficientIndex < UE_ARRAY_COUNT(SHCoefficients); CoefficientIndex++)
		{
			OldSHCoefficientsWrapperRT[CoefficientIndex] = SHCoefficientsWrapperRT[CoefficientIndex];
		}
		OldSkyBentNormalWrapperRT = SkyBentNormalWrapperRT;
		OldDirectionalLightShadowingWrapperRT = DirectionalLightShadowingWrapperRT;
	}

	void DebugCheckInvariants()
	{
		checkf(OldAmbientVectorWrapperRT == AmbientVectorWrapperRT,
			TEXT("The code relies on the fact that the texture extracted from RDG is exactly the same as the one registered"));
		for (int32 CoefficientIndex = 0; CoefficientIndex < UE_ARRAY_COUNT(SHCoefficients); CoefficientIndex++)
		{
			check(OldSHCoefficientsWrapperRT[CoefficientIndex] == SHCoefficientsWrapperRT[CoefficientIndex]);
		}
		check(OldSkyBentNormalWrapperRT == SkyBentNormalWrapperRT);
		check(OldDirectionalLightShadowingWrapperRT == DirectionalLightShadowingWrapperRT);
	}
};
	
void FVolumetricLightmapRenderer::BackgroundTick(FRHICommandListImmediate& RHICmdList)
{
	if (NumTotalBricks == 0)
	{
		return;
	}

	if (SamplesTaken >= (uint64)NumTotalBricks * NumTotalPassesToRender)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FVolumetricLightmapRenderer::BackgroundTick);

	FRDGBuilder GraphBuilder(RHICmdList);
	FSceneUniformBuffer SceneUniforms {};
	
	if (IsRayTracingEnabled())
	{
		if (!Scene->SetupRayTracingScene(GraphBuilder, SceneUniforms))
		{
			return;
		}
	}

	FVolumetricLightmapBrickDataRDG AccumulationBrickDataRDG;
	AccumulationBrickDataRDG.CreateFromBrickData(GraphBuilder, AccumulationBrickData, VolumetricLightmapData.BrickDataDimensions);
	AccumulationBrickDataRDG.DebugSetupInvariants();
		
	FVolumetricLightmapBrickDataRDG OutputBrickDataRDG;
	OutputBrickDataRDG.CreateFromBrickData(GraphBuilder, VolumetricLightmapData.BrickData, VolumetricLightmapData.BrickDataDimensions);	
	OutputBrickDataRDG.DebugSetupInvariants();
	
	RectLightAtlas::UpdateAtlasTexture(GraphBuilder, Scene->FeatureLevel);
	IESAtlas::UpdateAtlasTexture(GraphBuilder, GetFeatureLevelShaderPlatform(Scene->FeatureLevel));

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(Scene->FeatureLevel);

	const bool bIsViewportNonRealtime = !FGPULightmassModule::IsRealtimeOn();
	const int32 NumSamplesPerFrame = bIsViewportNonRealtime ? Scene->Settings->TilePassesInFullSpeedMode : Scene->Settings->TilePassesInSlowMode;

	FVolumetricLightmapPathTracingRGS::FParameters* PreviousPassParameters = nullptr;
	
	for (int32 SampleIndex = 0; SampleIndex < NumSamplesPerFrame; SampleIndex++)
	{
		const uint64 NumTotalSamples = (uint64)NumTotalBricks * NumTotalPassesToRender;
		const int32 BrickBatchIndexToWorkOn = SamplesTaken / (NumTotalPassesToRender * BrickBatchSize);
		const int32 BrickBatchOffset = BrickBatchIndexToWorkOn * BrickBatchSize;
		const int32 BricksToCalcThisFrame = FMath::Min(BrickBatchSize, NumTotalBricks - BrickBatchOffset);
		const int32 RenderPassIndex = (SamplesTaken - BrickBatchOffset * NumTotalPassesToRender) / BricksToCalcThisFrame;
		if (BricksToCalcThisFrame <= 0) continue;

#if RHI_RAYTRACING
		if (IsRayTracingEnabled())
		{
			FVolumetricLightmapPathTracingRGS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FVolumetricLightmapPathTracingRGS::FUseIrradianceCaching>(Scene->Settings->bUseIrradianceCaching);
			TShaderRef<FVolumetricLightmapPathTracingRGS> RayGenShader = GlobalShaderMap->GetShader<FVolumetricLightmapPathTracingRGS>(PermutationVector);

			FVolumetricLightmapPathTracingRGS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVolumetricLightmapPathTracingRGS::FParameters>();
			CA_ASSUME(PassParameters);

			PassParameters->FrameNumber = RenderPassIndex;
			PassParameters->VolumeMin = (FVector3f)VolumeMin; // LWC_TODO: precision loss
			PassParameters->VolumeSize = (FVector3f)VolumeSize; // LWC_TODO: precision loss
			PassParameters->IndirectionTextureDim = IndirectionTextureDimensions;
			PassParameters->TLAS = Scene->RayTracingSceneSRV;
			PassParameters->BrickRequests = BrickRequests.SRV;
			PassParameters->NumTotalBricks = NumTotalBricks;
			PassParameters->BrickBatchOffset = BrickBatchOffset;
			PassParameters->AmbientVector = AccumulationBrickDataRDG.AmbientVector;
			PassParameters->SHCoefficients0R = AccumulationBrickDataRDG.SHCoefficients[0];
			PassParameters->SHCoefficients1R = AccumulationBrickDataRDG.SHCoefficients[1];
			PassParameters->SHCoefficients0G = AccumulationBrickDataRDG.SHCoefficients[2];
			PassParameters->SHCoefficients1G = AccumulationBrickDataRDG.SHCoefficients[3];
			PassParameters->SHCoefficients0B = AccumulationBrickDataRDG.SHCoefficients[4];
			PassParameters->SHCoefficients1B = AccumulationBrickDataRDG.SHCoefficients[5];
			PassParameters->SkyBentNormal = AccumulationBrickDataRDG.SkyBentNormal;
			PassParameters->DirectionalLightShadowing = AccumulationBrickDataRDG.DirectionalLightShadowing;
			PassParameters->ViewUniformBuffer = Scene->ReferenceView->ViewUniformBuffer;
			PassParameters->IrradianceCachingParameters = Scene->IrradianceCache->IrradianceCachingParametersUniformBuffer;

			if (PreviousPassParameters == nullptr)
			{
				SetupPathTracingLightParameters(Scene->LightSceneRenderState, GraphBuilder, *Scene->ReferenceView, PassParameters);
				PreviousPassParameters = PassParameters;
			}
			else
			{
				PassParameters->LightGridParameters = PreviousPassParameters->LightGridParameters;
				PassParameters->SceneLightCount = PreviousPassParameters->SceneLightCount;
				PassParameters->SceneVisibleLightCount = PreviousPassParameters->SceneVisibleLightCount;
				PassParameters->SceneLights = PreviousPassParameters->SceneLights;
				PassParameters->SkylightTexture = PreviousPassParameters->SkylightTexture;
				PassParameters->SkylightTextureSampler = PreviousPassParameters->SkylightTextureSampler;
				PassParameters->SkylightPdf = PreviousPassParameters->SkylightPdf;
				PassParameters->SkylightInvResolution = PreviousPassParameters->SkylightInvResolution;
				PassParameters->SkylightMipCount = PreviousPassParameters->SkylightMipCount;
			}

			TArray<FLightShaderConstants> OptionalStationaryDirectionalLightShadowing;
			for (FDirectionalLightRenderState& DirectionalLight : Scene->LightSceneRenderState.DirectionalLights.Elements)
			{
				if (DirectionalLight.bStationary)
				{
					OptionalStationaryDirectionalLightShadowing.Add(DirectionalLight.GetLightShaderParameters());
					break;
				}
			}
			
			if (OptionalStationaryDirectionalLightShadowing.Num() == 0)
			{
				OptionalStationaryDirectionalLightShadowing.AddZeroed();
			}
			
			PassParameters->LightShaderParametersArray = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(CreateStructuredBuffer(
				GraphBuilder, TEXT("OptionalStationaryDirectionalLightShadowing"), sizeof(FLightShaderConstants),
				OptionalStationaryDirectionalLightShadowing.Num(),
				OptionalStationaryDirectionalLightShadowing.GetData(),
				sizeof(FLightShaderConstants) * OptionalStationaryDirectionalLightShadowing.Num()
			)));

			FSceneRenderState* SceneRenderState = Scene; // capture member variable
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("VolumetricLightmapPathTracing %d bricks %d rays", BricksToCalcThisFrame, BricksToCalcThisFrame * BrickSize * BrickSize * BrickSize),
				PassParameters,
				ERDGPassFlags::Compute,
				[
					PassParameters,
					RayGenShader,
					RayTracingPipelineState = Scene->RayTracingPipelineState,
					SBT = Scene->SBT,
					BricksToCalcThisFrame
				](FRHICommandList& RHICmdList)
				{
					FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
					SetShaderParameters(GlobalResources, RayGenShader, *PassParameters);

					RHICmdList.RayTraceDispatch(RayTracingPipelineState, RayGenShader.GetRayTracingShader(), SBT, GlobalResources,
					                            BricksToCalcThisFrame * BrickSize * BrickSize * BrickSize, 1);
				}
			);
		}
#endif

		FRDGTextureUAVRef ValidityBrickDataUAV = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(ValidityBrickData));
		
		// Output to OutputBrickData if this is the last pass for the current batch
		if (!bIsViewportNonRealtime || RenderPassIndex == NumTotalPassesToRender - 1)
		{
			TShaderMapRef<FFinalizeBrickResultsCS> ComputeShader(GlobalShaderMap);

			FFinalizeBrickResultsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFinalizeBrickResultsCS::FParameters>();
			PassParameters->NumTotalBricks = NumTotalBricks;
			PassParameters->BrickBatchOffset = BrickBatchOffset;
			PassParameters->NumTotalPassesToRender = NumTotalPassesToRender;
			
			if (Scene->Settings->bUseIrradianceCaching)
			{
				PassParameters->NumTotalPassesToRender -= Scene->Settings->IrradianceCacheQuality;	
			}
			
			PassParameters->BrickRequests = BrickRequests.UAV;
			PassParameters->AmbientVector = AccumulationBrickDataRDG.AmbientVectorRDG;
			PassParameters->SHCoefficients0R = AccumulationBrickDataRDG.SHCoefficientsRDG[0];
			PassParameters->SHCoefficients1R = AccumulationBrickDataRDG.SHCoefficientsRDG[1];
			PassParameters->SHCoefficients0G = AccumulationBrickDataRDG.SHCoefficientsRDG[2];
			PassParameters->SHCoefficients1G = AccumulationBrickDataRDG.SHCoefficientsRDG[3];
			PassParameters->SHCoefficients0B = AccumulationBrickDataRDG.SHCoefficientsRDG[4];
			PassParameters->SHCoefficients1B = AccumulationBrickDataRDG.SHCoefficientsRDG[5];
			PassParameters->SkyBentNormal = AccumulationBrickDataRDG.SkyBentNormalRDG;
			PassParameters->DirectionalLightShadowing = AccumulationBrickDataRDG.DirectionalLightShadowingRDG;
			PassParameters->OutAmbientVector = OutputBrickDataRDG.AmbientVector;
			PassParameters->OutSHCoefficients0R = OutputBrickDataRDG.SHCoefficients[0];
			PassParameters->OutSHCoefficients1R = OutputBrickDataRDG.SHCoefficients[1];
			PassParameters->OutSHCoefficients0G = OutputBrickDataRDG.SHCoefficients[2];
			PassParameters->OutSHCoefficients1G = OutputBrickDataRDG.SHCoefficients[3];
			PassParameters->OutSHCoefficients0B = OutputBrickDataRDG.SHCoefficients[4];
			PassParameters->OutSHCoefficients1B = OutputBrickDataRDG.SHCoefficients[5];
			PassParameters->OutSkyBentNormal = OutputBrickDataRDG.SkyBentNormal;
			PassParameters->OutDirectionalLightShadowing = OutputBrickDataRDG.DirectionalLightShadowing;
			PassParameters->ValidityMask = ValidityBrickDataUAV;

			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("FinalizeBrickResults"), ComputeShader, PassParameters, FIntVector(BricksToCalcThisFrame, 1, 1));
		}

		FRDGTextureUAV* IndirectionTextureUAV = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(IndirectionTexture));
		
		// Do some temporary copy and fill passes for preview
		if (!bIsViewportNonRealtime)
		{
			for (int32 MipLevel = VoxelizationVolumeMips.Num() - 1; MipLevel >= 0; MipLevel--)
			{
				// Copy padding strips from the neighbors of each brick
				TShaderMapRef<FCopyPaddingStripsCS> ComputeShader(GlobalShaderMap);

				FCopyPaddingStripsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCopyPaddingStripsCS::FParameters>();
				PassParameters->BrickDataDimensions = VolumetricLightmapData.BrickDataDimensions;
				PassParameters->IndirectionTextureDim = IndirectionTextureDimensions;
				PassParameters->FrameNumber = NumTotalPassesToRender;
				PassParameters->NumTotalBricks = NumTotalBricks;
				PassParameters->BrickBatchOffset = BrickBatchOffset;
				PassParameters->IndirectionTexture = IndirectionTextureUAV;
				PassParameters->MipLevel = MipLevel;
				PassParameters->VoxelizeVolume = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(VoxelizationVolumeMips[MipLevel]));
				PassParameters->ValidityMask = ValidityBrickDataUAV;
				PassParameters->BrickRequests = BrickRequests.UAV;
				PassParameters->OutAmbientVector = OutputBrickDataRDG.AmbientVector;
				PassParameters->OutSHCoefficients0R = OutputBrickDataRDG.SHCoefficients[0];
				PassParameters->OutSHCoefficients1R = OutputBrickDataRDG.SHCoefficients[1];
				PassParameters->OutSHCoefficients0G = OutputBrickDataRDG.SHCoefficients[2];
				PassParameters->OutSHCoefficients1G = OutputBrickDataRDG.SHCoefficients[3];
				PassParameters->OutSHCoefficients0B = OutputBrickDataRDG.SHCoefficients[4];
				PassParameters->OutSHCoefficients1B = OutputBrickDataRDG.SHCoefficients[5];
				PassParameters->OutSkyBentNormal = OutputBrickDataRDG.SkyBentNormal;
				PassParameters->OutDirectionalLightShadowing = OutputBrickDataRDG.DirectionalLightShadowing;

				FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("VolumetricLightmapCopyPaddingStrips %d bricks", BrickBatchSize), ComputeShader, PassParameters,
				                             FIntVector(BrickBatchSize, 1, 1));
			}

			if (RenderPassIndex == NumTotalPassesToRender - 1)
			{
				TShaderMapRef<FFillInvalidCellCS> ComputeShader(GlobalShaderMap);

				FFillInvalidCellCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFillInvalidCellCS::FParameters>();
				PassParameters->BrickDataDimensions = VolumetricLightmapData.BrickDataDimensions;
				PassParameters->IndirectionTextureDim = IndirectionTextureDimensions;
				PassParameters->FrameNumber = NumTotalPassesToRender;
				PassParameters->NumTotalBricks = NumTotalBricks;
				PassParameters->BrickBatchOffset = BrickBatchOffset;
				PassParameters->IndirectionTexture = IndirectionTextureUAV;
				PassParameters->ValidityMask = ValidityBrickDataUAV;
				PassParameters->BrickRequests = BrickRequests.UAV;
				PassParameters->OutAmbientVector = OutputBrickDataRDG.AmbientVector;
				PassParameters->OutSHCoefficients0R = OutputBrickDataRDG.SHCoefficients[0];
				PassParameters->OutSHCoefficients1R = OutputBrickDataRDG.SHCoefficients[1];
				PassParameters->OutSHCoefficients0G = OutputBrickDataRDG.SHCoefficients[2];
				PassParameters->OutSHCoefficients1G = OutputBrickDataRDG.SHCoefficients[3];
				PassParameters->OutSHCoefficients0B = OutputBrickDataRDG.SHCoefficients[4];
				PassParameters->OutSHCoefficients1B = OutputBrickDataRDG.SHCoefficients[5];
				PassParameters->OutSkyBentNormal = OutputBrickDataRDG.SkyBentNormal;
				PassParameters->OutDirectionalLightShadowing = OutputBrickDataRDG.DirectionalLightShadowing;

				FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("VolumetricLightmapFillInvalidCell %d bricks", BrickBatchSize), ComputeShader, PassParameters,
											 FIntVector(BrickBatchSize, 1, 1));
			}
		}

		// Do stitching over all brickes if this is the very last pass of all bricks
		if (SamplesTaken < NumTotalSamples && SamplesTaken + BricksToCalcThisFrame >= NumTotalSamples)
		{
			const int32 StitchingBrickBatchSize = 256;

			for (int32 MipLevel = VoxelizationVolumeMips.Num() - 1; MipLevel >= 0; MipLevel--)
			{
				// Copy padding strips from the neighbors of each brick
				for (int32 StitchingBrickBatchOffset = 0; StitchingBrickBatchOffset < NumTotalBricks; StitchingBrickBatchOffset += StitchingBrickBatchSize)
				{
					TShaderMapRef<FCopyPaddingStripsCS> ComputeShader(GlobalShaderMap);

					FCopyPaddingStripsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCopyPaddingStripsCS::FParameters>();
					PassParameters->BrickDataDimensions = VolumetricLightmapData.BrickDataDimensions;
					PassParameters->IndirectionTextureDim = IndirectionTextureDimensions;
					PassParameters->FrameNumber = NumTotalPassesToRender;
					PassParameters->NumTotalBricks = NumTotalBricks;
					PassParameters->BrickBatchOffset = StitchingBrickBatchOffset;
					PassParameters->IndirectionTexture = IndirectionTextureUAV;
					PassParameters->MipLevel = MipLevel;
					PassParameters->VoxelizeVolume = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(VoxelizationVolumeMips[MipLevel]));
					PassParameters->ValidityMask = ValidityBrickDataUAV;
					PassParameters->BrickRequests = BrickRequests.UAV;
					PassParameters->OutAmbientVector = OutputBrickDataRDG.AmbientVector;
					PassParameters->OutSHCoefficients0R = OutputBrickDataRDG.SHCoefficients[0];
					PassParameters->OutSHCoefficients1R = OutputBrickDataRDG.SHCoefficients[1];
					PassParameters->OutSHCoefficients0G = OutputBrickDataRDG.SHCoefficients[2];
					PassParameters->OutSHCoefficients1G = OutputBrickDataRDG.SHCoefficients[3];
					PassParameters->OutSHCoefficients0B = OutputBrickDataRDG.SHCoefficients[4];
					PassParameters->OutSHCoefficients1B = OutputBrickDataRDG.SHCoefficients[5];
					PassParameters->OutSkyBentNormal = OutputBrickDataRDG.SkyBentNormal;
					PassParameters->OutDirectionalLightShadowing = OutputBrickDataRDG.DirectionalLightShadowing;

					FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("VolumetricLightmapCopyPaddingStrips %d bricks", StitchingBrickBatchSize), ComputeShader, PassParameters,
												 FIntVector(StitchingBrickBatchSize, 1, 1));
				}
			}

			// Fill invalid cells by dilation
			for (int32 StitchingBrickBatchOffset = 0; StitchingBrickBatchOffset < NumTotalBricks; StitchingBrickBatchOffset += StitchingBrickBatchSize)
			{
				TShaderMapRef<FFillInvalidCellCS> ComputeShader(GlobalShaderMap);

				FFillInvalidCellCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFillInvalidCellCS::FParameters>();
				PassParameters->BrickDataDimensions = VolumetricLightmapData.BrickDataDimensions;
				PassParameters->IndirectionTextureDim = IndirectionTextureDimensions;
				PassParameters->FrameNumber = NumTotalPassesToRender;
				PassParameters->NumTotalBricks = NumTotalBricks;
				PassParameters->BrickBatchOffset = StitchingBrickBatchOffset;
				PassParameters->IndirectionTexture = IndirectionTextureUAV;
				PassParameters->ValidityMask = ValidityBrickDataUAV;
				PassParameters->BrickRequests = BrickRequests.UAV;
				PassParameters->OutAmbientVector = OutputBrickDataRDG.AmbientVector;
				PassParameters->OutSHCoefficients0R = OutputBrickDataRDG.SHCoefficients[0];
				PassParameters->OutSHCoefficients1R = OutputBrickDataRDG.SHCoefficients[1];
				PassParameters->OutSHCoefficients0G = OutputBrickDataRDG.SHCoefficients[2];
				PassParameters->OutSHCoefficients1G = OutputBrickDataRDG.SHCoefficients[3];
				PassParameters->OutSHCoefficients0B = OutputBrickDataRDG.SHCoefficients[4];
				PassParameters->OutSHCoefficients1B = OutputBrickDataRDG.SHCoefficients[5];
				PassParameters->OutSkyBentNormal = OutputBrickDataRDG.SkyBentNormal;
				PassParameters->OutDirectionalLightShadowing = OutputBrickDataRDG.DirectionalLightShadowing;

				FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("VolumetricLightmapFillInvalidCell %d bricks", StitchingBrickBatchSize), ComputeShader, PassParameters,
											 FIntVector(StitchingBrickBatchSize, 1, 1));
			}
			
			// Do 2 passes to propagate across 3 refinement levels
			for (int32 StitchPass = 0; StitchPass < 2; StitchPass++)
			{
				for (int32 StitchingBrickBatchOffset = 0; StitchingBrickBatchOffset < NumTotalBricks; StitchingBrickBatchOffset += StitchingBrickBatchSize)
				{
					TShaderMapRef<FStitchBorderCS> ComputeShader(GlobalShaderMap);

					FStitchBorderCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FStitchBorderCS::FParameters>();
					PassParameters->BrickDataDimensions = VolumetricLightmapData.BrickDataDimensions;
					PassParameters->IndirectionTextureDim = IndirectionTextureDimensions;
					PassParameters->FrameNumber = NumTotalPassesToRender;
					PassParameters->NumTotalBricks = NumTotalBricks;
					PassParameters->BrickBatchOffset = StitchingBrickBatchOffset;
					PassParameters->IndirectionTexture = IndirectionTextureUAV;
					PassParameters->ValidityMask = ValidityBrickDataUAV;
					PassParameters->BrickRequests = BrickRequests.UAV;
					PassParameters->OutAmbientVector = OutputBrickDataRDG.AmbientVector;
					PassParameters->OutSHCoefficients0R = OutputBrickDataRDG.SHCoefficients[0];
					PassParameters->OutSHCoefficients1R = OutputBrickDataRDG.SHCoefficients[1];
					PassParameters->OutSHCoefficients0G = OutputBrickDataRDG.SHCoefficients[2];
					PassParameters->OutSHCoefficients1G = OutputBrickDataRDG.SHCoefficients[3];
					PassParameters->OutSHCoefficients0B = OutputBrickDataRDG.SHCoefficients[4];
					PassParameters->OutSHCoefficients1B = OutputBrickDataRDG.SHCoefficients[5];
					PassParameters->OutSkyBentNormal = OutputBrickDataRDG.SkyBentNormal;
					PassParameters->OutDirectionalLightShadowing = OutputBrickDataRDG.DirectionalLightShadowing;

					FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("VolumetricLightmapStitching %d bricks", StitchingBrickBatchSize), ComputeShader, PassParameters,
												 FIntVector(StitchingBrickBatchSize, 1, 1));
				}
			}
		}

		SamplesTaken += BricksToCalcThisFrame;

		if (SamplesTaken >= NumTotalSamples)
		{
			break;
		}
	}

	AccumulationBrickDataRDG.QueueTextureExtractions(GraphBuilder);		
	OutputBrickDataRDG.QueueTextureExtractions(GraphBuilder);		
	
	GraphBuilder.Execute();
	
	AccumulationBrickDataRDG.DebugCheckInvariants();
	OutputBrickDataRDG.DebugCheckInvariants();

	if (IsRayTracingEnabled())
	{
		Scene->DestroyRayTracingScene();
	}
}

}
