// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "PrimitiveSceneInfo.h"
#include "PrecomputedVolumetricLightmap.h"
#include "Scene/Scene.h"

namespace GPULightmass
{

class FVolumetricLightmapRenderer
{
public:
	const int32 BrickBatchSize = 2048;
	
	FVolumetricLightmapRenderer(FSceneRenderState* Scene);

	void VoxelizeScene(FRHICommandListImmediate& RHICmdList);
	void BackgroundTick(FRHICommandListImmediate& RHICmdList);

	FPrecomputedVolumetricLightmap* GetPrecomputedVolumetricLightmapForPreview();

	float TargetDetailCellSize = 50.0f;
	int32 NumTotalBricks = 0;

	int32 FrameNumber = 0;
	int32 NumTotalPassesToRender = 0;
	uint64 SamplesTaken = 0;

private:
	FSceneRenderState* Scene;

	FPrecomputedVolumetricLightmap VolumetricLightmap;
	FPrecomputedVolumetricLightmapData VolumetricLightmapData;
	FVolumetricLightmapBrickData AccumulationBrickData;
	TRefCountPtr<IPooledRenderTarget> IndirectionTexture;

	FVector VolumeMin;
	FVector VolumeSize;
	FIntVector IndirectionTextureDimensions;

	TArray<TRefCountPtr<IPooledRenderTarget>> VoxelizationVolumeMips;

	FRWBuffer BrickAllocatorParameters;
	FRWBuffer BrickRequests;
	TRefCountPtr<IPooledRenderTarget> ValidityBrickData;
};

}
