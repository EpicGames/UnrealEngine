// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "Shader.h"
#include "ShaderCompilerCore.h"
#include "GlobalShader.h"
#include "MeshBatch.h"
#include "MaterialDomain.h"
#include "MaterialShaderType.h"
#include "MaterialShader.h"
#include "Materials/Material.h"
#include "MeshPassProcessor.h"
#include "MeshMaterialShader.h"
#include "MeshPassProcessor.inl"
#include "DataDrivenShaderPlatformInfo.h"

BEGIN_UNIFORM_BUFFER_STRUCT(FVLMVoxelizationParams, )
	SHADER_PARAMETER(FVector4f, VolumeCenter)
	SHADER_PARAMETER(FVector4f, VolumeExtent)
	SHADER_PARAMETER(int32, VolumeMaxDim)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, VoxelizeVolume)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint4>, IndirectionTexture)
END_UNIFORM_BUFFER_STRUCT()

typedef TUniformBufferRef<FVLMVoxelizationParams> FVLMVoxelizationUniformBufferRef;

bool IsSupportedVertexFactoryType(const FVertexFactoryType* VertexFactoryType);

class FVLMVoxelizationVS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FVLMVoxelizationVS, MeshMaterial);

protected:

	FVLMVoxelizationVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
	}

	FVLMVoxelizationVS()
	{
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData)
			&& IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5)
			&& IsSupportedVertexFactoryType(Parameters.VertexFactoryType)
			&& Parameters.MaterialParameters.bIsSpecialEngineMaterial;
	}
};

class FVLMVoxelizationGS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FVLMVoxelizationGS, MeshMaterial);

protected:

	FVLMVoxelizationGS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
	}

	FVLMVoxelizationGS()
	{
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData)
			&& IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5)
			&& IsSupportedVertexFactoryType(Parameters.VertexFactoryType)
			&& Parameters.MaterialParameters.bIsSpecialEngineMaterial;
	}
};

class FVLMVoxelizationPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FVLMVoxelizationPS, MeshMaterial);

protected:

	FVLMVoxelizationPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
	}

	FVLMVoxelizationPS()
	{
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData)
			&& IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5)
			&& IsSupportedVertexFactoryType(Parameters.VertexFactoryType)
			&& Parameters.MaterialParameters.bIsSpecialEngineMaterial;
	}
};

class FVLMVoxelizationMeshProcessor : public FMeshPassProcessor
{
public:
	FVLMVoxelizationMeshProcessor(const FScene* InScene, const FSceneView* InView, FMeshPassDrawListContext* InDrawListContext)
		: FMeshPassProcessor(TEXT("VLMVoxelization"), InScene, InView->GetFeatureLevel(), InView, InDrawListContext)
	{
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
		DrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
	}

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final
	{
		if (MeshBatch.bUseForMaterial && IsSupportedVertexFactoryType(MeshBatch.VertexFactory->GetType()))
		{
			const FMaterial& Material = MeshBatch.MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);

			if (IsOpaqueBlendMode(Material) || Material.IsMasked())
			{
				const FMaterialRenderProxy& DefaultProxy = *UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
				const FMaterial& DefaultMaterial = *DefaultProxy.GetMaterialNoFallback(FeatureLevel);

				Process(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, DefaultProxy, DefaultMaterial);
			}
		}
	}

private:
	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource)
	{
		const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<FVLMVoxelizationVS>();
		ShaderTypes.AddShaderType<FVLMVoxelizationGS>();
		ShaderTypes.AddShaderType<FVLMVoxelizationPS>();

		FMaterialShaders MaterialShaders;
		// We're using DefaultMaterial and it should never fail
		verify(MaterialResource.TryGetShaders(ShaderTypes, MeshBatch.VertexFactory->GetType(), MaterialShaders));
		
		TMeshProcessorShaders<
			FVLMVoxelizationVS,
			FVLMVoxelizationPS,
			FVLMVoxelizationGS> Shaders;

		MaterialShaders.TryGetVertexShader(Shaders.VertexShader);
		MaterialShaders.TryGetGeometryShader(Shaders.GeometryShader);
		MaterialShaders.TryGetPixelShader(Shaders.PixelShader);

		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(MaterialResource, OverrideSettings);
		ERasterizerCullMode MeshCullMode = CM_None;

		FMeshMaterialShaderElementData ShaderElementData;
		ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

		FMeshDrawCommandSortKey SortKey{};

		BuildMeshDrawCommands(
			MeshBatch,
			BatchElementMask,
			PrimitiveSceneProxy,
			MaterialRenderProxy,
			MaterialResource,
			DrawRenderState,
			Shaders,
			MeshFillMode,
			MeshCullMode,
			SortKey,
			EMeshPassFeatures::Default,
			ShaderElementData);

		return true;
	}

private:
	FMeshPassProcessorRenderState DrawRenderState;
};

class FClearVolumeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClearVolumeCS);
	SHADER_USE_PARAMETER_STRUCT(FClearVolumeCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, VolumeSize)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, VoxelizeVolume)
	END_SHADER_PARAMETER_STRUCT()
};

class FVoxelizeImportanceVolumeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVoxelizeImportanceVolumeCS);
	SHADER_USE_PARAMETER_STRUCT(FVoxelizeImportanceVolumeCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, VolumeSize)
		SHADER_PARAMETER(FVector3f, ImportanceVolumeMin)
		SHADER_PARAMETER(FVector3f, ImportanceVolumeMax)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, VoxelizeVolume)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVLMVoxelizationParams, VLMVoxelizationParams)
	END_SHADER_PARAMETER_STRUCT()
};

class FDownsampleVolumeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDownsampleVolumeCS);
	SHADER_USE_PARAMETER_STRUCT(FDownsampleVolumeCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, bIsHighestMip)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, VoxelizeVolumePrevMip)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, VoxelizeVolume)
	END_SHADER_PARAMETER_STRUCT()
};

class FCountNumBricksCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCountNumBricksCS);
	SHADER_USE_PARAMETER_STRUCT(FCountNumBricksCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, VolumeSize)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, VoxelizeVolume)
		SHADER_PARAMETER_UAV(RWBuffer<int>, BrickAllocatorParameters)
	END_SHADER_PARAMETER_STRUCT()
};

class FGatherBrickRequestsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGatherBrickRequestsCS);
	SHADER_USE_PARAMETER_STRUCT(FGatherBrickRequestsCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, VolumeSize)
		SHADER_PARAMETER(int32, BrickSize)
		SHADER_PARAMETER(int32, MipLevel)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, VoxelizeVolume)
		SHADER_PARAMETER_UAV(RWBuffer<int>, BrickAllocatorParameters)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint4>, BrickRequestsUnsorted)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, BrickRequestKeys)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, BrickRequestSortedIndices)
	END_SHADER_PARAMETER_STRUCT()
};

class FWriteSortedBrickRequestsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FWriteSortedBrickRequestsCS);
	SHADER_USE_PARAMETER_STRUCT(FWriteSortedBrickRequestsCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, NumTotalBricks)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint4>, BrickRequestsUnsorted)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, BrickRequestSortedIndices)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, BrickRequestSortedIndicesInverse)
		SHADER_PARAMETER_UAV(RWBuffer<uint4>, BrickRequests)
	END_SHADER_PARAMETER_STRUCT()
};

class FPermuteVoxelizeVolumeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPermuteVoxelizeVolumeCS);
	SHADER_USE_PARAMETER_STRUCT(FPermuteVoxelizeVolumeCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, VolumeSize)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, BrickRequestSortedIndicesInverse)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, VoxelizeVolume)
	END_SHADER_PARAMETER_STRUCT()
};

class FSplatVolumeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSplatVolumeCS);
	SHADER_USE_PARAMETER_STRUCT(FSplatVolumeCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, VolumeSize)
		SHADER_PARAMETER(int32, BrickSize)
		SHADER_PARAMETER(int32, bIsHighestMip)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, VoxelizeVolume)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint4>, IndirectionTexture)
		SHADER_PARAMETER_UAV(RWBuffer<int>, BrickAllocatorParameters)
	END_SHADER_PARAMETER_STRUCT()
};

class FStitchBorderCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FStitchBorderCS);
	SHADER_USE_PARAMETER_STRUCT(FStitchBorderCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, IndirectionTextureDim)
		SHADER_PARAMETER(FIntVector, BrickDataDimensions)
		SHADER_PARAMETER(uint32, FrameNumber)
		SHADER_PARAMETER(int32, NumTotalBricks)
		SHADER_PARAMETER(int32, BrickBatchOffset)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint4>, IndirectionTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, ValidityMask)
		SHADER_PARAMETER_UAV(RWBuffer<uint4>, BrickRequests)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, OutAmbientVector)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients0R)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients1R)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients0G)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients1G)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients0B)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients1B)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSkyBentNormal)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>, OutDirectionalLightShadowing)
	END_SHADER_PARAMETER_STRUCT()
};

class FCopyPaddingStripsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyPaddingStripsCS);
	SHADER_USE_PARAMETER_STRUCT(FCopyPaddingStripsCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, IndirectionTextureDim)
		SHADER_PARAMETER(FIntVector, BrickDataDimensions)
		SHADER_PARAMETER(uint32, FrameNumber)
		SHADER_PARAMETER(int32, MipLevel)
		SHADER_PARAMETER(int32, NumTotalBricks)
		SHADER_PARAMETER(int32, BrickBatchOffset)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint4>, IndirectionTexture)	
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, VoxelizeVolume)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, ValidityMask)
		SHADER_PARAMETER_UAV(RWBuffer<uint4>, BrickRequests)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, OutAmbientVector)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients0R)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients1R)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients0G)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients1G)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients0B)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients1B)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSkyBentNormal)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>, OutDirectionalLightShadowing)
	END_SHADER_PARAMETER_STRUCT()
};

class FFillInvalidCellCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FFillInvalidCellCS);
	SHADER_USE_PARAMETER_STRUCT(FFillInvalidCellCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, IndirectionTextureDim)
		SHADER_PARAMETER(FIntVector, BrickDataDimensions)
		SHADER_PARAMETER(uint32, FrameNumber)
		SHADER_PARAMETER(int32, NumTotalBricks)
		SHADER_PARAMETER(int32, BrickBatchOffset)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint4>, IndirectionTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, ValidityMask)
		SHADER_PARAMETER_UAV(RWBuffer<uint4>, BrickRequests)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, OutAmbientVector)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients0R)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients1R)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients0G)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients1G)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients0B)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients1B)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSkyBentNormal)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>, OutDirectionalLightShadowing)
	END_SHADER_PARAMETER_STRUCT()
};

class FFinalizeBrickResultsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FFinalizeBrickResultsCS);
	SHADER_USE_PARAMETER_STRUCT(FFinalizeBrickResultsCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShouldCompileRayTracingShadersForProject(Parameters.Platform) && IsPCPlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, NumTotalBricks)
		SHADER_PARAMETER(int32, BrickBatchOffset)
		SHADER_PARAMETER(int32, NumTotalPassesToRender)
		SHADER_PARAMETER_UAV(RWBuffer<uint4>, BrickRequests)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float4>, AmbientVector)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float4>, SHCoefficients0R)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float4>, SHCoefficients1R)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float4>, SHCoefficients0G)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float4>, SHCoefficients1G)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float4>, SHCoefficients0B)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float4>, SHCoefficients1B)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float4>, SkyBentNormal)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float>, DirectionalLightShadowing)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, OutAmbientVector)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients0R)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients1R)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients0G)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients1G)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients0B)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSHCoefficients1B)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutSkyBentNormal)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>, OutDirectionalLightShadowing)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, ValidityMask)
	END_SHADER_PARAMETER_STRUCT()
};
