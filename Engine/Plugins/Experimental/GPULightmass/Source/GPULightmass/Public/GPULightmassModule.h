// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Rendering/StaticLightingSystemInterface.h"

class FGPULightmassModule : public IModuleInterface, public IStaticLightingSystemImpl
{
public:
	void RunSelfTests();
	
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual IStaticLightingSystem* GetStaticLightingSystemForWorld(UWorld* InWorld) override;
	virtual bool IsStaticLightingSystemRunning() override;
	virtual void EditorTick() override;
	
	class FGPULightmass* CreateGPULightmassForWorld(UWorld* InWorld, class UGPULightmassSettings* Settings);
	void RemoveGPULightmassFromWorld(UWorld* InWorld);

	static bool GPULIGHTMASS_API IsRealtimeOn();

	// Due to limitations in our TMap implementation I cannot use TUniquePtr here
	// But the GPULightmassModule is the only owner of all static lighting systems, and all worlds weak refer to the systems
	TMap<UWorld*, class FGPULightmass*> StaticLightingSystems;

	FSimpleMulticastDelegate OnStaticLightingSystemsChanged;
};

DECLARE_LOG_CATEGORY_EXTERN(LogGPULightmass, Log, All);
