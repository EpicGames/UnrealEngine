// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Info.h"
#include "Subsystems/WorldSubsystem.h"
#include "GPULightmassSettings.generated.h"

UENUM()
enum class EGPULightmassMode : uint8
{
	FullBake,
	BakeWhatYouSee,
	// BakeSelected  UMETA(DisplayName = "Bake Selected (Not Implemented)")
};

UENUM()
enum class EGPULightmassDenoisingOptions : uint8
{
	None,
	OnCompletion,
	DuringInteractivePreview
};

UENUM()
enum class EGPULightmassDenoiser : uint8
{
	IntelOIDN  UMETA(DisplayName = "Intel Open Image Denoise"),
	SimpleFireflyRemover
};

UCLASS(BlueprintType)
class GPULIGHTMASS_API UGPULightmassSettings : public UObject
{
	GENERATED_BODY()

public:
	// If true, draw a green progress bar within each tile as it renders.
	// A red bar indicates that First Bounce Ray Guiding is in progress.
	// Bars may appear black in very bright scenes that have been exposed down.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General)
	bool bShowProgressBars = true;

	// Full Bake mode renders the full lightmap resolution for every object in the scene.
	// Bake What You See mode renders only the virtual texture tiles for objects in view,
	// at the mip level determined by the virtual texture system. The camera can be moved to render
	// more tiles. Bake What You See mode only saves its results if you press the Save button.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General)
	EGPULightmassMode Mode;

	// If enabled, denoise the results on the CPU after rendering. On Completion denoises the entire lightmap when it is finished.
	// During Interactive Preview denoises each tile as it finishes, which is useful for previewing but less efficient.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Denoising, DisplayName = "Denoise when")
	EGPULightmassDenoisingOptions DenoisingOptions = EGPULightmassDenoisingOptions::OnCompletion;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Denoising, meta = (EditCondition = "DenoisingOptions != EGPULightmassDenoisingOptions::None"))
	EGPULightmassDenoiser Denoiser = EGPULightmassDenoiser::IntelOIDN;

	// Whether to compress lightmap textures.  Disabling lightmap texture compression will reduce artifacts but increase memory and disk size by 4x.
	// Use caution when disabling this.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General)
	bool bCompressLightmaps = true;

	// Total number of ray paths executed per texel across all bounces.
	// Set this to the lowest value that gives artifact-free results with the denoiser.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GlobalIllumination, DisplayName = "GI Samples", meta = (ClampMin = "32", ClampMax = "65536", UIMax = "8192"))
	int32 GISamples = 512;

	// Number of samples for stationary shadows, which are calculated and stored separately from GI.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GlobalIllumination, meta = (ClampMin = "32", ClampMax = "65536", UIMax = "8192"))
	int32 StationaryLightShadowSamples = 128;

	// Irradiance Caching should be enabled with interior scenes to achieve more physically correct GI intensities,
	// albeit with some biasing. Without IC the results may be darker than expected. It should be disabled for exterior scenes.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GlobalIllumination)
	bool bUseIrradianceCaching = true;

	// If Irradiance Caching is enabled, First Bounce Ray Guiding will search the hemisphere over
	// each first bounce sample to find the brightest directions to weigh the rest of the samples towards.
	// This improves results for interior scenes with specific sources of light like a window.
	// The quality of this pass is controlled with the Trial Samples setting.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GlobalIllumination, meta = (EditCondition = "bUseIrradianceCaching"))
	bool bUseFirstBounceRayGuiding = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = VolumetricLightmap, DisplayName = "Quality Multiplier", meta = (ClampMin = "1", ClampMax = "256", UIMax = "32"))
	int32 VolumetricLightmapQualityMultiplier = 4;

	// Size of an Volumetric Lightmap voxel at the highest density (used around geometry), in world space units.
	// This setting has a large impact on build times and memory, use with caution.
	// Halving the DetailCellSize can increase memory by up to a factor of 8x.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = VolumetricLightmap, DisplayName = "Detail Cell Size", meta = (ClampMin = "1", ClampMax = "20000", UIMax = "2000"))
	int32 VolumetricLightmapDetailCellSize = 200;

	// Number of samples per Irradiance Cache cell.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = IrradianceCaching, DisplayName = "Quality", meta = (EditCondition = "bUseIrradianceCaching", ClampMin = "4", ClampMax = "65536", UIMax = "8192"))
	int32 IrradianceCacheQuality = 128;
	
	// Further prevent leaks caused by irradiance cache cells being placed inside geometry, at the cost of more fireflies and slower sampling speed. Recommended to be used with higher irradiance cache quality (>=256)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = IrradianceCaching, DisplayName = "Aggressive Leak Prevention", meta = (EditCondition = "bUseIrradianceCaching"))
	bool bUseIrradianceCacheBackfaceDetection = false;

	// Size of each Irradiance Cache cell. Smaller sizes will be slower but more accurate.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = IrradianceCaching, DisplayName = "Size", meta = (EditCondition = "bUseIrradianceCaching", ClampMin = "4", ClampMax = "1024"))
	int32 IrradianceCacheSpacing = 32;

	// Reject IC entries around corners to help reduce leaking and artifacts.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = IrradianceCaching, DisplayName = "Corner Rejection", meta = (EditCondition = "bUseIrradianceCaching", ClampMin = "0.0", ClampMax = "8.0"))
	float IrradianceCacheCornerRejection = 1.0f;

	// If true, visualize the Irradiance Cache cells. This can be useful for setting the IC size and quality.
	// The visualization may appear black in very bright scenes that have been exposed down. 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = IrradianceCaching, DisplayName = "Debug: Visualize", meta = (EditCondition = "bUseIrradianceCaching"))
	bool bVisualizeIrradianceCache = false;

	// Number of samples used for First Bounce Ray Guiding, which are thrown away before sampling for lighting.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FirstBounceRayGuiding, DisplayName = "Trial Samples", meta = (EditCondition = "bUseFirstBounceRayGuiding"))
	int32 FirstBounceRayGuidingTrialSamples = 128;

	// Baking speed multiplier when Realtime is enabled in the viewer.
	// Setting this too high can cause the editor to become unresponsive with heavy scenes.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = System, DisplayName = "Realtime Workload Factor", meta = (ClampMin = "1", ClampMax = "1024"))
	int32 TilePassesInSlowMode = 1;

	// Baking speed multiplier when Realtime is disabled in the viewer.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = System, DisplayName = "Non-Realtime Workload Factor", meta = (ClampMin = "1", ClampMax = "1024"))
	int32 TilePassesInFullSpeedMode = 8;

	// GPU Lightmass manages a pool for calculations of visible tiles. The pool size should be set based on the size of the
	// viewport and how many tiles will be visible on screen at once. Increasing this number increases GPU memory usage.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = System, meta = (ClampMin = "16", ClampMax = "128"))
	int32 LightmapTilePoolSize = 55;

public:
	void ApplyImmediateSettingsToRunningInstances();

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif // WITH_EDITOR
};

UCLASS(NotPlaceable)
class GPULIGHTMASS_API AGPULightmassSettingsActor : public AInfo
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	virtual bool ActorTypeIsMainWorldOnly() const override { return true; }
#endif

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TObjectPtr<UGPULightmassSettings> Settings;
};

UCLASS()
class GPULIGHTMASS_API UGPULightmassSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()
public:
	virtual void PostInitialize() override;

	UFUNCTION(BlueprintCallable, Category = GPULightmass)
	UGPULightmassSettings* GetSettings();

	UFUNCTION(BlueprintCallable, Category = GPULightmass)
	void Launch();

	UFUNCTION(BlueprintCallable, Category = GPULightmass)
	void Stop();

	UFUNCTION(BlueprintCallable, Category = GPULightmass)
	bool IsRunning();

	UFUNCTION(BlueprintCallable, Category = GPULightmass)
	void StartRecordingVisibleTiles();

	UFUNCTION(BlueprintCallable, Category = GPULightmass)
	void EndRecordingVisibleTiles();

	UFUNCTION(BlueprintCallable, Category = GPULightmass)
	int32 GetPercentage();

	UFUNCTION(BlueprintCallable, Category = GPULightmass)
	void SetRealtime(bool bInRealtime);

	UFUNCTION(BlueprintCallable, Category = GPULightmass)
	void Save();

	/* Accessor for the delegate called when the light build finishes successfully or is cancelled */
	FSimpleMulticastDelegate& OnLightBuildEnded()
	{
		return LightBuildEnded;
	}

private:
	AGPULightmassSettingsActor* GetSettingsActor();

private:
	/* Called when the light build finishes successfully or is cancelled */
	FSimpleMulticastDelegate LightBuildEnded;
};
