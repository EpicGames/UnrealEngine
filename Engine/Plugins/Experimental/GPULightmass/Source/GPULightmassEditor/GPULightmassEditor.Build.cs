// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GPULightmassEditor : ModuleRules
	{
		public GPULightmassEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"CoreUObject",
					"Engine",
					"Landscape",
					"RenderCore",
					"Renderer",
					"RHI",
					"Projects",
					"UnrealEd",
					"SlateCore",
					"Slate",
					"GPULightmass",
					"WorkspaceMenuStructure",
					"EditorStyle",
					"ToolMenus",
					"ToolWidgets",
				});
		}
	}
}
