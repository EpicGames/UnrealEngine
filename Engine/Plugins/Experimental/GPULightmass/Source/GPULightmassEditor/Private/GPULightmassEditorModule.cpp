// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPULightmassEditorModule.h"
#include "CoreMinimal.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"
#include "HAL/IConsoleManager.h"
#include "Containers/StaticBitArray.h"
#include "PropertyEditorModule.h"
#include "RenderingThread.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Interfaces/IPluginManager.h"
#include "ShaderCore.h"
#include "SceneInterface.h"
#include "LevelEditor.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "GPULightmassSettings.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "EditorFontGlyphs.h"
#include "GPULightmassModule.h"
#include "LevelEditorViewport.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "ToolMenus.h"
#include "SNegativeActionButton.h"
#include "SPrimaryButton.h"
#include "Widgets/Images/SImage.h"
#include "RenderUtils.h"

#define LOCTEXT_NAMESPACE "StaticLightingSystem"

IMPLEMENT_MODULE( FGPULightmassEditorModule, GPULightmassEditor )

FName GPULightmassSettingsTabName = TEXT("GPULightmassSettings");

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

static FAutoConsoleCommand GPULMBuildLighting(
	TEXT("GPULM.BuildLighting"),
	TEXT("Starts lighting build with GPU Lightmass"),
	FConsoleCommandDelegate::CreateStatic([] {
		FGPULightmassEditorModule* LightmassModule = FModuleManager::Get().GetModulePtr<FGPULightmassEditorModule>("GPULightmass");
		if (LightmassModule)
		{
			if (UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				if (!World->GetSubsystem<UGPULightmassSubsystem>()->IsRunning())
				{					
					World->GetSubsystem<UGPULightmassSubsystem>()->Launch();
				}
			}
		}		
}));

static bool IsRenderDocPresent()
{
#if PLATFORM_WINDOWS
	// Run only once as this func is called each time the UI is rendered
	static HMODULE RenderDocDLLHandle = GetModuleHandleW(L"renderdoc.dll");
	return RenderDocDLLHandle != nullptr;
#else
	// TODO: other platforms
	return false;
#endif
}

static bool IsCurrentRHIRayTracingCapable()
{
	switch(RHIGetInterfaceType())
	{
	case ERHIInterfaceType::D3D12:
	case ERHIInterfaceType::Vulkan:
		return true;
	default:
		return false;
	}
}

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

enum class ERayTracingDisabledReason : int32
{
	OK, // Ray tracing is supported and running
	DISABLED_BY_PROJECT_SETTINGS, // "Ray Tracing" or "Support Compute Skin Cache" is off in project settings
	DISABLED_BY_TARGET_PLATFORM, // FXXXTargetPlatform::UsesRayTracing() returns false (eg. WindowsTargetPlatform checks for bEnableRayTracing in its ini)
	INCOMPATIBLE_SHADER_PLATFORM, // An incompatible shader platform is chosen (eg. ES3.1)
	INCAPABLE_RHI, // The RHI is incapable of ray tracing in general (eg. DX11)
	INCOMPATIBLE_PLUGIN, // An incompatible plugin is enabled (specifically, RenderDoc)
	INCAPABLE_HARDWARE, // The video card isn't capable of hardware ray tracing (too old), or supports only software emulated ray tracing (GTX 10 series)
};

using ShaderPlatformMaskType = TStaticBitArray<EShaderPlatform::SP_NumPlatforms>;
extern RENDERCORE_API ShaderPlatformMaskType GRayTracingPlatformMask;

ERayTracingDisabledReason GetRayTracingDisabledReason(EShaderPlatform ShaderPlatform)
{
	if (IsRayTracingEnabled(ShaderPlatform))
	{
		return ERayTracingDisabledReason::OK;
	}
	
	static IConsoleVariable* RayTracingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing"));
	if (RayTracingCVar->GetInt() == 0)
	{
		return ERayTracingDisabledReason::DISABLED_BY_PROJECT_SETTINGS;
	}

	if (!RHISupportsRayTracing(ShaderPlatform))
	{
		return ERayTracingDisabledReason::INCOMPATIBLE_SHADER_PLATFORM;
	}
	// Shader platform statically supports ray tracing, but disabled by target platform at runtime
	else if (RHISupportsRayTracing(ShaderPlatform) && (GRayTracingPlatformMask[(int)ShaderPlatform]) == 0)
	{
		return ERayTracingDisabledReason::DISABLED_BY_TARGET_PLATFORM;
	}
	
	if (!IsCurrentRHIRayTracingCapable())
	{
		return ERayTracingDisabledReason::INCAPABLE_RHI;
	}

	if (IsRenderDocPresent())
	{
		return ERayTracingDisabledReason::INCOMPATIBLE_PLUGIN;
	}

	// TODO: better determination of reasons instead of pouring everything into this bucket 
	return ERayTracingDisabledReason::INCAPABLE_HARDWARE;
}

ERayTracingDisabledReason GetRayTracingDisabledReason()
{
	ERayTracingDisabledReason RayTracingStatus = ERayTracingDisabledReason::INCAPABLE_HARDWARE;
	if (UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		if (World->Scene)
		{
			RayTracingStatus = GetRayTracingDisabledReason(World->Scene->GetShaderPlatform());
		}
	}
	return RayTracingStatus;
}

static FText GenerateRayTracingDisabledReasonMessage(ERayTracingDisabledReason Reason)
{
	switch (Reason)
	{
	case ERayTracingDisabledReason::DISABLED_BY_PROJECT_SETTINGS:
		return LOCTEXT("GPULightmassHWRayTracingDisabled",
		               "GPU Lightmass requires 'Support Hardware Ray Tracing' enabled in your project settings.");
	case ERayTracingDisabledReason::DISABLED_BY_TARGET_PLATFORM:
	case ERayTracingDisabledReason::INCOMPATIBLE_SHADER_PLATFORM:
		return LOCTEXT("GPULightmassHWRayTracingDisabledByPlatformSettings",
		               "GPU Lightmass requires hardware ray tracing which is disabled by some of your project settings (an incompatible shader platform (eg. ES3.1) is enabled and active, or disabled on your current target platform).");
	case ERayTracingDisabledReason::INCAPABLE_RHI:
		return LOCTEXT("GPULightmassHWRayTracingDisabledRHI",
		               "GPU Lightmass requires hardware ray tracing which is not supported by the current RHI.");
	case ERayTracingDisabledReason::INCOMPATIBLE_PLUGIN:
		return LOCTEXT("GPULightmassHWRayTracingDisabledPossiblyByRenderDoc",
		               "GPU Lightmass requires hardware ray tracing support which is disabled by an incompatible plugin, possibly RenderDoc.");
	case ERayTracingDisabledReason::INCAPABLE_HARDWARE:
		return LOCTEXT("GPULightmassHWRayTracingDisabledGPU",
		               "GPU Lightmass requires hardware ray tracing which isn't supported by your GPU.");
	default:
		return FText();
	}
}

static bool IsPathTracingEnabled()
{
	static const auto CVarPathTracing = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.PathTracing"));
	return CVarPathTracing && CVarPathTracing->GetValueOnAnyThread() > 0;
}

void FGPULightmassEditorModule::StartupModule()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	LevelEditorModule.OnTabManagerChanged().AddRaw(this, &FGPULightmassEditorModule::RegisterTabSpawner);
	LevelEditorModule.OnMapChanged().AddRaw(this, &FGPULightmassEditorModule::OnMapChanged);
	auto BuildMenuExtender = FLevelEditorModule::FLevelEditorMenuExtender::CreateRaw(this, &FGPULightmassEditorModule::OnExtendLevelEditorBuildMenu);
	LevelEditorModule.GetAllLevelEditorToolbarBuildMenuExtenders().Add(BuildMenuExtender);

	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Build");
	FToolMenuSection& Section = Menu->FindOrAddSection("LevelEditorLighting");

	FUIAction ActionOpenGPULightmassSettingsTab(FExecuteAction::CreateLambda([]() {
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
		LevelEditorTabManager->TryInvokeTab(GPULightmassSettingsTabName);
	}), FCanExecuteAction());

	FToolMenuEntry& Entry = Section.AddMenuEntry(NAME_None, LOCTEXT("GPULightmassSettingsTitle", "GPU Lightmass"),
		LOCTEXT("OpensGPULightmassSettings", "Opens GPU Lightmass settings tab."), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Level.LightingScenarioIcon16x"), ActionOpenGPULightmassSettingsTab,
		EUserInterfaceActionType::Button);


}

void FGPULightmassEditorModule::ShutdownModule()
{
}

void FGPULightmassEditorModule::RegisterTabSpawner()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

	LevelEditorTabManager->RegisterTabSpawner(GPULightmassSettingsTabName, FOnSpawnTab::CreateRaw(this, &FGPULightmassEditorModule::SpawnSettingsTab))
		.SetDisplayName(LOCTEXT("GPULightmassSettingsTitle", "GPU Lightmass"))
		//.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Level.LightingScenarioIcon16x"))
		.SetMenuType(ETabSpawnerMenuType::Hidden);
}

TSharedRef<SDockTab> FGPULightmassEditorModule::SpawnSettingsTab(const FSpawnTabArgs& Args)
{
	FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.NotifyHook = GUnrealEd;

	SettingsView = PropPlugin.CreateDetailView(DetailsViewArgs);

	if (UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		if (World->GetSubsystem<UGPULightmassSubsystem>())
		{
			SettingsView->SetObject(World->GetSubsystem<UGPULightmassSubsystem>()->GetSettings());
		}
	}

	return SNew(SDockTab)
		.Label(NSLOCTEXT("GPULightmass", "GPULightmassSettingsTabTitle", "GPU Lightmass"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(12.0f, 8.0f)
			[
				// Start Build
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 8.f, 0.0f)
				[
					SNew(SPrimaryButton)
					.Icon(FAppStyle::Get().GetBrush("EditorViewport.LightingOnlyMode"))
					.Text_Lambda( []()
					{ 
						return FGPULightmassEditorModule::IsBakeWhatYouSeeMode() ? 
								LOCTEXT("GPULightmassSettingsStartInteractive", "Start Building Lighting") : 
								LOCTEXT("GPULightmassSettingsStartFull", "Build Lighting");
					})
					.IsEnabled_Lambda( []() {
						return (GetRayTracingDisabledReason() == ERayTracingDisabledReason::OK) && IsStaticLightingAllowed() && IsPathTracingEnabled();
					}) 
					.Visibility_Lambda([](){ return IsRunning() ? EVisibility::Collapsed : EVisibility::Visible; })
					.OnClicked_Raw(this, &FGPULightmassEditorModule::OnStartClicked)
				]

				// Save and Stop Building
				+SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 8.f, 0.0f)
				.AutoWidth()
				[
					SNew(SPrimaryButton)
					.Icon(FAppStyle::Get().GetBrush("EditorViewport.LightingOnlyMode"))
					.Text(LOCTEXT("GPULightmassSettingsSaveAndStop", "Save And Stop Building"))
					.Visibility_Lambda([](){ return IsRunning() && IsBakeWhatYouSeeMode() ? EVisibility::Visible : EVisibility::Collapsed; })
					.OnClicked_Raw(this, &FGPULightmassEditorModule::OnSaveAndStopClicked)
				]

				// Cancel Build
				+SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 8.f, 0.0f)
				.AutoWidth()
				[
					SNew(SNegativeActionButton)
					.Visibility_Lambda([](){ return IsRunning() ? EVisibility::Visible: EVisibility::Collapsed; })
					.OnClicked_Raw(this, &FGPULightmassEditorModule::OnCancelClicked)
					.Text(LOCTEXT("GPULightmassSettingsCancel", "Cancel Build"))
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.ToolTipText(LOCTEXT("GPULightmassSettingsRealtimeToggleTooltip", "GPU Lightmass runs in slow mode when the viewport is realtime to avoid freezing. Uncheck Viewport Realtime to get full speed."))
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.Padding(0, 0, 2, 0)
						[
							SNew(SCheckBox)
							.IsChecked_Lambda( [] () { return FGPULightmassEditorModule::IsRealtimeOn() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
							.OnCheckStateChanged_Lambda( [] (ECheckBoxState NewState) 
							{
								if (GCurrentLevelEditingViewportClient)
								{
									GCurrentLevelEditingViewportClient->SetRealtime( NewState == ECheckBoxState::Checked );
								}
							})
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.Padding(0, 0, 4, 0)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush(TEXT("EditorViewport.ToggleRealTime")))
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.Padding(0, 0, 2, 0)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("GPULightmassRealtimeEnabled", "Realtime Viewport"))
						]
					]

				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(12.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				.Padding(0, 0, 4, 0)
				[
					SNew(SImage)
					.Image_Lambda([this]() -> const FSlateBrush*
					{
						// Error
						if (Message.Severity <= ELogVerbosity::Error)
						{
							return FAppStyle::GetBrush(TEXT("Icons.ErrorWithColor"));
						}
						
						// Warning
						if (Message.Severity == ELogVerbosity::Warning)
						{
							return FAppStyle::GetBrush(TEXT("Icons.WarningWithColor"));
						}
						
						// Info
						return FAppStyle::GetBrush(TEXT("Icons.InfoWithColor"));
					})
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Fill)
				.Padding(0, 0, 2, 0)
				[
					SAssignNew(MessageWidget, STextBlock)
					.AutoWrapText(true)
					.Text_Lambda( [this]() -> FText
					{
						Message.Severity = ELogVerbosity::Error; // Default to error
						
						ERayTracingDisabledReason RayTracingStatus = GetRayTracingDisabledReason();
						if (RayTracingStatus != ERayTracingDisabledReason::OK)
						{
							Message.Text = GenerateRayTracingDisabledReasonMessage(RayTracingStatus);
							return Message.Text;
						}
						else if (!IsStaticLightingAllowed())
						{
							Message.Text = LOCTEXT("GPULightmassAllowStaticLightingDisabled", "GPU Lightmass requires Allow Static Lighting enabled in the project settings.");
							return Message.Text;
						}
						else if (!IsPathTracingEnabled())
						{
							Message.Text = LOCTEXT("GPULightmassPathTracingDisabled", "GPU Lightmass requires Path Tracing enabled in the project settings.");
							return Message.Text;
						}

						// Message is info unless issue found later
						Message.Severity = ELogVerbosity::Display;

						// Ready
						static FText ReadyMsg = FText(LOCTEXT("GPULightmassReady", "GPU Lightmass is ready."));

						// Ready, BWYS
						static FText BWYSReadyMsg = FText(LOCTEXT("GPULightmassReadyBWYS", "GPU Lightmass is ready. Lighting will rebuild continuously in Bake What You See mode until saved or canceled."));

						// Ready, BWYS+RT OFF Warning 
						static FText RtOffBWYSWarningMsg = LOCTEXT("GPULightmassSpeedReadyRTWarning", "Building Lighting when using Bake What You See Mode will automatically enable Viewport Realtime to start building. Lighting will rebuild continuously in Bake What You See mode until saved or canceled.");

						// Building FULL + RT Off Warning
						UWorld* World = GEditor->GetEditorWorldContext().World();
						FText BuildingMsg = FText::Format(LOCTEXT("GPULightmassBuildingLighting", "GPU Lightmass is building lighting for {0}."), FText::FromString(World->GetActiveLightingScenario() ? World->GetActiveLightingScenario()->GetOuter()->GetName() : World->GetName()));

						// Building FULL + RT ON Warning 
						static FText BuildingFullRTOnMsg = LOCTEXT("GPULightmassBuildingFullRTOn", "GPU Lightmass runs in slow mode when the viewport is realtime to avoid freezing. Uncheck Viewport Realtime to get full speed.");

						// Building BWYS + RT ON Warning 
						static FText BuildingRTOnMsg = LOCTEXT("GPULightmassBuildingInteractiveRTOn", "Disable Viewport Realtime to speed up building.");

						// Building BWYS + RT OFF Warning 
						static FText BuidlingRTOffMsg = LOCTEXT("GPULightmassBuildingInteractiveRTOff", "Re-enable Viewport Realtime to preview lighting.  Enabling Viewport Realtime will slow down building, to avoid freezing.");

						bool bIsRunning = IsRunning();
						bool bIsInteractive = IsBakeWhatYouSeeMode();
						bool bIsRealtime = IsRealtimeOn();
						if (bIsRunning)
						{
							if (bIsInteractive)
							{
								// Both messages below are considered warnings
								Message.Severity = ELogVerbosity::Warning;
								Message.Text = bIsRealtime ? BuildingRTOnMsg : BuidlingRTOffMsg;
								return Message.Text;
							}

							Message.Severity = bIsRealtime ? ELogVerbosity::Warning : ELogVerbosity::Display;
							Message.Text = bIsRealtime ? BuildingFullRTOnMsg : BuildingMsg;
							return Message.Text;
						}
						else if (bIsInteractive)
						{
							// Both messages below are considered warnings
							Message.Severity = ELogVerbosity::Warning;
							Message.Text = bIsRealtime ?  BWYSReadyMsg : RtOffBWYSWarningMsg;
							return Message.Text;
						}

						Message.Severity = bIsRealtime ? ELogVerbosity::Warning : ELogVerbosity::Display;
						Message.Text = bIsRealtime ? BuildingFullRTOnMsg : ReadyMsg;
						return Message.Text;
					})
				]
			]

			+ SVerticalBox::Slot()
			[
				SettingsView.ToSharedRef()
			]
		];
}

void FGPULightmassEditorModule::UpdateSettingsTab()
{
	if (SettingsView.IsValid())
	{
		SettingsView->ForceRefresh();
	}
}

bool FGPULightmassEditorModule::IsBakeWhatYouSeeMode()
{
	if (UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		if (UGPULightmassSubsystem* LMSubsystem = World->GetSubsystem<UGPULightmassSubsystem>())
		{
			return LMSubsystem->GetSettings()->Mode == EGPULightmassMode::BakeWhatYouSee;
		}
	}

	return false;
}

bool FGPULightmassEditorModule::IsRealtimeOn()
{
	return FGPULightmassModule::IsRealtimeOn();
}

bool FGPULightmassEditorModule::IsRunning() 
{
	if (UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		return World->GetSubsystem<UGPULightmassSubsystem>()->IsRunning();
	}

	return false;
}

FReply FGPULightmassEditorModule::OnStartClicked()
{
	if (UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		if (!World->GetSubsystem<UGPULightmassSubsystem>()->IsRunning())
		{
			if (IsBakeWhatYouSeeMode() && !IsRealtimeOn() && GCurrentLevelEditingViewportClient != nullptr)
			{
				GCurrentLevelEditingViewportClient->SetRealtime(true);
			}

			World->GetSubsystem<UGPULightmassSubsystem>()->Launch();
			World->GetSubsystem<UGPULightmassSubsystem>()->OnLightBuildEnded().AddRaw(this, &FGPULightmassEditorModule::UpdateSettingsTab);
		}
	}

	UpdateSettingsTab();

	return FReply::Handled();
}


FReply FGPULightmassEditorModule::OnSaveAndStopClicked()
{
	if (UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		if (World->GetSubsystem<UGPULightmassSubsystem>()->IsRunning())
		{
			World->GetSubsystem<UGPULightmassSubsystem>()->Save();
			World->GetSubsystem<UGPULightmassSubsystem>()->Stop();
			World->GetSubsystem<UGPULightmassSubsystem>()->OnLightBuildEnded().RemoveAll(this);
		}
	}

	UpdateSettingsTab();

	return FReply::Handled();
}

FReply FGPULightmassEditorModule::OnCancelClicked()
{

	if (UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		if (World->GetSubsystem<UGPULightmassSubsystem>()->IsRunning())
		{
			World->GetSubsystem<UGPULightmassSubsystem>()->Stop();
			World->GetSubsystem<UGPULightmassSubsystem>()->OnLightBuildEnded().RemoveAll(this);
		}
	}
	
	UpdateSettingsTab();

	return FReply::Handled();
}

void FGPULightmassEditorModule::OnMapChanged(UWorld* InWorld, EMapChangeType MapChangeType)
{
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (SettingsView.IsValid())
	{
		if (World->GetSubsystem<UGPULightmassSubsystem>())
		{
			SettingsView->SetObject(World->GetSubsystem<UGPULightmassSubsystem>()->GetSettings(), true);

			if (MapChangeType == EMapChangeType::LoadMap || MapChangeType == EMapChangeType::NewMap)
			{
				World->GetSubsystem<UGPULightmassSubsystem>()->OnLightBuildEnded().AddRaw(this, &FGPULightmassEditorModule::UpdateSettingsTab);
			}
			else if (MapChangeType == EMapChangeType::TearDownWorld)
			{
				World->GetSubsystem<UGPULightmassSubsystem>()->OnLightBuildEnded().RemoveAll(this);
			}
		}
	}
}

TSharedRef<FExtender> FGPULightmassEditorModule::OnExtendLevelEditorBuildMenu(const TSharedRef<FUICommandList> CommandList)
{
	TSharedRef<FExtender> Extender(new FExtender());

	Extender->AddMenuExtension("LevelEditorLighting", EExtensionHook::First, nullptr,
		FMenuExtensionDelegate::CreateRaw(this, &FGPULightmassEditorModule::CreateBuildMenu));

	return Extender;
}

void FGPULightmassEditorModule::CreateBuildMenu(FMenuBuilder& Builder)
{
	FUIAction ActionOpenGPULightmassSettingsTab(FExecuteAction::CreateLambda([]() {
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
		LevelEditorTabManager->TryInvokeTab(GPULightmassSettingsTabName);
	}), FCanExecuteAction());

	Builder.AddMenuEntry(LOCTEXT("GPULightmassSettingsTitle", "GPU Lightmass"),
		LOCTEXT("OpensGPULightmassSettings", "Opens GPU Lightmass settings tab."), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Level.LightingScenarioIcon16x"), ActionOpenGPULightmassSettingsTab,
		NAME_None, EUserInterfaceActionType::Button);
}

#undef LOCTEXT_NAMESPACE
