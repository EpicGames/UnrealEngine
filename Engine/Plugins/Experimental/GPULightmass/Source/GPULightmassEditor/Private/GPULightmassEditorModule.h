// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "TickableEditorObject.h"
#include "Input/Reply.h"

class FExtender;
class FMenuBuilder;
class FSpawnTabArgs;
class FUICommandList;
class IDetailsView;
class SDockTab;
class STextBlock;

enum class EMapChangeType : uint8;

class FGPULightmassEditorModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	TSharedPtr<IDetailsView> SettingsView;
	TSharedRef<SDockTab> SpawnSettingsTab(const FSpawnTabArgs& Args);	
	void UpdateSettingsTab();
	void RegisterTabSpawner();
	void OnMapChanged(UWorld* InWorld, EMapChangeType MapChangeType);

	TSharedRef<FExtender> OnExtendLevelEditorBuildMenu(const TSharedRef<FUICommandList> CommandList);
	void CreateBuildMenu(FMenuBuilder& Builder);

	static bool IsRealtimeOn();
	static bool IsRunning();
	static bool IsBakeWhatYouSeeMode();

	FReply OnStartClicked();
	FReply OnSaveAndStopClicked();
	FReply OnCancelClicked();

private:
	struct FMessage
	{
		ELogVerbosity::Type Severity;
		FText Text;
	};

	FMessage Message = {};
	TSharedPtr<STextBlock> MessageWidget;
};
